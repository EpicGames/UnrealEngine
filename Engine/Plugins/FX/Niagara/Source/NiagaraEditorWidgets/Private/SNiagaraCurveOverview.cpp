// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraCurveOverview.h"

#include "EditorFontGlyphs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraDataInterfaceCurveBase.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "RichCurveEditorModel.h"
#include "SCurveEditorPanel.h"
#include "Misc/CurveEditorIntegrationUtils.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/SCurveEditorTreeFilterStatusBar.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Tree/SCurveEditorTreeTextFilter.h"
#include "ViewModels/NiagaraCurveSelectionViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/SNiagaraParameterName.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraCurveEditor"

struct FNiagaraCurveOverviewTreeItem : public ICurveEditorTreeItem
{
	FNiagaraCurveOverviewTreeItem(TSharedRef<FNiagaraCurveSelectionTreeNode> InCurveSelectionTreeNode)
		: CurveSelectionTreeNode(InCurveSelectionTreeNode)
	{
	}

	TSharedRef<FNiagaraCurveSelectionTreeNode> GetCurveSelectionTreeNode()
	{
		return CurveSelectionTreeNode;
	}

	static void GetWidgetsforNode(TSharedRef<FNiagaraCurveSelectionTreeNode> InCurveSelectionTreeNode, TSharedPtr<SWidget>& OutIconWidget, TSharedPtr<SWidget>& OutDisplayNameWidget, TSharedPtr<SWidget>& OutSecondaryIconWidget, TSharedPtr<SWidget>& OutSecondaryDisplayName)
	{
		switch (InCurveSelectionTreeNode->GetStyleMode())
		{
			case ENiagaraCurveSelectionNodeStyleMode::TopLevelObject:
			{
				OutIconWidget =
					SNew(SImage)
					.Image(FNiagaraEditorWidgetsStyle::Get().GetBrush(FNiagaraStackEditorWidgetsUtilities::GetIconNameForExecutionSubcategory(InCurveSelectionTreeNode->GetExecutionSubcategory(), false)))
					.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(InCurveSelectionTreeNode->GetExecutionCategory())));

				OutDisplayNameWidget = 
					SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.TopLevelText");

				OutSecondaryDisplayName = SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetSecondDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.SecondaryText")
					.ColorAndOpacity(FSlateColor::UseSubduedForeground());

				break;
			}
			case ENiagaraCurveSelectionNodeStyleMode::Script:
			{
				OutIconWidget = 
					SNew(SImage)
					.Image(FNiagaraEditorWidgetsStyle::Get().GetBrush(FNiagaraStackEditorWidgetsUtilities::GetIconNameForExecutionSubcategory(InCurveSelectionTreeNode->GetExecutionSubcategory(), false)))
					.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(InCurveSelectionTreeNode->GetExecutionCategory())));

				OutDisplayNameWidget =
					SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.ScriptText");

				break;
			}
			case ENiagaraCurveSelectionNodeStyleMode::Module:
			{
				TWeakPtr<FNiagaraCurveSelectionTreeNode> CurveSelectionTreeNodeWeak = InCurveSelectionTreeNode;
				auto ToolTipLambda = [CurveSelectionTreeNodeWeak]()
				{
						TSharedPtr<FNiagaraCurveSelectionTreeNode> PinnedCurveSelectionTreeNode = CurveSelectionTreeNodeWeak.Pin();
						return CurveSelectionTreeNodeWeak.IsValid()
							? FText::Format(LOCTEXT("ModuleToolTipFormat", "{0} - {1}"),
								PinnedCurveSelectionTreeNode->GetParent().IsValid() ? PinnedCurveSelectionTreeNode->GetParent()->GetDisplayName() : LOCTEXT("UknownScript", "Unknown Script"),
								PinnedCurveSelectionTreeNode->GetDisplayName())
							: FText();
				};
				OutIconWidget =
					SNew(SImage)
					.Image(FNiagaraEditorWidgetsStyle::Get().GetBrush(FNiagaraStackEditorWidgetsUtilities::GetIconNameForExecutionSubcategory(InCurveSelectionTreeNode->GetExecutionSubcategory(), false)))
					.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetColorNameForExecutionCategory(InCurveSelectionTreeNode->GetExecutionCategory())))
					.ToolTipText_Lambda(ToolTipLambda);

				OutDisplayNameWidget =
					SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.ModuleText")
					.ToolTipText_Lambda(ToolTipLambda);

				OutSecondaryDisplayName = SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetSecondDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.SecondaryText")
					.ColorAndOpacity(FSlateColor::UseSubduedForeground());

				break;
			}
			case ENiagaraCurveSelectionNodeStyleMode::DynamicInput:
			{
				if (InCurveSelectionTreeNode->GetIsParameter())
				{
					OutDisplayNameWidget =
						SNew(SNiagaraParameterNameTextBlock)
						.IsReadOnly(true)
						.ReadOnlyTextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.InputText")
						.ParameterText(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName);
				}
				else
				{
					OutDisplayNameWidget =
						SNew(STextBlock)
						.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.InputText")
						.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName);
				}

				OutSecondaryIconWidget = SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FNiagaraStackEditorWidgetsUtilities::GetIconTextForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic))
					.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic)) * FLinearColor(1.0f, 1.0f, 1.0f, 0.5f));

				OutSecondaryDisplayName = SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetSecondDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.SecondaryText")
					.ColorAndOpacity(FSlateColor::UseSubduedForeground());

				break;
			}
			case ENiagaraCurveSelectionNodeStyleMode::DataInterface:
			{
				if (InCurveSelectionTreeNode->GetIsParameter())
				{
					OutDisplayNameWidget =
						SNew(SNiagaraParameterNameTextBlock)
						.IsReadOnly(true)
						.ReadOnlyTextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.InputText")
						.ParameterText(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName);
				}
				else
				{
					OutDisplayNameWidget =
						SNew(STextBlock)
						.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName)
						.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.InputText");
				}

				OutSecondaryIconWidget =
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FNiagaraStackEditorWidgetsUtilities::GetIconTextForInputMode(UNiagaraStackFunctionInput::EValueMode::Data))
					.ToolTipText(FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode::Data))
					.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForInputMode(UNiagaraStackFunctionInput::EValueMode::Data)) * FLinearColor(1.0f, 1.0f, 1.0f, 0.5f));

				OutSecondaryDisplayName = SNew(STextBlock)
					.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetSecondDisplayName)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.SecondaryText")
					.ColorAndOpacity(FSlateColor::UseSubduedForeground());

				break;
			}
			case ENiagaraCurveSelectionNodeStyleMode::CurveComponent:
			{
				OutIconWidget =
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
					.Text(FEditorFontGlyphs::Circle)
					.ColorAndOpacity(InCurveSelectionTreeNode->GetCurveColor());

				OutDisplayNameWidget =
					SNew(SBox)
					.VAlign(VAlign_Center)
					.MinDesiredHeight(22)
					[
						SNew(STextBlock)
						.Text(InCurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetDisplayName)
						.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.CurveComponentText")
					];

				break;
			}
		}
	}

	virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow) override
	{
		if (InColumnName == ColumnNames.Label)
		{
			TSharedRef<SHorizontalBox> LabelBox = SNew(SHorizontalBox)
				.IsEnabled(CurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetIsEnabledAndParentIsEnabled);
			auto AddWidgetsToLabelBox = [LabelBox](TSharedRef<FNiagaraCurveSelectionTreeNode> InCurveSelectionTreeNode)
			{
				TSharedPtr<SWidget> IconWidget;
				TSharedPtr<SWidget> DisplayNameWidget;
				TSharedPtr<SWidget> SecondaryIconWidget;
				TSharedPtr<SWidget> SecondaryDisplayNameWidget;
				GetWidgetsforNode(InCurveSelectionTreeNode, IconWidget, DisplayNameWidget, SecondaryIconWidget, SecondaryDisplayNameWidget);
				if (IconWidget.IsValid())
				{
					LabelBox->AddSlot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(0, 0, 4, 0)
						[
							SNew(SBox)
							.WidthOverride(FNiagaraEditorWidgetsStyle::Get().GetFloat("NiagaraEditor.Stack.IconHighlightedSize"))
							.HeightOverride(22)
							.HAlign(EHorizontalAlignment::HAlign_Center)
							.VAlign(EVerticalAlignment::VAlign_Center)
							[
								IconWidget.ToSharedRef()
							]
						];
				}
				if (DisplayNameWidget.IsValid())
				{
					LabelBox->AddSlot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(0, 0, 5, 0)
						[
							DisplayNameWidget.ToSharedRef()
						];
				}
				if (SecondaryIconWidget.IsValid())
				{
					LabelBox->AddSlot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(0, 0, 2, 0)
						[
							SNew(SBox)
							.WidthOverride(FNiagaraEditorWidgetsStyle::Get().GetFloat("NiagaraEditor.Stack.IconHighlightedSize"))
							.HeightOverride(22)
							.HAlign(EHorizontalAlignment::HAlign_Center)
							.VAlign(EVerticalAlignment::VAlign_Center)
							[
								SecondaryIconWidget.ToSharedRef()
							]
						];
				}

				if (SecondaryDisplayNameWidget.IsValid())
				{
					LabelBox->AddSlot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(0, 0, 7, 0)
						[
							SecondaryDisplayNameWidget.ToSharedRef()
						];
				}
			};

			if (CurveSelectionTreeNode->GetStyleMode() == ENiagaraCurveSelectionNodeStyleMode::DataInterface || CurveSelectionTreeNode->GetStyleMode() == ENiagaraCurveSelectionNodeStyleMode::DynamicInput)
			{
				// Data interfaces and dynamic inputs might be nested in several dynamic inputs which aren't shown in the tree so we show the parent icons and display names to disambiguate them.
				TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> InputNodes;
				TSharedPtr<FNiagaraCurveSelectionTreeNode> CurrentNode = CurveSelectionTreeNode;
				while (CurrentNode.IsValid())
				{
					InputNodes.Insert(CurrentNode.ToSharedRef(), 0);
					if (CurrentNode->GetParent().IsValid() && CurrentNode->GetParent()->GetStyleMode() == ENiagaraCurveSelectionNodeStyleMode::DynamicInput && CurrentNode->GetParent()->GetShowInTree() == false)
					{
						CurrentNode = CurrentNode->GetParent();
					}
					else
					{
						CurrentNode = nullptr;
					}
				}

				for (TSharedRef<FNiagaraCurveSelectionTreeNode> InputNode : InputNodes)
				{
					AddWidgetsToLabelBox(InputNode);
				}
			}
			else
			{
				AddWidgetsToLabelBox(CurveSelectionTreeNode);
			}

			if (CurveSelectionTreeNode->GetToolTipText().IsEmptyOrWhitespace() == false)
			{
				LabelBox->SetToolTipText(CurveSelectionTreeNode->GetToolTipText());
			}

			return LabelBox;
		}
		else if (InColumnName == ColumnNames.PinHeader)
		{
			return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, TableRow);
		}

		return nullptr;
	}

	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override
	{
		if(CurveSelectionTreeNode->GetCurve() != nullptr && CurveSelectionTreeNode->GetCurveDataInterface().IsValid())
		{
			TUniquePtr<FRichCurveEditorModelRaw> NewCurve = MakeUnique<FRichCurveEditorModelRaw>(CurveSelectionTreeNode->GetCurve(), CurveSelectionTreeNode->GetCurveDataInterface().Get());
			NewCurve->SetShortDisplayName(CurveSelectionTreeNode->GetDisplayName());
			NewCurve->SetChannelName(CurveSelectionTreeNode->GetCurveName());
			NewCurve->SetColor(CurveSelectionTreeNode->GetCurveColor());
			NewCurve->OnCurveModified().AddSP(CurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::NotifyCurveChanged);
			NewCurve->SetIsReadOnly(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(CurveSelectionTreeNode, &FNiagaraCurveSelectionTreeNode::GetCurveIsReadOnly)));
			OutCurveModels.Add(MoveTemp(NewCurve));
		}
	}

	virtual bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const override
	{ 
		const FCurveEditorTreeTextFilter* Filter = static_cast<const FCurveEditorTreeTextFilter*>(InFilter);
		for (const FCurveEditorTreeTextFilterTerm& Term : Filter->GetTerms())
		{
			if (NodeMatchesTextFilterTerm(Term))
			{
				return true;
			}
		}
		return false;
	}

	static bool CompareTreeItems(const FCurveEditorTreeItem& ItemA, const FCurveEditorTreeItem& ItemB)
	{
		const FNiagaraCurveOverviewTreeItem& NiagaraItemA = static_cast<const FNiagaraCurveOverviewTreeItem&>(*ItemA.GetItem().Get());
		const FNiagaraCurveOverviewTreeItem& NiagaraItemB = static_cast<const FNiagaraCurveOverviewTreeItem&>(*ItemB.GetItem().Get());
		return NiagaraItemA.CompareTo(NiagaraItemB);
	}

private:
	bool NodeMatchesTextFilterTerm(const FCurveEditorTreeTextFilterTerm& Term) const
	{
		bool bMatched = false;
		TSharedPtr<FNiagaraCurveSelectionTreeNode> CurrentCurveSelecitonTreeNode = CurveSelectionTreeNode;
		for (const FCurveEditorTreeTextFilterToken& Token : Term.ChildToParentTokens)
		{
			if (CurrentCurveSelecitonTreeNode.IsValid() == false)
			{
				// No match - ran out of parents
				return false;
			}
			else if (Token.Match(*CurrentCurveSelecitonTreeNode->GetDisplayName().ToString()) == false)
			{
				return false;
			}

			bMatched = true;
			CurrentCurveSelecitonTreeNode = CurrentCurveSelecitonTreeNode->GetParent();
		}

		return bMatched;
	}

	bool CompareTo(const FNiagaraCurveOverviewTreeItem& OtherItem) const
	{
		const TArray<int32>& SortIndices = CurveSelectionTreeNode->GetSortIndices();
		const TArray<int32>& OtherSortIndices = OtherItem.CurveSelectionTreeNode->GetSortIndices();
		for (int32 i = 0; i < SortIndices.Num() && i < OtherSortIndices.Num(); ++i)
		{
			if (SortIndices[i] < OtherSortIndices[i])
			{
				return true;
			}
		}
		return false;
	}

private:
	TSharedRef<FNiagaraCurveSelectionTreeNode> CurveSelectionTreeNode;
};

void SNiagaraCurveOverview::CreateCurveEditorTreeItemsRecursive(FCurveEditorTreeItemID ParentTreeItemID, const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& CurveSelectionTreeNodes, TArray<FGuid>& LastCurveSelectionTreeNodeIds)
{
	for (TSharedRef<FNiagaraCurveSelectionTreeNode> CurveSelectionTreeNode : CurveSelectionTreeNodes)
	{
		LastCurveSelectionTreeNodeIds.RemoveSwap(CurveSelectionTreeNode->GetNodeUniqueId());

		FCurveEditorTreeItemID LastParentTreeItemId;
		FCurveEditorTreeItem* ItemForTreeNode = nullptr;
		if(CurveSelectionTreeNode->GetShowInTree())
		{
			FCurveEditorTreeItemID TreeItemId;
			FCurveEditorTreeItemID* ExistingTreeItemIdPtr = CurveTreeNodeIdToTreeItemIdMap.Find(CurveSelectionTreeNode->GetNodeUniqueId());
			if(ExistingTreeItemIdPtr != nullptr)
			{
				TreeItemId = *ExistingTreeItemIdPtr;
				ItemForTreeNode = &CurveEditor->GetTreeItem(TreeItemId);
			}
			else
			{
				TSharedRef<FNiagaraCurveOverviewTreeItem> TreeItem = MakeShared<FNiagaraCurveOverviewTreeItem>(CurveSelectionTreeNode);
				FCurveEditorTreeItem* NewItem = CurveEditor->AddTreeItem(ParentTreeItemID);
				NewItem->SetStrongItem(TreeItem);
				TreeItemId = NewItem->GetID();
				CurveTreeNodeIdToTreeItemIdMap.Add(CurveSelectionTreeNode->GetNodeUniqueId(), TreeItemId);
				CurveEditorTree->SetItemExpansion(TreeItemId, CurveSelectionTreeNode->GetIsExpanded());
				ItemForTreeNode = NewItem;
			}
			LastParentTreeItemId = TreeItemId;
		}
		else
		{
			LastParentTreeItemId = ParentTreeItemID;
		}

		CreateCurveEditorTreeItemsRecursive(LastParentTreeItemId, CurveSelectionTreeNode->GetChildNodes(), LastCurveSelectionTreeNodeIds);
	}
}

void SNiagaraCurveOverview::RefreshCurveEditorTreeItems()
{
	FScopedCurveEditorTreeEventGuard ScopedCurveEditorTreeEventGuard = CurveEditor->GetTree()->ScopedEventGuard();

	TArray<FGuid> LastCurveSelectionTreeNodeIds;
	CurveTreeNodeIdToTreeItemIdMap.GetKeys(LastCurveSelectionTreeNodeIds);
	CreateCurveEditorTreeItemsRecursive(FCurveEditorTreeItemID::Invalid(), SystemViewModel->GetCurveSelectionViewModel()->GetRootNodes(), LastCurveSelectionTreeNodeIds);

	for (const FGuid& LastCurveSelectionTreeNodeId : LastCurveSelectionTreeNodeIds)
	{
		FCurveEditorTreeItemID UnusedTreeItemId = CurveTreeNodeIdToTreeItemIdMap.FindRef(LastCurveSelectionTreeNodeId);
		CurveEditor->RemoveTreeItem(UnusedTreeItemId);
		CurveTreeNodeIdToTreeItemIdMap.Remove(LastCurveSelectionTreeNodeId);
	}
}

void SNiagaraCurveOverview::CurveSelectionViewModelRefreshed()
{
	RefreshCurveEditorTreeItems();
}

void SNiagaraCurveOverview::CurveSelectionViewModelRequestSelectNode(FGuid NodeIdToSelect)
{
	FCurveEditorTreeItemID* TreeItemId = CurveTreeNodeIdToTreeItemIdMap.Find(NodeIdToSelect);
	if (TreeItemId != nullptr && TreeItemId->IsValid())
	{
		FCurveEditorTreeItem* TreeItem = CurveEditor->FindTreeItem(*TreeItemId);
		if (TreeItem != nullptr)
		{
			// First expand the item and all of its parents.
			FCurveEditorTreeItem* CurrentTreeItem = TreeItem;
			while (CurrentTreeItem != nullptr)
			{
				CurveEditorTree->SetItemExpansion(CurrentTreeItem->GetID(), true);
				FCurveEditorTreeItemID ParentId = CurrentTreeItem->GetParentID();
				CurrentTreeItem = ParentId.IsValid() 
					? CurveEditor->FindTreeItem(ParentId)
					: nullptr;
			}

			// Then set the selection.
			TArray<FCurveEditorTreeItemID> SelectedTreeItemIds;
			SelectedTreeItemIds.Add(*TreeItemId);
			CurveEditorTree->ClearSelection();
			CurveEditorTree->SetItemSelection(SelectedTreeItemIds, true, ESelectInfo::Direct);
			CurveEditorTree->SetItemExpansion(*TreeItemId, true);
		}
	}
}

void SNiagaraCurveOverview::Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;
	SystemViewModel->GetCurveSelectionViewModel()->OnRefreshed().AddSP(this, &SNiagaraCurveOverview::CurveSelectionViewModelRefreshed);
	SystemViewModel->GetCurveSelectionViewModel()->OnRequestSelectNode().AddSP(this, &SNiagaraCurveOverview::CurveSelectionViewModelRequestSelectNode);

	CurveEditor = MakeShared<FCurveEditor>();
	FCurveEditorInitParams InitParams;
	InitParams.ResolveCurveModelDelegate.BindStatic(&UE::NiagaraEditorWidgets::ResolveNiagaraCurveModel); // Needed for undo / redo.
	CurveEditor->InitCurveEditor(InitParams);
	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");
	CurveEditor->GetTree()->SetSortPredicate(FCurveEditorTree::FCurveTreeItemSortPredicate::CreateStatic(&FNiagaraCurveOverviewTreeItem::CompareTreeItems));

	CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
		.MinimumViewPanelHeight(0.0f)
		.TreeContent()
		[
			SNew(SVerticalBox)

			// Search Box
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SCurveEditorTreeTextFilter, CurveEditor)
			]

			// Curve Tree
			+ SVerticalBox::Slot()
			[
				SAssignNew(CurveEditorTree, SCurveEditorTree, CurveEditor)
				.SelectColumnWidth(0.0f)
				.OnMouseButtonDoubleClick(this, &SNiagaraCurveOverview::CurveTreeItemDoubleClicked)
			]

			// Search status
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SCurveEditorTreeFilterStatusBar, CurveEditor)
			]
		];

	RefreshCurveEditorTreeItems();

	FSlimHorizontalToolBarBuilder ToolBarBuilder(CurveEditorPanel->GetCommands(), FMultiBoxCustomization::None, CurveEditorPanel->GetToolbarExtender(), true);
	ToolBarBuilder.BeginSection("Asset");
	ToolBarBuilder.EndSection();

	TSharedPtr<SOverlay> OverlayWidget;
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ToolBarBuilder.MakeWidget()
		]
		+ SVerticalBox::Slot()
		.Padding(0, 0, 0, 5)
		[
			CurveEditorPanel.ToSharedRef()
		]
	];
}

void SNiagaraCurveOverview::CurveTreeItemDoubleClicked(FCurveEditorTreeItemID TreeItemId)
{
	const FCurveEditorTreeItem* TreeItem = CurveEditor->FindTreeItem(TreeItemId);
	if (TreeItem != nullptr)
	{
		TSharedPtr<ICurveEditorTreeItem> CurveEditorTreeItem = TreeItem->GetItem();
		if (CurveEditorTreeItem.IsValid())
		{
			TSharedPtr<FNiagaraCurveOverviewTreeItem> NiagaraCurveOverviewTreeItem = StaticCastSharedPtr<FNiagaraCurveOverviewTreeItem>(CurveEditorTreeItem);
			TOptional<FObjectKey> DisplayedObjectKey = NiagaraCurveOverviewTreeItem->GetCurveSelectionTreeNode()->GetDisplayedObjectKey();
			if (DisplayedObjectKey.IsSet())
			{
				SystemViewModel->GetSelectionViewModel()->EmptySelection();
				SystemViewModel->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectKeyDeferred(DisplayedObjectKey.GetValue());

				SystemViewModel->GetDocumentViewModel()->DrawAttentionToPrimaryDocument();
			}
		}
	}
}

SNiagaraCurveOverview::~SNiagaraCurveOverview()
{
	if (SystemViewModel.IsValid())
	{
		if (SystemViewModel->GetCurveSelectionViewModel() != nullptr)
		{
			SystemViewModel->GetCurveSelectionViewModel()->OnRefreshed().RemoveAll(this);
		}
		SystemViewModel.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
