// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "CurveEditorTypes.h"

class FNiagaraSystemViewModel;
struct FNiagaraCurveSelectionTreeNode;
class FCurveEditor;
class SCurveEditorPanel;
class SCurveEditorTree;

/** A curve editor control for curves in a niagara System. */
class SNiagaraCurveOverview : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraCurveOverview) { }
	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	~SNiagaraCurveOverview();

private:
	void CreateCurveEditorTreeItemsRecursive(FCurveEditorTreeItemID ParentTreeItemID, const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& CurveSelectionTreeNodes, TArray<FGuid>& LastCurveSelectionTreeNodeIds);

	void RefreshCurveEditorTreeItems();

	void CurveSelectionViewModelRefreshed();

	void CurveSelectionViewModelRequestSelectNode(FGuid NodeIdToSelect);

	void CurveTreeItemDoubleClicked(FCurveEditorTreeItemID TreeItemId);

private:
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;

	TSharedPtr<FCurveEditor> CurveEditor;
	TSharedPtr<SCurveEditorPanel> CurveEditorPanel;
	TSharedPtr<SCurveEditorTree> CurveEditorTree;
	TMap<FGuid, FCurveEditorTreeItemID> CurveTreeNodeIdToTreeItemIdMap;
};
