// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraCurveThumbnail.h"
#include "Curves/RichCurve.h"

#define LOCTEXT_NAMESPACE "SNiagaraCurveThumbnail"

void SNiagaraCurveThumbnail::Construct(const FArguments& InArgs, const FRichCurve& CurveToDisplay)
{
	Width = InArgs._Width;
	Height = InArgs._Height;

	float TimeMin;
	float TimeMax;
	float ValueMin;
	float ValueMax;
	CurveToDisplay.GetTimeRange(TimeMin, TimeMax);
	CurveToDisplay.GetValueRange(ValueMin, ValueMax);

	const float TimeRange = TimeMax - TimeMin;
	const float ValueRange = ValueMax - ValueMin;
	if (!FMath::IsNearlyZero(TimeRange) && !FMath::IsNearlyZero(ValueRange))
	{
		constexpr int32 Points = 13;
		const float TimeIncrement = TimeRange / (Points - 1);
		for (int32 i = 0; i < Points; i++)
		{
			const float Time = TimeMin + i * TimeIncrement;
			const float Value = CurveToDisplay.Eval(Time);

			const float NormalizedX = (Time - TimeMin) / TimeRange;
			const float NormalizedY = (Value - ValueMin) / ValueRange;
			CurvePoints.Add(FVector2D(NormalizedX * Width, (1 - NormalizedY) * Height));
		}
	}
}

int32 SNiagaraCurveThumbnail::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), CurvePoints, ESlateDrawEffect::None, InWidgetStyle.GetForegroundColor(), true, 2.0f);
	return LayerId;
}

FVector2D SNiagaraCurveThumbnail::ComputeDesiredSize(float) const
{
	return FVector2D(Width, Height);
}

#undef LOCTEXT_NAMESPACE
