// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "Input/Reply.h"
#include "Widgets/SLeafWidget.h"

class UCurveBase;
class UCurveFloat;
struct FAssetData;
struct FRichCurve;

class SNiagaraCurveThumbnail : public SLeafWidget
{
	SLATE_BEGIN_ARGS(SNiagaraCurveThumbnail)
		: _Width(16)
		, _Height(8)
		{
		}
		SLATE_ARGUMENT(float, Width)
		SLATE_ARGUMENT(float, Height)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FRichCurve& CurveToDisplay);

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	TArray<FVector2D> CurvePoints;
	float Width;
	float Height;
};
