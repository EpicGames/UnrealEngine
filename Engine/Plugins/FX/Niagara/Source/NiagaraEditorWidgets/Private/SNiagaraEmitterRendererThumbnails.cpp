// Copyright Epic Games, Inc. All Rights Reserved.


#include "SNiagaraEmitterRendererThumbnails.h"

#include "NiagaraEmitterHandle.h"
#include "SlateOptMacros.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackRenderItemGroup.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/SToolTip.h"
#include "Materials/Material.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/Layout/SScaleBox.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

static constexpr float RendererThumbnailSize = 24.f;

#define LOCTEXT_NAMESPACE "NiagaraEmitterRendererThumbnails"

FNiagaraRendererStackEntryObserver::FNiagaraRendererStackEntryObserver()
{
	
}

FNiagaraRendererStackEntryObserver::~FNiagaraRendererStackEntryObserver()
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		if(UNiagaraStackViewModel* StackViewModel = EmitterHandleViewModelWeak.Pin()->GetEmitterStackViewModel())
		{
			StackViewModel->OnDataObjectChanged().RemoveAll(this);
			StackViewModel->OnStructureChanged().RemoveAll(this);
		}
	}

	UMaterial::OnMaterialCompilationFinished().RemoveAll(this);
}

void FNiagaraRendererStackEntryObserver::Initialize(TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	EmitterHandleViewModelWeak = InEmitterHandleViewModel;

	if(UNiagaraStackViewModel* StackViewModel = EmitterHandleViewModelWeak.Pin()->GetEmitterStackViewModel())
	{
		StackViewModel->OnDataObjectChanged().AddSP(this, &FNiagaraRendererStackEntryObserver::OnStackDataObjectChanged);
		StackViewModel->OnStructureChanged().AddSP(this, &FNiagaraRendererStackEntryObserver::OnRenderGroupStructureChanged);
	}
	
	UMaterial::OnMaterialCompilationFinished().AddSP(this, &FNiagaraRendererStackEntryObserver::OnMaterialCompiled);
}

void FNiagaraRendererStackEntryObserver::OnRenderGroupStructureChanged(ENiagaraStructureChangedFlags NiagaraStructureChangedFlags) const
{
	OnRenderersChanged.Broadcast();
}

void FNiagaraRendererStackEntryObserver::OnMaterialCompiled(class UMaterialInterface* MaterialInterface) const
{
	if (EmitterHandleViewModelWeak.IsValid())
	{
		bool bUsingThisMaterial = false;
		TArray<UNiagaraStackEntry*> RendererPreviewStackEntries;
		EmitterHandleViewModelWeak.Pin()->GetRendererEntries(RendererPreviewStackEntries);
		FNiagaraEmitterInstance* InInstance = EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetSimulation().IsValid() ? EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetSimulation().Pin().Get() : nullptr;
		for (UNiagaraStackEntry* Entry : RendererPreviewStackEntries)
		{
			UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(Entry);
			UNiagaraRendererProperties* RendererProperties = RendererItem ? RendererItem->GetRendererProperties() : nullptr;
			if (RendererProperties == nullptr)
			{
				continue;
			}

			TArray<UMaterialInterface*> Materials;
			RendererProperties->GetUsedMaterials(InInstance, Materials);
			if (Materials.Contains(MaterialInterface))
			{
				bUsingThisMaterial = true;
				break;
			}
		}

		if (bUsingThisMaterial)
		{
			OnRenderersChanged.Broadcast();
		}
	}
}

void FNiagaraRendererStackEntryObserver::OnStackDataObjectChanged(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange NiagaraDataObjectChange) const
{
	for (UObject* ChangedObject : ChangedObjects)
	{
		if (ChangedObject->IsA<UNiagaraRendererProperties>())
		{
			OnRenderersChanged.Broadcast();
			break;
		}
	}
}

void SNiagaraEmitterRendererThumbnails::Construct(const FArguments& InArgs, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	EmitterHandleViewModelWeak = InEmitterHandleViewModel;
	OnRendererThumbnailClickedDelegate = InArgs._OnRendererThumbnailClicked;
	
	RendererObserver = MakeShared<FNiagaraRendererStackEntryObserver>();
	RendererObserver->Initialize(InEmitterHandleViewModel);
	RendererObserver->OnRenderersChanged.AddSP(this, &SNiagaraEmitterRendererThumbnails::RefreshThumbnails);
	
	ChildSlot
	[
		SAssignNew(Content, SBox)
	];

	RefreshThumbnails();
}

void SNiagaraEmitterRendererThumbnails::RefreshThumbnails()
{	
	TArray<UNiagaraStackEntry*> RendererPreviewStackEntries;
	EmitterHandleViewModelWeak.Pin()->GetRendererEntries(RendererPreviewStackEntries);
	FNiagaraEmitterInstance* InInstance = EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetSimulation().IsValid() ? EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetSimulation().Pin().Get() : nullptr;
	
	FToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None, nullptr, true);
	ToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed);
		
	for (int32 StackEntryIndex = 0; StackEntryIndex < RendererPreviewStackEntries.Num(); StackEntryIndex++)
	{
		UNiagaraStackEntry* Entry = RendererPreviewStackEntries[StackEntryIndex];
		UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(Entry);
		UNiagaraRendererProperties* RendererProperties = RendererItem ? RendererItem->GetRendererProperties() : nullptr;
		if (RendererProperties == nullptr)
		{
			continue;
		}

		TArray<TSharedPtr<SWidget>> Widgets;
		TArray<TSharedPtr<SWidget>> TooltipWidgets;
		RendererProperties->GetRendererWidgets(InInstance, Widgets, UThumbnailManager::Get().GetSharedThumbnailPool());
		RendererProperties->GetRendererTooltipWidgets(InInstance, TooltipWidgets, UThumbnailManager::Get().GetSharedThumbnailPool());

		if (TooltipWidgets.Num() < Widgets.Num())
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Renderer(%ls) has a mismatch of widgets(%d) to tooltips(%d)", *GetFullNameSafe(RendererProperties), Widgets.Num(), TooltipWidgets.Num());
		}

		// Fallback widget & tooltip
		if(Widgets.Num() == 0)
		{
			RendererProperties->CreateDefaultRendererWidget(Widgets);
		}

		while ( TooltipWidgets.Num() < Widgets.Num() )
		{
			FText BaseText = LOCTEXT("DefaultRendererTooltip", "{0}");
			TooltipWidgets.Add(SNew(STextBlock).Text(FText::FormatOrdered(BaseText, RendererProperties->GetClass()->GetDisplayNameText())));
		}
			
		for (int32 WidgetIndex = 0; WidgetIndex < Widgets.Num(); WidgetIndex++)
		{
			SetupRendererThumbnailTooltip(Widgets[WidgetIndex], TooltipWidgets[WidgetIndex]);
				
			// We wrap the thumbnail in a scalebox so it will scale to whatever its parent dictates
			TSharedPtr<SWidget> RendererThumbnailWidget = SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.HeightOverride(RendererThumbnailSize)
				.WidthOverride(RendererThumbnailSize)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						Widgets[WidgetIndex].ToSharedRef()	
					]
				];
				
			// If we have on click functionality, we wrap the thumbnail in a button
			if(OnRendererThumbnailClickedDelegate.IsBound())
			{
				RendererThumbnailWidget = SNew(SButton)
					.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
					.ContentPadding(FMargin(1.f))
					.OnClicked(this, &SNiagaraEmitterRendererThumbnails::OnRendererThumbnailClicked, RendererItem)
					[
						RendererThumbnailWidget.ToSharedRef()
					];
			}
				
			ToolBarBuilder.AddWidget(
				RendererThumbnailWidget.ToSharedRef()
			);
		}
	
		// if we had a widget for this entry, add a separator for the next entry's widgets, except for the last entry
		if(Widgets.Num() > 0 && StackEntryIndex < RendererPreviewStackEntries.Num() - 1)
		{
			ToolBarBuilder.AddSeparator();
		}
	}
	
	Content->SetContent(ToolBarBuilder.MakeWidget());
}

void SNiagaraEmitterRendererThumbnails::SetupRendererThumbnailTooltip(TSharedPtr<SWidget> InWidget, TSharedPtr<SWidget> InTooltipWidget)
{
	TSharedPtr<SToolTip> ThumbnailTooltipWidget;
	// If this is just text, don't constrain the size
	if (InTooltipWidget->GetType() == TEXT("STextBlock"))
	{
		ThumbnailTooltipWidget = SNew(SToolTip)
			.Content()
			[
				InTooltipWidget.ToSharedRef()
			];
	}
	else
	{
		ThumbnailTooltipWidget = SNew(SToolTip)
			.Content()
			[
				SNew(SBox)
				.MaxDesiredHeight(64.0f)
				.MinDesiredHeight(64.0f)
				.MaxDesiredWidth(64.0f)
				.MinDesiredWidth(64.0f)
				[
					InTooltipWidget.ToSharedRef()
				]
			];
	}
	
	InWidget->SetToolTip(ThumbnailTooltipWidget);
}

FReply SNiagaraEmitterRendererThumbnails::OnRendererThumbnailClicked(UNiagaraStackRendererItem* StackRendererItem) const
{
	if(OnRendererThumbnailClickedDelegate.IsBound())
	{
		return OnRendererThumbnailClickedDelegate.Execute(StackRendererItem);
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
