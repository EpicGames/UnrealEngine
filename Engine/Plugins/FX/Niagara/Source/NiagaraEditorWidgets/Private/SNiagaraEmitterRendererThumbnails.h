// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackRenderItemGroup.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API NIAGARAEDITORWIDGETS_API


struct FNiagaraRendererStackEntryObserver : public TSharedFromThis<FNiagaraRendererStackEntryObserver>
{
public:
	FNiagaraRendererStackEntryObserver();
	~FNiagaraRendererStackEntryObserver();

	void Initialize(TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);
	
	DECLARE_MULTICAST_DELEGATE(FOnRendererGroupStructureChanged);

	FOnRendererGroupStructureChanged OnRenderersChanged;
	
private:	
	void OnRenderGroupStructureChanged(ENiagaraStructureChangedFlags NiagaraStructureChangedFlags) const;
	void OnMaterialCompiled(class UMaterialInterface* MaterialInterface) const;
	void OnStackDataObjectChanged(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange NiagaraDataObjectChange) const;

private:
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelWeak;
};

/**
 * 
 */
class SNiagaraEmitterRendererThumbnails : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnRendererThumbnailClicked, UNiagaraStackRendererItem* StackRendererItem);
	
	SLATE_BEGIN_ARGS(SNiagaraEmitterRendererThumbnails)
		{}
		SLATE_EVENT(FOnRendererThumbnailClicked, OnRendererThumbnailClicked)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);
private:
	UE_API void RefreshThumbnails();

	static UE_API void SetupRendererThumbnailTooltip(TSharedPtr<SWidget> InWidget, TSharedPtr<SWidget> InTooltipWidget);
	UE_API FReply OnRendererThumbnailClicked(UNiagaraStackRendererItem* StackRendererItem) const;

private:
	TSharedPtr<SBox> Content;
	TSharedPtr<FNiagaraRendererStackEntryObserver> RendererObserver;
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelWeak;
	FOnRendererThumbnailClicked OnRendererThumbnailClickedDelegate;
};

#undef UE_API
