// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraOverviewGraph.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_Niagara.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GraphEditAction.h"
#include "GraphEditorActions.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraOverviewGraphNodeFactory.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraStackEditorData.h"
#include "SNiagaraOverviewGraphTitleBar.h"
#include "SNiagaraStack.h"
#include "ScopedTransaction.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Interfaces/IMainFrameModule.h"
#include "Logging/StructuredLog.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Templates/SharedPointer.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/SItemSelector.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "Widgets/AssetBrowser/SNiagaraAddEmitterToSystemWindow.h"

#define LOCTEXT_NAMESPACE "NiagaraOverviewGraph"

void SNiagaraOverviewGraph::Construct(const FArguments& InArgs, TSharedRef<FNiagaraOverviewGraphViewModel> InViewModel, const FAssetData& InEditedAsset)
{
	ViewModel = InViewModel;
	ViewModel->GetNodeSelection()->OnSelectedObjectsChanged().AddSP(this, &SNiagaraOverviewGraph::ViewModelSelectionChanged);
	ViewModel->GetSystemViewModel()->OnPreClose().AddSP(this, &SNiagaraOverviewGraph::PreClose);
	ViewModel->OnNodesPasted().AddSP(this, &SNiagaraOverviewGraph::NodesPasted);

	bUpdatingViewModelSelectionFromGraph = false;
	bUpdatingGraphSelectionFromViewModel = false;

	SGraphEditor::FGraphEditorEvents Events;
	Events.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SNiagaraOverviewGraph::GraphSelectionChanged);
	Events.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SNiagaraOverviewGraph::OnCreateGraphActionMenu);
	Events.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &SNiagaraOverviewGraph::OnVerifyNodeTitle);
	Events.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &SNiagaraOverviewGraph::OnNodeTitleCommitted);

	FGraphAppearanceInfo AppearanceInfo;
	if (ViewModel->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		AppearanceInfo.CornerText = LOCTEXT("NiagaraOverview_AppearanceCornerTextEmitter", "EMITTER");

	}
	else if (ViewModel->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		AppearanceInfo.CornerText = LOCTEXT("NiagaraOverview_AppearanceCornerTextSystem", "SYSTEM");
	}
	else
	{
		ensureMsgf(false, TEXT("Encountered unhandled SystemViewModel Edit Mode!"));
		AppearanceInfo.CornerText = LOCTEXT("NiagaraOverview_AppearanceCornerTextGeneric", "NIAGARA");
	}
	
	TSharedRef<SWidget> TitleBarWidget = SNew(SNiagaraOverviewGraphTitleBar, ViewModel->GetSystemViewModel(), InEditedAsset).Visibility(EVisibility::SelfHitTestInvisible);

	TSharedRef<FUICommandList> Commands = ViewModel->GetCommands();
	Commands->MapAction(
		FGraphEditorCommands::Get().CreateComment,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnCreateComment));
	Commands->MapAction(
		FNiagaraEditorModule::Get().GetCommands().ZoomToFit,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::ZoomToFit));
	Commands->MapAction(
		FNiagaraEditorModule::Get().GetCommands().ZoomToFitAll,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::ZoomToFitAll));
	// Alignment Commands
	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnAlignTop)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnAlignMiddle)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnAlignBottom)
	);

	// Distribution Commands
	Commands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnDistributeNodesH)
	);

	Commands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnDistributeNodesV)
	);

	Commands->MapAction(FNiagaraEditorCommands::Get().OpenAddEmitterMenu,
		FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OpenAddEmitterMenu),
		FCanExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::CanAddEmitters),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SNiagaraOverviewGraph::CanAddEmitters)
	);

	Commands->MapAction(
		FNiagaraEditorCommands::Get().HideDisabledModules,
		FExecuteAction::CreateLambda([this]()
		{
			UNiagaraStackEditorData& EditorData = ViewModel->GetSystemViewModel()->GetEditorData().GetStackEditorData();
			EditorData.bHideDisabledModules = !EditorData.bHideDisabledModules;
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			UNiagaraStackEditorData& EditorData = ViewModel->GetSystemViewModel()->GetEditorData().GetStackEditorData();
			return EditorData.bHideDisabledModules;
		}));
	
	GraphEditor = SNew(SGraphEditor)
		.AdditionalCommands(Commands)
		.Appearance(AppearanceInfo)
		.TitleBar(TitleBarWidget)
		.GraphToEdit(ViewModel->GetGraph())
		.GraphEvents(Events)
		.ShowGraphStateOverlay(false);

	GraphEditor->SetNodeFactory(MakeShared<FNiagaraOverviewGraphNodeFactory>());

	GraphEditor->GetCurrentGraph()->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &SNiagaraOverviewGraph::OnNodesCreated));

	const UNiagaraEditorSettings* NiagaraSettings = GetDefault<UNiagaraEditorSettings>();
	FNiagaraGraphViewSettings ViewSettings = ViewModel->GetViewSettings();
	if (NiagaraSettings->bAlwaysZoomToFitSystemGraph == false && ViewSettings.IsValid())
	{
		GraphEditor->SetViewLocation(ViewSettings.GetLocation(), ViewSettings.GetZoom());
		ZoomToFitFrameDelay = 0;
	}
	else
	{
		// When initialzing the graph control the stacks inside the nodes aren't actually available until two frames later due to
		// how the underlying list view works.  In order to zoom to fix correctly we have to delay for an extra fram so we use a
		// counter here instead of a simple bool.
		ZoomToFitFrameDelay = 2;
	}

	GraphEditor->GetCurrentGraph()->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &SNiagaraOverviewGraph::OnNodesCreated));

	ChildSlot
	[
		GraphEditor.ToSharedRef()
	];
}

void SNiagaraOverviewGraph::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (ZoomToFitFrameDelay > 0)
	{
		ZoomToFitFrameDelay--;
		if(ZoomToFitFrameDelay == 0)
		{
			GraphEditor->ZoomToFit(false);
		}
	}
}

void SNiagaraOverviewGraph::ViewModelSelectionChanged()
{
	if (bUpdatingViewModelSelectionFromGraph == false)
	{
		if (FNiagaraEditorUtilities::SetsMatch(GraphEditor->GetSelectedNodes(), ViewModel->GetNodeSelection()->GetSelectedObjectsResolved()) == false)
		{
			TGuardValue<bool> UpdateGuard(bUpdatingGraphSelectionFromViewModel, true);
			GraphEditor->ClearSelectionSet();
			for (UObject* SelectedNode : ViewModel->GetNodeSelection()->GetSelectedObjectsResolved())
			{
				UEdGraphNode* GraphNode = Cast<UEdGraphNode>(SelectedNode);
				if (GraphNode != nullptr)
				{
					GraphEditor->SetNodeSelection(GraphNode, true);
				}
			}
		}
	}
}

void SNiagaraOverviewGraph::GraphSelectionChanged(const TSet<UObject*>& SelectedNodes)
{
	if (bUpdatingGraphSelectionFromViewModel == false)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingViewModelSelectionFromGraph, true);
		if (SelectedNodes.Num() == 0)
		{
			ViewModel->GetNodeSelection()->ClearSelectedObjects();
		}
		else
		{
			ViewModel->GetNodeSelection()->SetSelectedObjects(SelectedNodes);
		}
	}
}

void SNiagaraOverviewGraph::PreClose()
{
	if (ViewModel.IsValid() && GraphEditor.IsValid())
	{
		FVector2f Location;
		float Zoom;
		GraphEditor->GetViewLocation(Location, Zoom);
		ViewModel->SetViewSettings(FNiagaraGraphViewSettings(Location, Zoom));
	}
}

void SNiagaraOverviewGraph::OpenAddEmitterMenu()
{
	IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	TSharedPtr<SWindow>	ParentWindow = MainFrame.GetParentWindow();

	FSoftObjectPath Path = GetDefault<UNiagaraEditorSettings>()->EmitterAssetWizardConfiguration;
	UObject* ConfigUObject = Path.TryLoad();
	if(UTaggedAssetBrowserConfiguration* ConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(ConfigUObject))
	{
		TSharedRef<SWindow> AddEmitterWindow = SNew(SNiagaraAddEmitterToSystemWindow, *ConfigurationAsset, ViewModel->GetSystemViewModel());
		FSlateApplication::Get().AddModalWindow(AddEmitterWindow, ParentWindow);
	}
	else
	{
		UE_LOGFMT(LogNiagaraEditor, Warning, "Could not find emitter wizard configuration asset. Check Niagara Editor settings under Project Settings");
	}
}

bool SNiagaraOverviewGraph::CanAddEmitters() const
{
	return ViewModel->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset ? true : false;
}

FActionMenuContent SNiagaraOverviewGraph::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	if (ViewModel->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		FMenuBuilder MenuBuilder(true, ViewModel->GetCommands(), TSharedPtr<FExtender>(), false, &FAppStyle::Get(), false);
		
		MenuBuilder.BeginSection(TEXT("NiagaraOverview_EditGraph"), LOCTEXT("EditGraph", "Edit Graph"));
		{
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().OpenAddEmitterMenu);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("MinimalEmitterLabel", "Add minimal emitter"),
				FNiagaraEditorUtilities::Tooltips::GetMinimalEmitterCreationTooltip(),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnCreateEmptyEmitter));
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddStatelessEmitter", "Add lightweight emitter"),
				GetCreateStatelessEmitterTooltip(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnCreateStatelessEmitter),
					FCanExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::CanCreateStatelessEmitter)
				)
			);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CommentsLabel", "Add Comment"),
				LOCTEXT("AddCommentBoxToolTip", "Add a comment box"),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnCreateComment));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearIsolatedLabel", "Clear Isolated"),
				LOCTEXT("ClearIsolatedToolTip", "Clear the current set of isolated emitters."),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &SNiagaraOverviewGraph::OnClearIsolated));
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(TEXT("NiagaraOverview_View"), LOCTEXT("View", "View"));
		{
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ZoomToFit);
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ZoomToFitAll);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(TEXT("NiagaraOverview_Edit"), LOCTEXT("Edit", "Edit"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		}
		MenuBuilder.EndSection();

		TSharedRef<SWidget> ActionMenu = MenuBuilder.MakeWidget();

		return FActionMenuContent(ActionMenu, ActionMenu);
	}
	return FActionMenuContent(SNullWidget::NullWidget, SNullWidget::NullWidget);
}

void SNiagaraOverviewGraph::OnCreateEmptyEmitter()
{
	ViewModel->GetSystemViewModel()->AddMinimalEmitter();
}

bool SNiagaraOverviewGraph::CanCreateStatelessEmitter() const
{
	return FNiagaraEditorUtilities::IsClassVisibileInGlobalFilter(UNiagaraStatelessEmitter::StaticClass());
}

FText SNiagaraOverviewGraph::GetCreateStatelessEmitterTooltip() const
{
	return CanCreateStatelessEmitter() ?
		LOCTEXT("AddStatelessEmitterToolTip", "Adds a lightweight emitter without any modules or renderers.") :
		LOCTEXT("AddStatelessEmitterNotAllowedToolTip", "Lightweight emitters are not available due to project restrictions.");
}

void SNiagaraOverviewGraph::OnCreateStatelessEmitter()
{
	ViewModel->GetSystemViewModel()->AddStatelessEmitter();
}

void SNiagaraOverviewGraph::OnCreateComment()
{
	// Emitter assets have a transient overview graph, so any created comments would also be transient. We skip creating these comments instead.
	if (ViewModel->GetSystemViewModel()->GetEditMode() != ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		FNiagaraSchemaAction_NewComment CommentAction = FNiagaraSchemaAction_NewComment(GraphEditor);
		CommentAction.PerformAction(ViewModel->GetGraph(), nullptr, GraphEditor->GetPasteLocation2f(), false);
	}
}

void SNiagaraOverviewGraph::OnClearIsolated()
{
	ViewModel->GetSystemViewModel()->IsolateEmitters(TArray<FGuid>());
}

bool SNiagaraOverviewGraph::OnVerifyNodeTitle(const FText& NewText, UEdGraphNode* Node, FText& OutErrorMessage) const
{
	UNiagaraOverviewNode* NiagaraNode = Cast<UNiagaraOverviewNode>(Node);
	if (NiagaraNode != nullptr)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> NodeEmitterHandleViewModel = ViewModel->GetSystemViewModel()->GetEmitterHandleViewModelById(NiagaraNode->GetEmitterHandleGuid());
		if (ensureMsgf(NodeEmitterHandleViewModel.IsValid(), TEXT("Failed to find EmitterHandleViewModel with matching Emitter GUID to Overview Node!")))
		{
			return NodeEmitterHandleViewModel->VerifyNameTextChanged(NewText, OutErrorMessage);
		}
	}

	return true;
}

void SNiagaraOverviewGraph::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		// When you request rename on spawn but accept the value, we want to not add a transaction if they just hit "Enter".
		bool bRename = true;
		FText CurrentNodeTitleText = NodeBeingChanged->GetNodeTitle(ENodeTitleType::FullTitle);
		if (CurrentNodeTitleText.EqualTo(NewText))
		{
			return;
		}

		if (NodeBeingChanged->IsA(UNiagaraOverviewNode::StaticClass())) //@TODO System Overview: renaming system or emitters locally through this view
		{
			UNiagaraOverviewNode* OverviewNodeBeingChanged = Cast<UNiagaraOverviewNode>(NodeBeingChanged);
			TSharedPtr<FNiagaraEmitterHandleViewModel> NodeEmitterHandleViewModel = ViewModel->GetSystemViewModel()->GetEmitterHandleViewModelById(OverviewNodeBeingChanged->GetEmitterHandleGuid());
			if (ensureMsgf(NodeEmitterHandleViewModel.IsValid(), TEXT("Failed to find EmitterHandleViewModel with matching Emitter GUID to Overview Node!")))
			{
				NodeEmitterHandleViewModel->OnNameTextComitted(NewText, CommitInfo);
			}
			else
			{
				bRename = false;
			}
		}

		if (bRename)
		{
			const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));
			NodeBeingChanged->Modify();
			NodeBeingChanged->OnRenameNode(NewText.ToString());
		}
	}
}

void SNiagaraOverviewGraph::NodesPasted(const TSet<UEdGraphNode*>& PastedNodes)
{
	if (PastedNodes.Num() != 0)
	{
		PositionPastedNodes(PastedNodes);
		GraphEditor->NotifyGraphChanged();
	}
}

void SNiagaraOverviewGraph::PositionPastedNodes(const TSet<UEdGraphNode*>& PastedNodes)
{
	FVector2f AvgNodePosition(0.0f, 0.0f);

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		AvgNodePosition.X += PastedNode->NodePosX;
		AvgNodePosition.Y += PastedNode->NodePosY;
	}

	float InvNumNodes = 1.0f / float(PastedNodes.Num());
	AvgNodePosition.X *= InvNumNodes;
	AvgNodePosition.Y *= InvNumNodes;

	FVector2f PasteLocation = GraphEditor->GetPasteLocation2f();
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		
		PastedNode->NodePosX = (PastedNode->NodePosX - AvgNodePosition.X) + PasteLocation.X;
		PastedNode->NodePosY = (PastedNode->NodePosY - AvgNodePosition.Y) + PasteLocation.Y;

		PastedNode->SnapToGrid(16);
	}
}

void SNiagaraOverviewGraph::ZoomToFit()
{
	GraphEditor->ZoomToFit(true);
}

void SNiagaraOverviewGraph::ZoomToFitAll()
{
	GraphEditor->ZoomToFit(false);
}


void SNiagaraOverviewGraph::OnAlignTop()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignTop();
	}
}

void SNiagaraOverviewGraph::OnAlignMiddle()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignMiddle();
	}
}

void SNiagaraOverviewGraph::OnAlignBottom()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignBottom();
	}
}

void SNiagaraOverviewGraph::OnDistributeNodesH()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnDistributeNodesH();
	}
}

void SNiagaraOverviewGraph::OnDistributeNodesV()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnDistributeNodesV();
	}
}

void SNiagaraOverviewGraph::OnNodesCreated(const FEdGraphEditAction& Action)
{
	if((Action.Action & GRAPHACTION_AddNode) != 0)
	{
		FVector2f PasteLocation = GraphEditor->GetPasteLocation2f();
		int32 NodeCounter = 0;

		for(const UEdGraphNode* NewNode : Action.Nodes)
		{
			UEdGraphNode* EditableNode = const_cast<UEdGraphNode*>(NewNode);
			EditableNode->NodePosX = PasteLocation.X + NodeCounter * 300.f;
			EditableNode->NodePosY = PasteLocation.Y;
			NodeCounter++;
		}

		if(Action.Nodes.Num() == 1)
		{
			GraphEditor->JumpToNode(Action.Nodes.Array()[0]);
		}
	}
}

#undef LOCTEXT_NAMESPACE // "NiagaraOverviewGraph"
