// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "GraphEditor.h"
#include "Widgets/SItemSelector.h"

class FNiagaraOverviewGraphViewModel;
struct FActionMenuContent;
class FMenuBuilder;
class UEdGraph;
class UEdGraphNode;

class SNiagaraOverviewGraph : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraOverviewGraph) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraOverviewGraphViewModel> InViewModel, const FAssetData& InEditedAsset);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void ViewModelSelectionChanged();

	void GraphSelectionChanged(const TSet<UObject*>& SelectedNodes);

	void PreClose();
	
	void OpenAddEmitterMenu();
	bool CanAddEmitters() const;

	/** Called to create context menu when right-clicking on graph */
	FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	void OnCreateEmptyEmitter();
	bool CanCreateStatelessEmitter() const;
	FText GetCreateStatelessEmitterTooltip() const;
	void OnCreateStatelessEmitter();
	void OnCreateComment();
	void OnClearIsolated();

	bool OnVerifyNodeTitle(const FText& NewText, UEdGraphNode* Node, FText& OutErrorMessage) const;
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Called when nodes are pasted in the System Overview View Model. */
	void NodesPasted(const TSet<UEdGraphNode*>& PastedNodes);

	/** A callback function to position nodes after they've been created */
	void OnNodesCreated(const FEdGraphEditAction& Action);
	
	/** Sets the position on a group of newly pasted nodes. */
	void PositionPastedNodes(const TSet<UEdGraphNode*>& PastedNodes);

	void ZoomToFit();
	void ZoomToFitAll();

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();

	void OnDistributeNodesH();
	void OnDistributeNodesV();
private:
	TSharedPtr<FNiagaraOverviewGraphViewModel> ViewModel;
	TSharedPtr<SGraphEditor> GraphEditor;

	bool bUpdatingViewModelSelectionFromGraph;
	bool bUpdatingGraphSelectionFromViewModel;

	int32 ZoomToFitFrameDelay;
};