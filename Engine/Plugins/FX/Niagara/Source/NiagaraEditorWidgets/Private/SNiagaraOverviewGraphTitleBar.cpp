// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraOverviewGraphTitleBar.h"

#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraSystemEmitterState.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "SNiagaraScalabilityPreviewSettings.h"
#include "SNiagaraSystemEffectTypeBar.h"
#include "UserAssetTagEditorUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraOverviewGraphTitleBar"

SNiagaraOverviewGraphTitleBar::~SNiagaraOverviewGraphTitleBar()
{
	ClearListeners();
}

void SNiagaraOverviewGraphTitleBar::Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, const FAssetData& InEditedAsset)
{
	SystemViewModel = InSystemViewModel;
	EditedAsset = InEditedAsset;

	bScalabilityModeActive = SystemViewModel.Pin()->GetScalabilityViewModel()->IsActive();
	SystemViewModel.Pin()->GetScalabilityViewModel()->OnScalabilityModeChanged().AddSP(this, &SNiagaraOverviewGraphTitleBar::OnUpdateScalabilityMode);

	AddAssetListeners();
	RebuildWidget();
}

void SNiagaraOverviewGraphTitleBar::RebuildWidget()
{
	ContainerWidget = SNew(SVerticalBox);

	// we only allow the Niagara system effect type bar on system assets
	if (bScalabilityModeActive && SystemViewModel.IsValid() && SystemViewModel.Pin()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		ContainerWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.HAlign(HAlign_Fill)
			[
				SNew(SNiagaraSystemEffectTypeBar, SystemViewModel.Pin()->GetSystem())
			]
		];
	}
	
	ContainerWidget->AddSlot()
	.HAlign(HAlign_Fill)
	.AutoHeight()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
		.HAlign(HAlign_Fill)
		[
			SNew(STextBlock)
			.Text(SystemViewModel.Pin()->GetOverviewGraphViewModel().Get(), &FNiagaraOverviewGraphViewModel::GetDisplayName)
			.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
			.Justification(ETextJustify::Center)
		]
	];

	// warning note for affected assets
	if (SystemViewModel.IsValid() && SystemViewModel.Pin()->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		ContainerWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.ColorAndOpacity(this, &SNiagaraOverviewGraphTitleBar::GetEmitterSubheaderColor)
			.Visibility(this, &SNiagaraOverviewGraphTitleBar::GetEmitterSubheaderVisibility)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraOverviewGraphTitleBar::GetEmitterSubheaderText)
				.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
				.Justification(ETextJustify::Center)
			]
		];
	}

	// usage of deprecated assets note
	if (SystemViewModel.IsValid() && SystemViewModel.Pin()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		ContainerWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.ColorAndOpacity(this, &SNiagaraOverviewGraphTitleBar::GetSystemSubheaderColor)
			.Visibility(this, &SNiagaraOverviewGraphTitleBar::GetSystemSubheaderVisibility)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraOverviewGraphTitleBar::GetSystemSubheaderText)
				.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
				.Justification(ETextJustify::Center)
			]
		];
	}
	
	// Warning about compilation mode setting and impacting iteration
	if (SystemViewModel.IsValid() && SystemViewModel.Pin()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		ContainerWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.ColorAndOpacity(this, &SNiagaraOverviewGraphTitleBar::GetSystemCompileModeHeaderColor)
			.Visibility(this, &SNiagaraOverviewGraphTitleBar::GetSystemCompileModeHeaderVisibility)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraOverviewGraphTitleBar::GetSystemCompileModeHeaderText)
				.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
				.Justification(ETextJustify::Center)
			]
		];
	}

	// Clear information about LWE & performance
	if (SystemViewModel.IsValid() && SystemViewModel.Pin()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		ContainerWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.ColorAndOpacity(this, &SNiagaraOverviewGraphTitleBar::GetStatelessInfoHeaderColor)
			.Visibility(this, &SNiagaraOverviewGraphTitleBar::GetStatelessInfoVisibility)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraOverviewGraphTitleBar::GetStatelessInfoText)
				.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
				.Justification(ETextJustify::Center)
			]
		];
	}

	if(bScalabilityModeActive)
	{
		ContainerWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.Padding(5.f)
		.AutoHeight()
		[			
			SNew(SNiagaraScalabilityPreviewSettings, *SystemViewModel.Pin()->GetScalabilityViewModel()).Visibility(EVisibility::SelfHitTestInvisible)			
		];
	}

	ChildSlot
	[
		ContainerWidget.ToSharedRef()
	];
}

void SNiagaraOverviewGraphTitleBar::OnUpdateScalabilityMode(bool bActive)
{
	bScalabilityModeActive = bActive;
	RebuildWidget();
}

FText SNiagaraOverviewGraphTitleBar::GetEmitterSubheaderText() const
{
	int32 SearchLimit = GetDefault<UNiagaraEditorSettings>()->GetAssetStatsSearchLimit();
	int32 AffectedAssets = GetEmitterAffectedAssets();
	FText LimitReachedText = AffectedAssets >= SearchLimit ? LOCTEXT("EmitterSubheaderLimitReachedText", "more than ") : FText();
	return FText::Format(LOCTEXT("EmitterSubheaderText", "Note: editing this emitter will affect {0}{1} dependent {1}|plural(one=asset,other=assets)! (across all versions)"), LimitReachedText, FText::AsNumber(AffectedAssets));
}

EVisibility SNiagaraOverviewGraphTitleBar::GetEmitterSubheaderVisibility() const
{
	return GetEmitterAffectedAssets() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FLinearColor SNiagaraOverviewGraphTitleBar::GetEmitterSubheaderColor() const
{
	return GetEmitterAffectedAssets() >= 5 ? FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.SystemOverview.AffectedAssetsWarningColor") : FLinearColor::White;
}

FText SNiagaraOverviewGraphTitleBar::GetSystemSubheaderText() const
{
	TArray<FText> DeprecatedEmitterNames;
	if (SystemViewModel.IsValid())
	{
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterModel : SystemViewModel.Pin()->GetEmitterHandleViewModels())
		{
			if (FVersionedNiagaraEmitterData* EmitterData = EmitterModel->GetEmitterHandle()->GetEmitterData())
			{
				if (FVersionedNiagaraEmitterData* ParentData = EmitterData->GetParent().GetEmitterData())
				{
					bool bHasDeprecatedParentAsset = UE::UserAssetTags::HasUserAssetTag(EmitterData->GetParent().Emitter, "Deprecated");
					
					if (bHasDeprecatedParentAsset || ParentData->bDeprecated)
					{
						DeprecatedEmitterNames.Add(FText::FromName(EmitterModel->GetName()));
					}
				}
			}
		}
	}
	return FText::Format(LOCTEXT("SystemSubheaderText", "The following emitters are marked as deprecated: {0}"), FText::Join(FText::FromString(", "), DeprecatedEmitterNames));
}

EVisibility SNiagaraOverviewGraphTitleBar::GetSystemSubheaderVisibility() const
{
	return IsUsingDeprecatedEmitter() ? EVisibility::Visible : EVisibility::Collapsed;
}

FLinearColor SNiagaraOverviewGraphTitleBar::GetSystemSubheaderColor() const
{
	return IsUsingDeprecatedEmitter() ? FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.SystemOverview.AffectedAssetsWarningColor") : FLinearColor::White;
}

EVisibility SNiagaraOverviewGraphTitleBar::GetSystemCompileModeHeaderVisibility() const
{
	if (SystemViewModel.IsValid())
	{
		const UNiagaraSystem& NiagaraSystem = SystemViewModel.Pin()->GetSystem();
		return NiagaraSystem.GetCompileForEdit() ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

FText SNiagaraOverviewGraphTitleBar::GetSystemCompileModeHeaderText() const
{
	return LOCTEXT("SystemInCompiledForPerformance", "Note: System is compiled for performance.  Edits may result in more compilation.");
}

FLinearColor SNiagaraOverviewGraphTitleBar::GetStatelessInfoHeaderColor() const
{
	if (SystemViewModel.IsValid())
	{
		const UNiagaraSystem& NiagaraSystem = SystemViewModel.Pin()->GetSystem();
		const FNiagaraSystemStateData& SystemStateData = NiagaraSystem.GetSystemStateData();
		if (SystemStateData.bRunUpdateScript)
		{
			return FLinearColor::Red;
		}
		else if (SystemStateData.bRunSpawnScript)
		{
			return FLinearColor::Yellow;
		}
		return FLinearColor::Green;
	}
	return FLinearColor::White;
}

EVisibility SNiagaraOverviewGraphTitleBar::GetStatelessInfoVisibility() const
{
	if (SystemViewModel.IsValid())
	{
		const UNiagaraSystem& NiagaraSystem = SystemViewModel.Pin()->GetSystem();
		if (NiagaraSystem.GetEmitterHandles().Num() > 0)
		{
			bool bHasEnabledEmitters = false;
			for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem.GetEmitterHandles())
			{
				if (EmitterHandle.GetIsEnabled() == false)
				{
					continue;
				}

				bHasEnabledEmitters = true;
				if (EmitterHandle.GetEmitterMode() != ENiagaraEmitterMode::Stateless)
				{
					return EVisibility::Collapsed;
				}
			}
			return bHasEnabledEmitters ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	return EVisibility::Collapsed;
}

FText SNiagaraOverviewGraphTitleBar::GetStatelessInfoText() const
{
	if (SystemViewModel.IsValid())
	{
		const UNiagaraSystem& NiagaraSystem = SystemViewModel.Pin()->GetSystem();
		const FNiagaraSystemStateData& SystemStateData = NiagaraSystem.GetSystemStateData();
		if (SystemStateData.bRunUpdateScript)
		{
			return LOCTEXT("LWEInfo_WorstPerf", "Lightweight Performance: (Normal) Spawn & update scripts are running");
		}
		else if (SystemStateData.bRunSpawnScript)
		{
			return LOCTEXT("LWEInfo_GoodPerf", "Lightweight Performance: (Good) Spawn script is running");
		}
		return LOCTEXT("LWEInfo_BestPerf", "Lightweight Performance: (Best) No scripts are running");
	}
	return FText::GetEmpty();
}

FLinearColor SNiagaraOverviewGraphTitleBar::GetSystemCompileModeHeaderColor() const
{
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.SystemOverview.AffectedAssetsWarningColor");
}

void SNiagaraOverviewGraphTitleBar::ResetAssetCount(const FAssetData&)
{
	EmitterAffectedAssets.Reset();
}

void SNiagaraOverviewGraphTitleBar::AddAssetListeners()
{
	if (EditedAsset.IsValid())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetUpdated().AddRaw(this, &SNiagaraOverviewGraphTitleBar::ResetAssetCount);
		AssetRegistry.OnAssetAdded().AddRaw(this, &SNiagaraOverviewGraphTitleBar::ResetAssetCount);
		AssetRegistry.OnAssetRemoved().AddRaw(this, &SNiagaraOverviewGraphTitleBar::ResetAssetCount);
	}
}

void SNiagaraOverviewGraphTitleBar::ClearListeners()
{
	if (EditedAsset.IsValid())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetUpdated().RemoveAll(this);
		AssetRegistry.OnAssetAdded().RemoveAll(this);
		AssetRegistry.OnAssetRemoved().RemoveAll(this);
	}
}

int32 SNiagaraOverviewGraphTitleBar::GetEmitterAffectedAssets() const
{
	if (!EditedAsset.IsValid())
	{
		return 0;
	}

	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	if (EditorSettings->GetDisplayAffectedAssetStats() == false)
	{
		return 0;
	}
	
	if (!EmitterAffectedAssets.IsSet())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		if (AssetRegistry.IsLoadingAssets())
		{
			// We are still discovering assets, wait a bit
			return 0;
		}

		EmitterAffectedAssets = FNiagaraEditorUtilities::GetReferencedAssetCount(EditedAsset, [](const FAssetData& AssetToCheck)
		{
			if (AssetToCheck.GetClass() == UNiagaraSystem::StaticClass())
			{
				return FNiagaraEditorUtilities::ETrackAssetResult::Count;
			}
			if (AssetToCheck.GetClass() == UNiagaraEmitter::StaticClass())
			{
				return FNiagaraEditorUtilities::ETrackAssetResult::CountRecursive;
			}
			return FNiagaraEditorUtilities::ETrackAssetResult::Ignore;
		});
	}
	return EmitterAffectedAssets.Get(0);
}

bool SNiagaraOverviewGraphTitleBar::IsUsingDeprecatedEmitter() const
{
	if (SystemViewModel.IsValid())
	{
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterModel : SystemViewModel.Pin()->GetEmitterHandleViewModels())
		{
			if (FVersionedNiagaraEmitterData* EmitterData = EmitterModel->GetEmitterHandle()->GetEmitterData())
			{
				if(FVersionedNiagaraEmitterData* ParentData = EmitterData->GetParent().GetEmitterData())
				{
					bool bHasDeprecatedParentAsset = UE::UserAssetTags::HasUserAssetTag(EmitterData->GetParent().Emitter, "Deprecated");
					return ParentData->bDeprecated || bHasDeprecatedParentAsset;
				}
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
