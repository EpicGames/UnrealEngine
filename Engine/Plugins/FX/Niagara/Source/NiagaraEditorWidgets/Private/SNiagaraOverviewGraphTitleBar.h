// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraSystemScalabilityViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/SCompoundWidget.h"
//#include "SNiagaraOverviewGraph.h"

class SNiagaraOverviewGraph;
class SVerticalBox;
struct FAssetData;

class SNiagaraOverviewGraphTitleBar : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraOverviewGraphTitleBar)
		: _OverviewGraph(nullptr)
	{}
		SLATE_ARGUMENT(TSharedPtr<SNiagaraOverviewGraph>, OverviewGraph)
	SLATE_END_ARGS()

public:
	virtual ~SNiagaraOverviewGraphTitleBar() override;
	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, const FAssetData& InEditedAsset);

	void RebuildWidget();
	//virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	bool bScalabilityModeActive = false;
	TSharedPtr<SVerticalBox> ContainerWidget;
	TWeakPtr<SNiagaraOverviewGraph> OverviewGraph;

	void OnUpdateScalabilityMode(bool bActive);

	FText GetEmitterSubheaderText() const;
	EVisibility GetEmitterSubheaderVisibility() const;
	FLinearColor GetEmitterSubheaderColor() const;

	FText GetSystemSubheaderText() const;
	EVisibility GetSystemSubheaderVisibility() const;
	FLinearColor GetSystemSubheaderColor() const;
	
	EVisibility GetSystemCompileModeHeaderVisibility() const;
	FText GetSystemCompileModeHeaderText() const;
	FLinearColor GetSystemCompileModeHeaderColor() const;

	FLinearColor GetStatelessInfoHeaderColor() const;
	EVisibility GetStatelessInfoVisibility() const;
	FText GetStatelessInfoText() const;

	int32 GetEmitterAffectedAssets() const;
	bool IsUsingDeprecatedEmitter() const;
	void ResetAssetCount(const FAssetData& InAssetData);
	void AddAssetListeners();
	void ClearListeners();

	mutable TOptional<int32> EmitterAffectedAssets;
	FAssetData EditedAsset;
};
