// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraOverviewInlineParameterBox.h"
#include "Engine/Texture2D.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Logging/StructuredLog.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSystem.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "Styling/StyleColors.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SNiagaraOverviewInlineParameterBox"

void SNiagaraOverviewInlineParameterBox::Construct(const FArguments& InArgs, UNiagaraStackModuleItem& InStackModuleItem)
{
	ModuleItem = &InStackModuleItem;

	ChildSlot
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SAssignNew(Container, SScrollBox)
		.Orientation(EOrientation::Orient_Horizontal)
		.ScrollBarThickness(FVector2D(2.f, 2.f))
		.ScrollBarVisibility(EVisibility::Collapsed)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
	];

	ConstructChildren();
	
	ModuleItem->OnStructureChanged().AddSP(this, &SNiagaraOverviewInlineParameterBox::OnModuleItemStructureChanged);
}

SNiagaraOverviewInlineParameterBox::~SNiagaraOverviewInlineParameterBox()
{
	for(TWeakObjectPtr<UNiagaraStackFunctionInput> BoundInput : BoundFunctionInputs)
	{
		if(BoundInput.IsValid())
		{
			BoundInput->OnValueChanged().RemoveAll(this);
		}		
	}
	
	BoundFunctionInputs.Empty();
}

FReply SNiagaraOverviewInlineParameterBox::NavigateToStack(TWeakObjectPtr<UNiagaraStackFunctionInput> FunctionInput)
{
	// even if we can't navigate, the button should consume the input
	if(!ModuleItem.IsValid() || !FunctionInput.IsValid() || FunctionInput->IsFinalized())
	{
		return FReply::Handled();
	}
	
	ModuleItem->GetSystemViewModel()->GetSelectionViewModel()->UpdateSelectedEntries({ModuleItem.Get()}, {}, true);

	// we attempt to find a substitute entry, i.e. if our current input is an inline edit toggle, we try to find the input that the toggle is handling instead
	TWeakObjectPtr<UNiagaraStackFunctionInput> SubstituteEntry = FindSubstituteEntry(FunctionInput.Get());
	if(SubstituteEntry.IsValid())
	{
		FunctionInput = SubstituteEntry;
	}

	FNiagaraStackEntrySelectionConfig Config;
	Config.bScrollToAndExpand = true;
	Config.bSelectRow = true;
	Config.bPulse = true;
	ModuleItem->GetSystemViewModel()->GetSelectionViewModel()->GetSelectionStackViewModel()->SelectStackEntry(FunctionInput.Get(), Config);
	
	return FReply::Handled();
}

void SNiagaraOverviewInlineParameterBox::ConstructChildren()
{
	Container->ClearChildren();
	ImageBrushes.Empty();
	
	for(TWeakObjectPtr<UNiagaraStackFunctionInput> BoundInput : BoundFunctionInputs)
	{
		if(BoundInput.IsValid())
		{
			BoundInput->OnValueChanged().RemoveAll(this);
		}
	}
	BoundFunctionInputs.Empty();
	
	TArray<TSharedRef<SWidget>> ParameterWidgets = GenerateParameterWidgets();
	
	for (TSharedRef<SWidget> ParameterWidget : ParameterWidgets)
	{
		Container->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Padding(FMargin(1.f, 0.f))
		[
			ParameterWidget
		];
	}

	SetVisibility(ParameterWidgets.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);
	Container->SetEnabled(ModuleItem->GetIsEnabledAndOwnerIsEnabled());
}

TArray<TSharedRef<SWidget>> SNiagaraOverviewInlineParameterBox::GenerateParameterWidgets()
{
	TArray<TSharedRef<SWidget>> ParameterWidgets;

	// the stack module item should be valid at this point, but we check to make sure
	if(!ModuleItem.IsValid())
	{
		return ParameterWidgets;
	}
	
	UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(ModuleItem->GetModuleNode().GetFunctionScriptSource());
	// in case the script source is no longer valid (i.e. the asset was deleted), we return immediately
	if(ScriptSource == nullptr)
	{
		return ParameterWidgets;
	}

	// we cache the sort order of the widgets we create to sort them after all widgets have been created
	TMap<TSharedRef<SWidget>, int32> SortOrder;
	
	TArray<UNiagaraStackFunctionInput*> FunctionInputs = ModuleItem->GetInlineParameterInputs();
	for(UNiagaraStackFunctionInput* FunctionInput : FunctionInputs)
	{
		FunctionInput->OnValueChanged().RemoveAll(this);
		FunctionInput->OnValueChanged().AddSP(this, &SNiagaraOverviewInlineParameterBox::ConstructChildren);
		BoundFunctionInputs.Add(FunctionInput);
		
		TSharedPtr<SWidget> Widget;
		switch (FunctionInput->GetValueMode())
		{
			case UNiagaraStackFunctionInput::EValueMode::Local:
				Widget = GenerateParameterWidgetFromLocalValue(FunctionInput);
				break;
		case UNiagaraStackFunctionInput::EValueMode::Dynamic:
				Widget = GenerateParameterWidgetFromDynamicInput(FunctionInput);
				break;
			case UNiagaraStackFunctionInput::EValueMode::Data:
				Widget = GenerateParameterWidgetFromDataInterface(FunctionInput);
				break;
			case UNiagaraStackFunctionInput::EValueMode::ObjectAsset:
				Widget = SNullWidget::NullWidget;
				break;
			default:
				break;
		}

		if(Widget == SNullWidget::NullWidget)
		{
			continue;
		}

		SortOrder.Add(Widget.ToSharedRef(), FunctionInput->GetInputMetaData()->InlineParameterSortPriority);
		ParameterWidgets.Add(Widget.ToSharedRef());
	}
	
	ParameterWidgets.Sort([&](TSharedRef<SWidget> WidgetA, TSharedRef<SWidget> WidgetB)
	{
		return SortOrder[WidgetA] < SortOrder[WidgetB];
	});
	
	return ParameterWidgets;
}

TSharedRef<SWidget> SNiagaraOverviewInlineParameterBox::GenerateParameterWidgetFromLocalValue(UNiagaraStackFunctionInput* FunctionInput)
{
	bool bGenerateProperWidget = true;
	
	FNiagaraTypeDefinition Type = FunctionInput->GetInputType();
	TOptional<FNiagaraVariableMetaData> InputMetaData = FunctionInput->GetInputMetaData();
	TSharedPtr<INiagaraEditorTypeUtilities> TypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(Type);

	// we construct a variable with the data from our local struct memory because type utilities don't work on raw data
	FNiagaraVariable Variable(Type, FunctionInput->GetInputParameterHandle().GetParameterHandleString());
	Variable.SetData(FunctionInput->GetLocalValueStruct()->GetStructMemory());
	
	// by default we display type information. In some cases we have overrides, but even then we want to keep the type information as we continue using it in the tooltips
	const FText ValueText = TypeUtilities->GetStackDisplayText(Variable);
	FText DisplayedText = ValueText;
	
	TSharedPtr<IToolTip> TooltipOverride;
	
	FLinearColor TypeColor = GetTypeColor(FunctionInput);
	FLinearColor DisplayedColor = DetermineBackgroundColor(FunctionInput);

	UTexture2D* Icon = nullptr;
	if (const UEnum* Enum = Type.GetEnum())
	{
		int32 EnumIndex = Enum->GetIndexByValue(*(int32*)FunctionInput->GetLocalValueStruct()->GetStructMemory());
		if(InputMetaData->InlineParameterEnumOverrides.IsValidIndex(EnumIndex))
		{
			const FNiagaraEnumParameterMetaData& EnumParameterMetaData = InputMetaData->InlineParameterEnumOverrides[EnumIndex];

			// we only want to use the override name if it has been set
			DisplayedText = !EnumParameterMetaData.OverrideName.IsNone() ? FText::FromName(EnumParameterMetaData.OverrideName) : DisplayedText;			
			Icon = EnumParameterMetaData.IconOverride;
			DisplayedColor = EnumParameterMetaData.bUseColorOverride ? EnumParameterMetaData.ColorOverride : DisplayedColor;
		}
	}
	else if(Type == FNiagaraTypeDefinition::GetIntDef() && InputMetaData->WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle)
	{
		int32 Value = *(int32*)FunctionInput->GetLocalValueStruct()->GetStructMemory();
		if(InputMetaData->WidgetCustomization.EnumStyleDropdownValues.IsValidIndex(Value))
		{
			FString ValueString = FString::FromInt(Value);
			FText DisplayName = InputMetaData->WidgetCustomization.EnumStyleDropdownValues[Value].DisplayName;
			FText TooltipActualValueText = FText::FormatOrdered(FText::FromString("({0})"), FText::FromString(ValueString));
			DisplayedText = DisplayName.IsEmptyOrWhitespace() == false ? DisplayName : FText::FromString(ValueString);
			
			TSharedRef<SWidget> ValueOverrideTooltip = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IntAsEnumParameterTooltipTextOverride", "Value: "))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(DisplayedText)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(TooltipActualValueText)
			];
			
			TooltipOverride = FNiagaraParameterUtilities::GetTooltipWidget(Variable, false, ValueOverrideTooltip);
		}
	}
	else if(Type == FNiagaraTypeDefinition::GetBoolDef() && InputMetaData->bEnableBoolOverride)
	{
		const FNiagaraBoolParameterMetaData& BoolParameterMetaData = InputMetaData->InlineParameterBoolOverride;

		int32 Val = *(int32*)FunctionInput->GetLocalValueStruct()->GetStructMemory();
		bool bParameterValue = Val == 0 ? false : true;

		if(bParameterValue == true)
		{
			DisplayedText = !BoolParameterMetaData.OverrideNameTrue.IsNone() ? FText::FromName(BoolParameterMetaData.OverrideNameTrue) : DisplayedText;			
			Icon = BoolParameterMetaData.IconOverrideTrue;
		}
		else
		{
			DisplayedText = !BoolParameterMetaData.OverrideNameFalse.IsNone() ? FText::FromName(BoolParameterMetaData.OverrideNameFalse) : DisplayedText;			
			Icon = BoolParameterMetaData.IconOverrideFalse;
		}
		
		if(BoolParameterMetaData.DisplayMode != ENiagaraBoolDisplayMode::DisplayAlways)
		{
			// in case we only want to display the variable in one case, we make sure to test against the correct value
			bool bDisplayBool = BoolParameterMetaData.DisplayMode == ENiagaraBoolDisplayMode::DisplayIfTrue ? true : false;  
			if(bDisplayBool != bParameterValue)
			{
				bGenerateProperWidget = false;
			}			
		}		
	}
	else if(Type == FNiagaraTypeDefinition::GetColorDef())
	{
		DisplayedText = FText::FromName(FunctionInput->GetInputParameterHandle().GetName());
		DisplayedColor = *(FLinearColor*)FunctionInput->GetLocalValueStruct()->GetStructMemory();
	}

	TSharedPtr<SWidget> ParameterWidget;

	// we might have cases in which we don't want to display a parameter's data, such as if we only want to display a bool parameter when it is set to True
	if(bGenerateProperWidget)
	{
		TWeakObjectPtr<UNiagaraStackFunctionInput> FunctionInputWeak = FunctionInput;
		
		TSharedRef<SButton> ParameterWidgetButton = SNew(SButton)
		.Text(DisplayedText)
		.TextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.SystemOverview.InlineParameterText"))
		.ButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>("NiagaraEditor.SystemOverview.InlineParameterButton"))
		.ContentPadding(FMargin(0.f))
		.OnClicked(this, &SNiagaraOverviewInlineParameterBox::NavigateToStack, FunctionInputWeak);

		// if we have an icon available, we use it instead of any text. Border color is irrelevant here.
		if(Icon != nullptr)
		{
			FSlateImageBrush& ImageBrush = ImageBrushes.Emplace_GetRef(Icon, FVector2D(16.f, 16.f));
			TSharedRef<SImage> ParameterWidgetIcon= SNew(SImage).Image(&ImageBrush);
			ParameterWidgetButton->SetContent(ParameterWidgetIcon);
			ParameterWidgetButton->SetButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>("NiagaraEditor.SystemOverview.InlineParameterButton.Icon"));
		}
		// otherwise, we use text and set the border to its specified color (type, override, instance override color)
		else
		{
			// if we are using a custom color, we set the button style to use a white base in order to get rid of a tint
			if(DisplayedColor != TypeColor)
			{
				ParameterWidgetButton->SetButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>("NiagaraEditor.SystemOverview.InlineParameterButton.NoTint"));
			}
			// if we are an enum and haven't overridden the color, we use a style that better fits the text
			else if(Type.GetEnum())
			{
				ParameterWidgetButton->SetButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>("NiagaraEditor.SystemOverview.InlineParameterButton.Enum"));
			}
			
			ParameterWidgetButton->SetBorderBackgroundColor(DisplayedColor);
		}			

		ParameterWidget = ParameterWidgetButton;
	}
	else
	{
		return SNullWidget::NullWidget;
	}
	
	// we construct a tooltip widget that shows the parameter the value is associated with
	if(TooltipOverride != nullptr)
	{
		ParameterWidget->SetToolTip(TooltipOverride);
	}
	else
	{
		TSharedRef<SToolTip> TooltipWidget = FNiagaraParameterUtilities::GetTooltipWidget(Variable);
		ParameterWidget->SetToolTip(TooltipWidget);
	}
	return ParameterWidget.ToSharedRef();
}

TSharedRef<SWidget> SNiagaraOverviewInlineParameterBox::GenerateParameterWidgetFromDynamicInput(UNiagaraStackFunctionInput* FunctionInput)
{
	if(!ensure(FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic))
	{
		UE_LOGFMT(LogNiagaraEditor, Verbose, "Wrong input configuration for {0}", __FUNCTION__);
		return SNullWidget::NullWidget;
	}
	
	if(!ensure(FunctionInput->GetInputFunctionCallNode().GetScriptData() != nullptr))
	{
		UE_LOGFMT(LogNiagaraEditor, Verbose, "Invalid dynamic input script {0}", __FUNCTION__);
		return SNullWidget::NullWidget;
	}

	if(!ensure(FunctionInput->GetDynamicInputNode() != nullptr && FunctionInput->GetDynamicInputNode()->GetScriptData() != nullptr))
	{
		UE_LOGFMT(LogNiagaraEditor, Verbose, "Invalid dynamic input script {0}", __FUNCTION__);
		return SNullWidget::NullWidget;
	}

	FVersionedNiagaraScriptData* ScriptData = FunctionInput->GetDynamicInputNode()->GetScriptData();

	if(ScriptData->InlineOverviewDisplayName.IsEmptyOrWhitespace() == false)
	{
		TWeakObjectPtr<UNiagaraStackFunctionInput> FunctionInputWeak = FunctionInput;

		TSharedRef<SButton> ParameterWidgetButton = SNew(SButton)
		.Text(ScriptData->InlineOverviewDisplayName)
		.TextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.SystemOverview.InlineParameterText"))
		.ButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>("NiagaraEditor.SystemOverview.InlineParameterButton"))
		.ContentPadding(FMargin(0.f))
		.OnClicked(this, &SNiagaraOverviewInlineParameterBox::NavigateToStack, FunctionInputWeak);

		ParameterWidgetButton->SetBorderBackgroundColor(DetermineBackgroundColor(FunctionInput));

		FNiagaraTypeDefinition Type = FunctionInput->GetInputType();
		FNiagaraVariable Variable(Type, FunctionInput->GetInputParameterHandle().GetParameterHandleString());

		TSharedRef<SVerticalBox> DynamicInputInformationBox = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(4.f)
				.AutoWidth()
				[
					SNew(STextBlock).Text(LOCTEXT("DynamicInputLabelKey", "Dynamic Input: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(4.f)
				.AutoWidth()
				[
					SNew(STextBlock).Text(FText::FormatOrdered(LOCTEXT("DynamicInputLabelValue", "{0}"), FText::AsCultureInvariant(GetNameSafe(FunctionInput->GetDynamicInputNode()->FunctionScript))))
				]
			];
		
		ParameterWidgetButton->SetToolTip(FNiagaraParameterUtilities::GetTooltipWidget(Variable, false, DynamicInputInformationBox));
		return ParameterWidgetButton;
	}

	return SNullWidget::NullWidget;	
}

TSharedRef<SWidget> SNiagaraOverviewInlineParameterBox::GenerateParameterWidgetFromDataInterface(UNiagaraStackFunctionInput* FunctionInput)
{
	// if(DataInterfaceInput.DataInterface.IsValid())
	// {
	// 	DataInterfaceInput.OnValueChanged.RemoveAll(this);
	// 	DataInterfaceInput.OnValueChanged.AddSP(this, &SNiagaraOverviewInlineParameterBox::ConstructChildren);
	// 	TArray<FNiagaraDataInterfaceError> Errors;
	// 	TArray<FNiagaraDataInterfaceFeedback> Warnings;
	// 	TArray<FNiagaraDataInterfaceFeedback> Infos;
	//
	// 	// these can be nullptr depending on context
	// 	UNiagaraSystem* System = DataInterfaceInput.DataInterface->GetTypedOuter<UNiagaraSystem>();
	// 	UNiagaraComponent* Component = DataInterfaceInput.DataInterface->GetTypedOuter<UNiagaraComponent>();
	//
	// 	//@TODO Finish this up.
	// 	//DataInterfaceInput.DataInterface->GetInlineFeedback(System, Component, Errors, Warnings, Infos);
	//
	// 	TSharedPtr<SHorizontalBox> InfoBox = SNew(SHorizontalBox);
	// 	for(FNiagaraDataInterfaceFeedback& Feedback : Infos)
	// 	{
	// 		InfoBox->AddSlot()
	// 		.AutoWidth()
	// 		[
	// 			SNew(STextBlock)
	// 			.Text(Feedback.GetFeedbackSummaryText())
	// 			.ToolTipText(Feedback.GetFeedbackText())
	// 			.TextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.SystemOverview.InlineParameterText"))
	// 		];			
	// 	}
	//
	// 	return InfoBox.ToSharedRef();
	//}

	return SNullWidget::NullWidget;
}

TWeakObjectPtr<UNiagaraStackFunctionInput> SNiagaraOverviewInlineParameterBox::FindSubstituteEntry(const UNiagaraStackFunctionInput* InInput)
{
	// there can only be substitute entries for inline edit condition toggles
	if(!InInput->GetInputMetaData().IsSet() || !InInput->GetInputMetaData()->bInlineEditConditionToggle)
	{
		return nullptr;
	}
	
	TArray<UNiagaraStackFunctionInput*> AllInputs;
	ModuleItem->GetParameterInputs(AllInputs);

	// we gather all edit conditions here so we can later map them back
	for(UNiagaraStackFunctionInput* Input : AllInputs)
	{
		if(Input->GetHasEditCondition())
		{
			if(Input->GetEditConditionVariable().IsSet())
			{
				FNiagaraParameterHandle EditHandle(Input->GetEditConditionVariable().GetValue().GetName());

				if(InInput->GetInputParameterHandle() == EditHandle)
				{
					return Input;
				}
			}
		}
	}

	return nullptr;
}

void SNiagaraOverviewInlineParameterBox::OnModuleItemStructureChanged(ENiagaraStructureChangedFlags)
{
	ConstructChildren();
}

FLinearColor SNiagaraOverviewInlineParameterBox::GetTypeColor(UNiagaraStackFunctionInput* FunctionInput) const
{
	FNiagaraTypeDefinition Type = FunctionInput->GetInputType();
	TOptional<FNiagaraVariableMetaData> InputMetaData = FunctionInput->GetInputMetaData();

	// by default we display type information. In some cases we have overrides, but even then we want to keep the type information as we continue using it in the tooltips
	const FLinearColor OriginalTypeColor = UEdGraphSchema_Niagara::GetTypeColor(Type);
	FLinearColor ActualTypeColor = OriginalTypeColor;

	// if we have an enum type or integer acting as an enum, we want to override the type color to be gray instead
	if(Type.GetEnum() || (Type == FNiagaraTypeDefinition::GetIntDef() && InputMetaData.IsSet() && InputMetaData->WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle))
	{
		ActualTypeColor = FLinearColor(0.02f, 0.02f, 0.02f, 1.f);
	}

	return ActualTypeColor;
}

FLinearColor SNiagaraOverviewInlineParameterBox::DetermineBackgroundColor(UNiagaraStackFunctionInput* FunctionInput) const
{
	TOptional<FNiagaraVariableMetaData> InputMetaData = FunctionInput->GetInputMetaData();	
	return InputMetaData->bOverrideColor ? InputMetaData->InlineParameterColorOverride : GetTypeColor(FunctionInput);
}

#undef LOCTEXT_NAMESPACE
