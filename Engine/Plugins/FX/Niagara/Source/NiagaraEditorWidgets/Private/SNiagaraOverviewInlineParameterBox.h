// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateImageBrush.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"

class SWrapBox;
class SScrollBox;
class UNiagaraStackViewModel;
class UNiagaraStackEntry;
class SBox;
class SInlineEditableTextBlock;

class SNiagaraOverviewInlineParameterBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraOverviewInlineParameterBox)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
	
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackModuleItem& InStackModuleItem);

	virtual ~SNiagaraOverviewInlineParameterBox() override;

private:
	/** Selects the module item in the stack and searches for the function input */
	FReply NavigateToStack(TWeakObjectPtr<UNiagaraStackFunctionInput> FunctionInput);
	void ConstructChildren();
	TArray<TSharedRef<SWidget>> GenerateParameterWidgets();
	TSharedRef<SWidget> GenerateParameterWidgetFromLocalValue(UNiagaraStackFunctionInput* FunctionInput);
	TSharedRef<SWidget> GenerateParameterWidgetFromDynamicInput(UNiagaraStackFunctionInput* FunctionInput);
	TSharedRef<SWidget> GenerateParameterWidgetFromDataInterface(UNiagaraStackFunctionInput* FunctionInput);
	
	/** We build a substitute map of entries that we want to */
	TWeakObjectPtr<UNiagaraStackFunctionInput> FindSubstituteEntry(const UNiagaraStackFunctionInput* Input);

	void OnModuleItemStructureChanged(ENiagaraStructureChangedFlags);

	/** Gets the color based on type. */
	FLinearColor GetTypeColor(UNiagaraStackFunctionInput* FunctionInput) const;

	/** Determines the color for a function input. This is usually based on type, but can be overridden and handles special cases. */
	FLinearColor DetermineBackgroundColor(UNiagaraStackFunctionInput* FunctionInput) const;
private:
	/** The module item whose parameters this box is representing */
	TWeakObjectPtr<UNiagaraStackModuleItem> ModuleItem;
	
	TSharedPtr<SScrollBox> Container;
	
	/** We keep track of the function inputs we are observing through this parameter box so we can unbind the delegates later */
	TArray<TWeakObjectPtr<UNiagaraStackFunctionInput>> BoundFunctionInputs;

	TArray<FSlateImageBrush> ImageBrushes;
};
