// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraOverviewStack.h"

#include "IDocumentation.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "ScopedTransaction.h"
#include "SNiagaraOverviewInlineParameterBox.h"
#include "SNiagaraStack.h"
#include "Stack/SNiagaraStackInheritanceIcon.h"
#include "Stack/SNiagaraStackIssueIcon.h"
#include "Stack/SNiagaraStackItemGroupAddButton.h"
#include "Stack/SNiagaraStackRowPerfWidget.h"
#include "Styling/StyleColors.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "SNiagaraStack.h"
#include "Stack/SNiagaraStackItemGroupAddButton.h"
#include "Stack/SNiagaraStackIssueIcon.h"
#include "Stack/SNiagaraStackRowPerfWidget.h"
#include "NiagaraStackCommandContext.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorModule.h"
#include "NiagaraNodeAssignment.h"
#include "Widgets/SNiagaraParameterName.h"
#include "SNiagaraOverviewInlineParameterBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "ScopedTransaction.h"
#include "SNiagaraStackNote.h"
#include "Styling/StyleColors.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackSystemSettingsGroup.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "NiagaraOverviewStack"

class SNiagaraSystemOverviewEntryListRow : public STableRow<UNiagaraStackEntry*>
{
	SLATE_BEGIN_ARGS(SNiagaraSystemOverviewEntryListRow)
		: _Style(&GetDefaultStyle())
		{}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_EVENT(FOnTableRowDragLeave, OnDragLeave)
		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_DEFAULT_SLOT(FArguments, Content);
	SLATE_END_ARGS();

	virtual ~SNiagaraSystemOverviewEntryListRow() override
	{
		StackEntry->OnStructureChanged().RemoveAll(this);
		ScalabilityViewModel->OnScalabilityModeChanged().RemoveAll(this);
	}
	
	void Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel, UNiagaraStackEntry* InStackEntry, TSharedRef<FNiagaraStackCommandContext> InStackCommandContext, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		StackViewModel = InStackViewModel;
		StackEntry = InStackEntry;
		StackCommandContext = InStackCommandContext;
		FSlateColor IconColor = FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetColorNameForExecutionCategory(StackEntry->GetExecutionCategoryName()));

		ScalabilityViewModel = InStackEntry->GetSystemViewModel()->GetScalabilityViewModel();
		bScalabilityModeActive = ScalabilityViewModel->IsActive();
		ScalabilityViewModel->OnScalabilityModeChanged().AddSP(this, &SNiagaraSystemOverviewEntryListRow::UpdateScalabilityMode);

		ExpandedImage = FCoreStyle::Get().GetBrush("TreeArrow_Expanded");
		CollapsedImage = FCoreStyle::Get().GetBrush("TreeArrow_Collapsed");
		
		TSharedRef<FNiagaraSystemViewModel> SystemViewModel = StackEntry->GetSystemViewModel();
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = StackEntry->GetEmitterViewModel();
		if (EmitterViewModel.IsValid())
		{
			EmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelForEmitter(EmitterViewModel->GetEmitter());
		}

		StackEntry->OnStructureChanged().AddSP(this, &SNiagaraSystemOverviewEntryListRow::InvalidateCachedData);

		TSharedPtr<SBorder> TooltipBorder;
		
		TSharedPtr<SOverlay> RowOverlay;
		STableRow<UNiagaraStackEntry*>::Construct(STableRow<UNiagaraStackEntry*>::FArguments()
			.Style(InArgs._Style)
			.OnDragDetected(InArgs._OnDragDetected)
			.OnDragLeave(InArgs._OnDragLeave)
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
		[
			SAssignNew(RowOverlay, SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(ScalabilityOverlay, SOverlay)
				+ SOverlay::Slot()
				[
					SAssignNew(TooltipBorder, SBorder)
					.BorderImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("Niagara.TableViewRowBorder"))
					.Padding(FMargin(0, 0, 0, 1))
					[
						SNew(SHorizontalBox)
						// Expand button
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Padding(0, 0, 0, 0)
						[					
							SNew(SButton)
							.ButtonStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.SimpleButton")
							.Visibility_Lambda([this]()
							{
								if (StackEntry->GetCanExpandInOverview())
								{	
									return GetOverviewChildrenCount() > 0
										? EVisibility::Visible
										: EVisibility::Hidden;
								}
								else
								{
									return EVisibility::Collapsed;
								}
							})
							.OnClicked_Lambda([this]()
							{
								const bool bWillBeExpanded = !StackEntry->GetIsExpandedInOverview();
								FText Message = FText::Format(LOCTEXT("ChangedCollapseState", "{0} {1}"), bWillBeExpanded ?
									LOCTEXT("ChangedCollapseState_Expanded", "Expanded") : LOCTEXT("ChangedCollapseState_Collapsed", "Collapsed"),
									StackEntry->GetAlternateDisplayName().IsSet() ? StackEntry->GetAlternateDisplayName().GetValue() : StackEntry->GetDisplayName());
								
								FScopedTransaction Transaction(Message);
								StackEntry->GetStackEditorData().Modify();
								
								StackEntry->SetIsExpandedInOverview(bWillBeExpanded);									
								return FReply::Handled();
							})
							.ContentPadding(FMargin(0.0f, 1.0f))
							[
								SNew(SImage)
								.Image(this, &SNiagaraSystemOverviewEntryListRow::GetExpandButtonImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
							]							
						]
						+ SHorizontalBox::Slot()
						.Padding(TAttribute<FMargin>(this, &SNiagaraSystemOverviewEntryListRow::GetInnerContentPadding))
						[
							InArgs._Content.Widget
						]
					]
				]	
			]
			// Stack Issue Highlight
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(this, &SNiagaraSystemOverviewEntryListRow::GetIsssueHighlightColor)
				.Visibility_Static(&SNiagaraOverviewStack::GetIssueIconVisibility, StackEntry)
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBox)
					.WidthOverride(2)
					.HeightOverride(10)
				]
			]
		],
		InOwnerTableView);

		if (StackEntry->IsA<UNiagaraStackItem>())
		{
			// Execution Category Item Highlight
			RowOverlay->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(IconColor)
				.Padding(FMargin(0))
				[
					SNew(SBox)
					.WidthOverride(2)
				]
			];
		}
		
		// we special case the red overlay for renderer items as they have their own scalability settings
		if(UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(StackEntry))
		{			
			ScalabilityOverlay->AddSlot()
			[
				SNew(SBorder)
				.Visibility_Lambda([this, RendererItem]()
				{
					return bScalabilityModeActive && RendererItem->IsExcludedFromScalability() && !RendererItem->IsOwningEmitterExcludedFromScalability() ? EVisibility::HitTestInvisible : EVisibility::Collapsed; 
				})
				.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.SystemOverview.ExcludedFromScalability.RendererItem"))
				.BorderBackgroundColor_Lambda([this]()
				{
					return FLinearColor(1, 1, 1, FNiagaraEditorUtilities::GetScalabilityTintAlpha(EmitterHandleViewModel.Pin()->GetEmitterHandle()));
				})
			];
		}

		if(StackEntry->UsesCustomTooltipWidget())
		{
			TooltipBorder->SetToolTip(TAttribute<TSharedPtr<IToolTip>>::Create(
				TAttribute<TSharedPtr<IToolTip>>::FGetter::CreateUObject(StackEntry, &UNiagaraStackEntry::GetCustomTooltipWidget)));
		}
		else
		{
			TooltipBorder->SetToolTipText(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(StackEntry, &UNiagaraStackEntry::GetTooltipText)));
		}
	}

	static const FTableRowStyle& GetDefaultStyle()
	{
		return FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.SystemOverview.TableViewRow.Item");
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			return FReply::Handled()
				.SetUserFocus(OwnerTablePtr.Pin()->AsWidget(), EFocusCause::Mouse)
				.CaptureMouse(SharedThis(this));
		}
		return STableRow<UNiagaraStackEntry*>::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Add the item represented by this row to the selection.
			UNiagaraSystemSelectionViewModel* Selection = StackEntry->GetSystemViewModel()->GetSelectionViewModel();
			if (Selection->ContainsEntry(StackEntry) == false)
			{
				TArray<UNiagaraStackEntry*> SelectedEntries;
				TArray<UNiagaraStackEntry*> DeselectedEntries;
				SelectedEntries.Add(StackEntry);
				bool bClearSelection = true;
				Selection->UpdateSelectedEntries(SelectedEntries, DeselectedEntries, bClearSelection);
			}

			FMenuBuilder MenuBuilder(true, StackCommandContext->GetCommands());
			bool bMenuItemsAdded = false;

			if (StackEntry->IsA<UNiagaraStackModuleItem>())
			{
				bMenuItemsAdded |= FNiagaraStackEditorWidgetsUtilities::AddStackModuleItemContextMenuActions(MenuBuilder, *CastChecked<UNiagaraStackModuleItem>(StackEntry), this->AsShared());
			}

			if (StackEntry->IsA<UNiagaraStackItem>())
			{
				bMenuItemsAdded |= FNiagaraStackEditorWidgetsUtilities::AddStackItemContextMenuActions(MenuBuilder, *CastChecked<UNiagaraStackItem>(StackEntry));
			}

			if (StackEntry->IsA<UNiagaraStackSimulationStageGroup>())
			{
				FNiagaraStackEditorWidgetsUtilities::AddStackSimulationStageGroupContextMenuActions(MenuBuilder, CastChecked<UNiagaraStackSimulationStageGroup>(StackEntry), this->AsShared());
				bMenuItemsAdded = true;
			}

			bMenuItemsAdded |= FNiagaraStackEditorWidgetsUtilities::AddStackEntryAssetContextMenuActions(MenuBuilder, *StackEntry);
			bMenuItemsAdded |= StackCommandContext->AddEditMenuItems(MenuBuilder);
			bMenuItemsAdded |= FNiagaraStackEditorWidgetsUtilities::AddStackNoteContextMenuActions(MenuBuilder, *StackEntry);
		
			if (bMenuItemsAdded)
			{
				FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
				FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
				return FReply::Handled().ReleaseMouseCapture();
			}
			return STableRow<UNiagaraStackEntry*>::OnMouseButtonUp(MyGeometry, MouseEvent).ReleaseMouseCapture();
		}
		return STableRow<UNiagaraStackEntry*>::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (UNiagaraStackModuleItem* ModuleItem = Cast<UNiagaraStackModuleItem>(StackEntry))
		{
			// if the user double-clicks a collapsed module we uncollapse it
			UNiagaraStackEditorData& StackEditorData = ModuleItem->GetSystemViewModel()->GetEditorData().GetStackEditorData();
			if (ModuleItem->GetIsEnabled() == false && StackEditorData.bHideDisabledModules)
			{
				StackEditorData.bHideDisabledModules = false;
				return FReply::Handled();
			}
			
			if (ModuleItem->OpenSourceAsset())
			{
				return FReply::Handled();
			}			
		}
		else if (UNiagaraStackRendererItem* RenderItem = Cast<UNiagaraStackRendererItem>(StackEntry))
		{
			RenderItem->OpenSourceCodeLocation();
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

private:
	FMargin GetInnerContentPadding() const
	{
		float InnerContentLeftPadding = StackEntry->IsA<UNiagaraStackItem>() ? 5 : 0;
		float InnerContentRightPadding = 1;
		return FMargin(InnerContentLeftPadding, 1, 1, 1);
	}

	const FSlateBrush* GetExpandButtonImage() const
	{
		return StackEntry->GetIsExpandedInOverview() ? ExpandedImage : CollapsedImage;
	}

	EVisibility GetNoteVisibility() const
	{
		if(StackEntry->GetStackNote() == nullptr)
		{
			return EVisibility::Collapsed;
		}
		
		return EVisibility::Visible;
	}

	uint32 GetOverviewChildrenCount() const
	{
		if (ChildrenCount.IsSet() == false)
		{
        	TArray<UNiagaraStackEntry*> Children;
        	StackEntry->GetFilteredChildrenOfTypes(Children, {UNiagaraStackItem::StaticClass()});
        	ChildrenCount = Children.Num();
        }
        	
        return ChildrenCount.GetValue();
	}

	void InvalidateCachedData(ENiagaraStructureChangedFlags Flags)
	{
		ChildrenCount.Reset();
		IssueHighlightColor.Reset();
	}

	void UpdateScalabilityMode(bool bActivated)
	{
		bScalabilityModeActive = bActivated;
	}

	FSlateColor GetIsssueHighlightColor() const
	{
		if (IssueHighlightColor.IsSet() == false)
		{
			float DesaturateAmount = .1f;
			if (StackEntry->GetTotalNumberOfErrorIssues() > 0)
			{
				IssueHighlightColor = FStyleColors::AccentRed.GetSpecifiedColor().Desaturate(DesaturateAmount);
			}
			else if (StackEntry->GetTotalNumberOfWarningIssues() > 0)
			{
				IssueHighlightColor = FStyleColors::AccentYellow.GetSpecifiedColor().Desaturate(DesaturateAmount);
			}
			else if (StackEntry->GetTotalNumberOfInfoIssues() > 0)
			{
				IssueHighlightColor = FStyleColors::AccentBlue.GetSpecifiedColor().Desaturate(DesaturateAmount);
			}
			else
			{
				IssueHighlightColor = FStyleColors::Transparent;
			}
		}
		return IssueHighlightColor.GetValue();
	}

private:
	UNiagaraStackViewModel* StackViewModel = nullptr;
	UNiagaraStackEntry* StackEntry = nullptr;
	UNiagaraSystemScalabilityViewModel* ScalabilityViewModel = nullptr;
	TSharedPtr<SOverlay> ScalabilityOverlay;
	TSharedPtr<FNiagaraStackCommandContext> StackCommandContext;
	FSlateColor ExcludedFromScalabilityColor;
	TAttribute<EVisibility> IssueIconVisibility;
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;
	const FSlateBrush* ExpandedImage = nullptr;
	const FSlateBrush* CollapsedImage = nullptr;
	bool bScalabilityModeActive = false;
	mutable TOptional<uint32> ChildrenCount;
	mutable TOptional<FSlateColor> IssueHighlightColor;
};

class SNiagaraSystemOverviewEnabledCheckBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnCheckedChanged, bool /*bIsChecked*/);

	SLATE_BEGIN_ARGS(SNiagaraSystemOverviewEnabledCheckBox) {}
		SLATE_ATTRIBUTE(bool, IsChecked)
		SLATE_EVENT(FOnCheckedChanged, OnCheckedChanged);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		IsChecked = InArgs._IsChecked;
		OnCheckedChanged = InArgs._OnCheckedChanged;

		ChildSlot
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SNiagaraSystemOverviewEnabledCheckBox::OnButtonClicked)
			.IsChecked(this, &SNiagaraSystemOverviewEnabledCheckBox::IsEnabled)
			.ToolTipText(LOCTEXT("EnableCheckBoxToolTip", "Enable or disable this item."))
		];
	}

private:
	void OnButtonClicked(ECheckBoxState InNewState)
	{
		OnCheckedChanged.ExecuteIfBound(IsChecked.IsBound() && !IsChecked.Get());
	}

	ECheckBoxState IsEnabled() const
	{
		if (IsChecked.IsBound())
		{
			return IsChecked.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Unchecked;
	}

private:
	TAttribute<bool> IsChecked;
	FOnCheckedChanged OnCheckedChanged;
};

class SNiagaraSystemOverviewItemName : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraSystemOverviewItemName) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackItem* InStackItem)
	{
		StackItem = InStackItem;
		StackItem->OnAlternateDisplayNameChanged().AddSP(this, &SNiagaraSystemOverviewItemName::UpdateFromAlternateDisplayName);

		if (StackItem->IsA<UNiagaraStackModuleItem>() &&
			CastChecked<UNiagaraStackModuleItem>(StackItem)->GetModuleNode().IsA<UNiagaraNodeAssignment>())
		{
			UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(&CastChecked<UNiagaraStackModuleItem>(StackItem)->GetModuleNode());
			AssignmentNode->OnAssignmentTargetsChanged().AddSP(this, &SNiagaraSystemOverviewItemName::UpdateFromAssignmentTargets);
		}

		UpdateContent();
	}

	~SNiagaraSystemOverviewItemName()
	{
		if (StackItem.IsValid() && StackItem->IsFinalized() == false)
		{
			StackItem->OnAlternateDisplayNameChanged().RemoveAll(this);

			if (StackItem->IsA<UNiagaraStackModuleItem>() &&
				CastChecked<UNiagaraStackModuleItem>(StackItem)->GetModuleNode().IsA<UNiagaraNodeAssignment>())
			{
				UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(&CastChecked<UNiagaraStackModuleItem>(StackItem)->GetModuleNode());
				AssignmentNode->OnAssignmentTargetsChanged().RemoveAll(this);
			}
		}
	}

private:
	void UpdateContent()
	{
		UNiagaraStackModuleItem* ModuleItem = Cast<UNiagaraStackModuleItem>(StackItem);
		UNiagaraNodeAssignment* AssignmentNode = ModuleItem != nullptr 
			? Cast<UNiagaraNodeAssignment>(&ModuleItem->GetModuleNode())
			: nullptr;

		if (StackItem->GetAlternateDisplayName().IsSet() == false &&
			AssignmentNode != nullptr &&
			AssignmentNode->GetAssignmentTargets().Num() > 0)
		{
			NameTextBlock.Reset();

			TSharedPtr<SWidget> ParameterWidget;
			if (AssignmentNode->GetAssignmentTargets().Num() == 1)
			{
				ParameterWidget = SNew(SNiagaraParameterName)
					.ParameterName(AssignmentNode->GetAssignmentTargets()[0].GetName())
					.IsReadOnly(true);
			}
			else
			{
				TSharedRef<SVerticalBox> ParameterBox = SNew(SVerticalBox);
				for (const FNiagaraVariable& AssignmentTarget : AssignmentNode->GetAssignmentTargets())
				{
					ParameterBox->AddSlot()
					.AutoHeight()
					.Padding(0.0f, 1.0f, 0.0f, 1.0f)
					[
						SNew(SScaleBox)
						.UserSpecifiedScale(0.85f)
						.Stretch(EStretch::UserSpecified)
						.HAlign(HAlign_Left)
						[
							SNew(SNiagaraParameterName)
							.ParameterName(AssignmentTarget.GetName())
							.IsReadOnly(true)
						]
					];
				}
				ParameterWidget = ParameterBox;
			}

			ChildSlot
			[
				SNew(SHorizontalBox)
				.IsEnabled_UObject(StackItem.Get(), &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 6.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.OverviewStack.ItemText")
					.Text(LOCTEXT("SetVariablesPrefix", "Set:"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				[
					ParameterWidget.ToSharedRef()
				]
			];
		}
		else
		{
			ChildSlot
			[
				SAssignNew(NameTextBlock, STextBlock)
				.Text(this, &SNiagaraSystemOverviewItemName::GetItemDisplayName)
				.IsEnabled_UObject(StackItem.Get(), &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

			if(StackItem->UsesCustomTooltipWidget())
			{
				NameTextBlock->SetToolTip(TAttribute<TSharedPtr<IToolTip>>::Create(
					TAttribute<TSharedPtr<IToolTip>>::FGetter::CreateUObject(StackItem.Get(), &UNiagaraStackEntry::GetCustomTooltipWidget)));
			}
			else
			{
				NameTextBlock->SetToolTipText(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(StackItem.Get(), &UNiagaraStackEntry::GetTooltipText)));
			}
			
			UpdateTextStyle();
		}
	}

	void UpdateTextStyle()
	{
		if (NameTextBlock.IsValid())
		{
			if (StackItem->GetAlternateDisplayName().IsSet())
			{
				NameTextBlock->SetTextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.SystemOverview.AlternateItemText"));
			}
			else
			{
				NameTextBlock->SetTextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.OverviewStack.ItemText"));
			}
		}
	}

	FText GetItemDisplayName() const
	{
		if (StackItem.IsValid() && StackItem->IsFinalized() == false)
		{
			return StackItem->GetAlternateDisplayName().IsSet() ? StackItem->GetAlternateDisplayName().GetValue() : StackItem->GetDisplayName();
		}
		return FText::GetEmpty();
	}

	FText GetItemToolTipText() const
	{
		if (StackItem.IsValid() && StackItem->IsFinalized() == false)
		{
			FText CurrentToolTip = StackItem->GetTooltipText();
			if (ToolTipCache.IsSet() == false || LastToolTipCache.IdenticalTo(CurrentToolTip) == false)
			{
				if(StackItem->GetAlternateDisplayName().IsSet())
				{
					FText AlternateNameAndOriginalName = FText::Format(LOCTEXT("AlternateNameAndOriginalNameFormat", "{0} ({1})"), StackItem->GetAlternateDisplayName().GetValue(), StackItem->GetDisplayName());
					if (CurrentToolTip.IsEmptyOrWhitespace())
					{
						ToolTipCache = AlternateNameAndOriginalName;
					}
					else
					{
						ToolTipCache = FText::Format(LOCTEXT("AlternateDisplayNameToolTipFormat", "{0}\n\n{1}"), AlternateNameAndOriginalName, StackItem->GetTooltipText());
					}
				}
				else
				{
					ToolTipCache = CurrentToolTip;
				}
			}
			LastToolTipCache = CurrentToolTip;
			return ToolTipCache.GetValue();
		}
		return FText::GetEmpty();
	}

	void UpdateFromAlternateDisplayName()
	{
		if (StackItem.IsValid() && StackItem->IsFinalized() == false)
		{
			if (StackItem->GetAlternateDisplayName().IsSet())
			{
				if (NameTextBlock.IsValid() == false)
				{
					UpdateContent();
				}
				else
				{
					UpdateTextStyle();
				}
			}
			else
			{
				if (NameTextBlock.IsValid())
				{
					UpdateContent();
				}
			}
		}
		ToolTipCache.Reset();
	}

	void UpdateFromAssignmentTargets()
	{
		if (StackItem.IsValid() && StackItem->IsFinalized() == false)
		{
			UpdateContent();
		}
		ToolTipCache.Reset();
	}

private:
	mutable FText LastToolTipCache;
	mutable TOptional<FText> ToolTipCache;
	TSharedPtr<STextBlock> NameTextBlock;
	TWeakObjectPtr<UNiagaraStackItem> StackItem;
};

void SNiagaraOverviewStack::Construct(const FArguments& InArgs, UNiagaraStackViewModel& InStackViewModel, UNiagaraSystemSelectionViewModel& InOverviewSelectionViewModel)
{
	AllowedClasses = InArgs._AllowedClasses;
	
	StackCommandContext = MakeShared<FNiagaraStackCommandContext>();

	bUpdatingOverviewSelectionFromStackSelection = false;
	bUpdatingStackSelectionFromOverviewSelection = false;

	
	StackViewModel = &InStackViewModel;
	OverviewSelectionViewModel = &InOverviewSelectionViewModel;
	OverviewSelectionViewModel->OnEntrySelectionChanged().AddSP(this, &SNiagaraOverviewStack::SystemSelectionChanged);

	ChildSlot
	[
		SAssignNew(EntryListView, SListView<UNiagaraStackEntry*>)
		.Clipping(EWidgetClipping::OnDemand)
		.ListItemsSource(&FlattenedEntryList)
		.OnGenerateRow(this, &SNiagaraOverviewStack::OnGenerateRowForEntry)
		.OnSelectionChanged(this, &SNiagaraOverviewStack::OnSelectionChanged)
		.SelectionMode(ESelectionMode::Multi)
		.OnItemToString_Debug_Static(&FNiagaraStackEditorWidgetsUtilities::StackEntryToStringForListDebug)
	];

	InStackViewModel.OnExpansionChanged().AddSP(this, &SNiagaraOverviewStack::EntryExpansionChanged);
	InStackViewModel.OnExpansionInOverviewChanged().AddSP(this, &SNiagaraOverviewStack::EntryExpansionInOverviewChanged);
	InStackViewModel.OnStructureChanged().AddSP(this, &SNiagaraOverviewStack::EntryStructureChanged);
		
	bRefreshEntryListPending = true;
	RefreshEntryList();
	SystemSelectionChanged();
}

SNiagaraOverviewStack::~SNiagaraOverviewStack()
{
	if (StackViewModel != nullptr)
	{
		StackViewModel->OnStructureChanged().RemoveAll(this);
		StackViewModel->OnExpansionInOverviewChanged().RemoveAll(this);
		StackViewModel->OnExpansionChanged().RemoveAll(this);
	}

	if (OverviewSelectionViewModel != nullptr)
	{
		OverviewSelectionViewModel->OnEntrySelectionChanged().RemoveAll(this);
	}
}

bool SNiagaraOverviewStack::SupportsKeyboardFocus() const
{
	return true;
}

FReply SNiagaraOverviewStack::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (StackCommandContext->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SNiagaraOverviewStack::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	RefreshEntryList();
}

EVisibility SNiagaraOverviewStack::GetIssueIconVisibility(UNiagaraStackEntry* StackEntry)
{
	if (StackEntry != nullptr && StackEntry->IsFinalized() == false)
	{
		bool bDisplayIcon;
		if (StackEntry->IsA<UNiagaraStackItemGroup>() && StackEntry->GetCanExpandInOverview() && StackEntry->GetIsExpandedInOverview())
		{
			// If the entry is a group and it can expand and it is expanded, we only want to show the stack issue icon if the group itself has issues.
			bDisplayIcon = StackEntry->GetIssues().Num() > 0;
		}
		else
		{
			bDisplayIcon = StackEntry->HasIssuesOrAnyChildHasIssues();
		}
		return bDisplayIcon ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

const FSlateBrush* SNiagaraOverviewStack::GetUsageIcon(UNiagaraStackEntry* StackEntry)
{
	if (StackEntry != nullptr && StackEntry->IsFinalized() == false)
	{
		bool bRead = false;
		bool bWrite = false;

		StackEntry->GetRecursiveUsages(bRead, bWrite);
		if (bRead && bWrite)
		{
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.ReadWriteIcon");
		}
		else if (bRead)
		{
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.ReadIcon");
		}
		else if (bWrite)
		{
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.WriteIcon");
		}
		else
		{
			return FAppStyle::Get().GetBrush("WhiteBrush");
		}
	}
	return FAppStyle::Get().GetBrush("WhiteBrush");
}


int32 SNiagaraOverviewStack::GetUsageIconWidth(UNiagaraStackEntry* StackEntry)
{
	const FSlateBrush* Brush = FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.ReadWriteIcon");
	if (Brush)
		return Brush->GetImageSize().X;
	return 20;
}

int32 SNiagaraOverviewStack::GetUsageIconHeight(UNiagaraStackEntry* StackEntry)
{
	const FSlateBrush* Brush = FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.ReadWriteIcon");
	if (Brush)
		return Brush->GetImageSize().Y;
	return 20;
}

EVisibility SNiagaraOverviewStack::GetUsageIconVisibility(UNiagaraStackEntry* StackEntry)
{
	if (StackEntry != nullptr && StackEntry->IsFinalized() == false)
	{
		bool bDisplayIcon = StackEntry->HasUsagesOrAnyChildHasUsages();
	
		return bDisplayIcon ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}


FText SNiagaraOverviewStack::GetUsageTooltip(UNiagaraStackEntry* StackEntry)
{
	if (StackEntry != nullptr && StackEntry->IsFinalized() == false)
	{
		bool bRead = false;
		bool bWrite = false;

		StackEntry->GetRecursiveUsages(bRead, bWrite);
		if (bRead && bWrite)
		{
			return LOCTEXT("UsageTooltipRW", "Reads and writes selected parameter in Parameters panel.");
		}
		else if (bRead)
		{
			return LOCTEXT("UsageTooltipR", "Reads selected parameter in Parameters panel.");
		}
		else if (bWrite)
		{
			return LOCTEXT("UsageTooltipW", "Writes selected parameter in Parameters panel.");
		}
		else
		{
			return LOCTEXT("UsageTooltipNone", "Does not touch selected parameter in Parameters panel.");
		}
	}
	return FText::GetEmpty();
}

void SNiagaraOverviewStack::AddEntriesRecursive(UNiagaraStackEntry& EntryToAdd, TArray<UNiagaraStackEntry*>& EntryList, const TArray<UClass*>& AcceptableClasses, TArray<UNiagaraStackEntry*> ParentChain)
{
	if (AcceptableClasses.ContainsByPredicate([&] (UClass* Class) { return EntryToAdd.IsA(Class); }) && EntryToAdd.GetShouldShowInOverview())
	{
		EntryList.Add(&EntryToAdd);
		EntryObjectKeyToParentChain.Add(FObjectKey(&EntryToAdd), ParentChain);

		if(!EntryToAdd.GetStackEditorData().GetStackEntryIsExpandedInOverview(EntryToAdd.GetStackEditorDataKey(), true))
		{
			return;
		}
		
		TArray<UNiagaraStackEntry*> Children;
		EntryToAdd.GetFilteredChildren(Children);
		ParentChain.Add(&EntryToAdd);
		for (UNiagaraStackEntry* Child : Children)
		{
			checkf(Child != nullptr, TEXT("Stack entry had null child."));
			AddEntriesRecursive(*Child, EntryList, AcceptableClasses, ParentChain);
		}
	}
}

void SNiagaraOverviewStack::RefreshEntryList()
{
	if (bRefreshEntryListPending)
	{
		FlattenedEntryList.Empty();
		EntryObjectKeyToParentChain.Empty();

		UNiagaraStackEntry* RootEntry = StackViewModel->GetRootEntry();
		checkf(RootEntry != nullptr, TEXT("Root entry was null."));

		TArray<UNiagaraStackEntry*> RootChildren;		
		RootEntry->GetFilteredChildren(RootChildren);
		for (UNiagaraStackEntry* RootChild : RootChildren)
		{
			TArray<UNiagaraStackEntry*> ParentChain;
			AddEntriesRecursive(*RootChild, FlattenedEntryList, AllowedClasses, ParentChain);
		}

		bRefreshEntryListPending = false;
		EntryListView->RequestListRefresh();
	}
}

void SNiagaraOverviewStack::EntryExpansionChanged()
{
	bRefreshEntryListPending = true;
}

void SNiagaraOverviewStack::EntryExpansionInOverviewChanged()
{
	bRefreshEntryListPending = true;
}

void SNiagaraOverviewStack::EntryStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	bRefreshEntryListPending = true;
}

TSharedRef<ITableRow> SNiagaraOverviewStack::OnGenerateRowForEntry(UNiagaraStackEntry* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	FVector2D IconSize = FNiagaraEditorWidgetsStyle::Get().GetVector("NiagaraEditor.Stack.IconSize");
	TSharedPtr<SWidget> Content;
	const FTableRowStyle* RowStyle = &SNiagaraSystemOverviewEntryListRow::GetDefaultStyle();
	if (Item->IsA<UNiagaraStackItem>())
	{
		UNiagaraStackItem* StackItem = CastChecked<UNiagaraStackItem>(Item);
		TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

		FSlateColor IconColor(FColor::White);

		int32 WidthMax = SNiagaraOverviewStack::GetUsageIconWidth(Item);
		int32 HeightMax = SNiagaraOverviewStack::GetUsageIconHeight(Item);

		// Stack Usage Highlight
		ContentBox->AddSlot()
			.VAlign(VAlign_Center)
			//.AutoWidth()
			.MaxWidth(WidthMax)
			.Padding(4, 0, 0, 0)
			[
				SNew(SBorder)
				.BorderImage_Static(&SNiagaraOverviewStack::GetUsageIcon, Item)
				.Visibility_Static(&SNiagaraOverviewStack::GetUsageIconVisibility, Item)
				.ToolTipText_Static(& SNiagaraOverviewStack::GetUsageTooltip, Item)
				.BorderBackgroundColor(IconColor)
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBox)
					.WidthOverride(WidthMax)
					.HeightOverride(HeightMax)
				]
			];


		// Icon Brush
		if (StackItem->GetSupportedIconMode() == UNiagaraStackEntry::EIconMode::Brush)
		{
			ContentBox->AddSlot()
			.Padding(0, 1, 0, 1)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				.WidthOverride(IconSize.X)
				.HeightOverride(IconSize.Y)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image_UObject(StackItem, &UNiagaraStackItem::GetIconBrush)
					.DesiredSizeOverride(FVector2D(14.0f, 14.0f))
				]
			];
		}

		// Icon Text
		if (StackItem->GetSupportedIconMode() == UNiagaraStackEntry::EIconMode::Text)
		{
			ContentBox->AddSlot()
			.Padding(0, 1, 0, 1)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				.WidthOverride(IconSize.X)
				.HeightOverride(IconSize.Y)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
					.Text(StackItem->GetIconText())
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
			];
		}

		// Name
		ContentBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4, 2, 0, 2)
		[
			SNew(SNiagaraSystemOverviewItemName, StackItem)
		];

		// Issue Icon
		ContentBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		[
			SNew(SNiagaraStackIssueIcon, StackViewModel, StackItem)
			.IconMode(SNiagaraStackIssueIcon::EIconMode::Compact)
			.Visibility_Static(&SNiagaraOverviewStack::GetIssueIconVisibility, Item)
		];

		TSharedRef<SHorizontalBox> OptionsBox = SNew(SHorizontalBox);
		
		if(StackItem->SupportsStackNotes())
		{
			// Stack note
			OptionsBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SNew(SNiagaraStackInlineNote, StackItem)
				.bInteractable(false)
				.Visibility_Lambda([StackItem]()
				{
					if(StackItem->GetStackNote() == nullptr)
					{
						return EVisibility::Collapsed;
					}
	
					return EVisibility::Visible;
				})
			];
		}
		

		// Scratch icon
		UNiagaraStackModuleItem* StackModuleItem = Cast<UNiagaraStackModuleItem>(StackItem);
		if (StackModuleItem && StackModuleItem->IsScratchModule())
		{			
			OptionsBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(3, 0, 3, 1)
				[					
					SNew(SImage)
					.ToolTipText(LOCTEXT("ScratchPadOverviewTooltip", "This module is a scratch pad script."))
					.Image(FNiagaraEditorStyle::Get().GetBrush("Tab.ScratchPad"))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.DesiredSizeOverride(FVector2D(14.0f, 14.0f))
				];
		}

		TArray<FNiagaraStackItemCustomIcon> CustomIcons = StackItem->GetCustomIcons();
		for (const FNiagaraStackItemCustomIcon& CustomIcon : CustomIcons)
		{
			if (CustomIcon.IconBrush == nullptr)
			{
				continue;
			}

			OptionsBox->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(3, 0)
			[
				SNew(SBorder)
				.Visibility(CustomIcon.VisibleFunction)
				.ToolTipText(CustomIcon.ToolTipText)
				[
					SNew(SImage)
					.Image(CustomIcon.IconBrush)
				]
			];
		}

		// Debug draw 
		if (StackItem->SupportsDebugDraw())
		{
			TOptional<FText> TooltipText = StackItem->GetDebugDrawTooltip();
			OptionsBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(3, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ForegroundColor(FLinearColor::Transparent)
					.ToolTipText(TooltipText.Get(FNiagaraEditorSharedTexts::DebugDrawUIActionBaseText))
					.OnClicked(this, &SNiagaraOverviewStack::ToggleModuleDebugDraw, StackItem)
					.ContentPadding(FMargin(1.0f))
					.ButtonStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.SimpleButton")
					[
						SNew(SImage)
						.Image(this, &SNiagaraOverviewStack::GetDebugIconBrush, StackItem)
					]
				];
		}

		// Inheritance Icon
		if (StackItem->SupportsInheritance())
		{
			OptionsBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(3, 0, 0, 0)
				[
					SNew(SNiagaraStackInheritanceIcon, StackItem)
				];
		}

		// Enabled checkbox
		OptionsBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(3, 0, 0, 0)
			[
				SNew(SNiagaraSystemOverviewEnabledCheckBox)
				.Visibility(this, &SNiagaraOverviewStack::GetEnabledCheckBoxVisibility, StackItem)
				.IsEnabled_UObject(StackItem, &UNiagaraStackEntry::GetOwnerIsEnabled)
				.IsChecked_UObject(StackItem, &UNiagaraStackEntry::GetIsEnabled)
				.OnCheckedChanged_UObject(StackItem, &UNiagaraStackItem::SetIsEnabled)
			];

		// in case we have at least one inline input, we add a wrap box to make sure we have enough space for both the module name and parameters
		if(StackModuleItem && StackModuleItem->GetModuleNode().HasValidScriptAndGraph() && StackModuleItem->GetInlineParameterInputs().Num() > 0)
		{
			ContentBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SWrapBox)
				// the wrap box will put the inline box into a second row if content box + parameter box would exceed the preferred size
				.PreferredSize(250.f)
				+ SWrapBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					// despite that, we also have to cap the content box size, since the wrapbox can't wrap a single widget
					.MaxDesiredWidth(250.f)
					[
						ContentBox
					]
				]
				+ SWrapBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(3, 0, 0, 0)
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.MaxDesiredWidth(150.f)
					[
						SNew(SNiagaraOverviewInlineParameterBox, *StackModuleItem)
					]
				]
			]
			// Perf widget
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3, 0, 0, 0)
			[
				SNew(SNiagaraStackRowPerfWidget, Item, ENiagaraPerformanceStateScopeMode::Detailed)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				OptionsBox
			];
		}
		// if we don't, we don't use the wrap box for perf reasons
		else
		{
			ContentBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MaxDesiredWidth(250.f)
				[
					ContentBox
				]
			]
			// Perf widget
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3, 0, 0, 0)
			[
				SNew(SNiagaraStackRowPerfWidget, Item, ENiagaraPerformanceStateScopeMode::Detailed)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				OptionsBox
			];
		}

		if (StackModuleItem)
		{
			SAssignNew(Content, SWidgetSwitcher)
		   .WidgetIndex_Lambda([StackModuleItem, this]() { return (StackViewModel->ShouldHideDisabledModules() && StackModuleItem->GetIsEnabled() == false) ? 1 : 0; })
		   + SWidgetSwitcher::Slot()
		   [
			   ContentBox
		   ]
		   + SWidgetSwitcher::Slot()
		   [
			   SNew(SBox)
			   .HeightOverride(4)
		   ];
		}
		else
		{
			Content = ContentBox;
		}
		RowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.SystemOverview.TableViewRow.Item");
	}
	else if (Item->IsA<UNiagaraStackItemGroup>())
	{
		FName IconName = FNiagaraStackEditorWidgetsUtilities::GetIconNameForExecutionSubcategory(Item->GetExecutionSubcategoryName(), true);
		const FSlateBrush* IconBrush = IconName != NAME_None ? FNiagaraEditorWidgetsStyle::Get().GetBrush(IconName) : FAppStyle::Get().GetDefaultBrush();
		
		UNiagaraStackItemGroup* StackItemGroup = CastChecked<UNiagaraStackItemGroup>(Item);
		TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

		FLinearColor IconColor = FNiagaraEditorWidgetsStyle::Get().GetColor(
			FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(StackItemGroup->GetExecutionCategoryName())).Desaturate(.25f);
		
		// Icon Brush
		if (StackItemGroup->GetSupportedIconMode() == UNiagaraStackEntry::EIconMode::Brush)
		{
			ContentBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 1, 1, 1)
				[
					SNew(SBox)
					.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
					.WidthOverride(IconSize.X)
					.HeightOverride(IconSize.Y)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image_UObject(StackItemGroup, &UNiagaraStackItemGroup::GetIconBrush)
						.DesiredSizeOverride(FVector2D(14.0f, 14.0f))
						.ColorAndOpacity(IconColor)
					]
				];
		}

		// Icon Text
		if (StackItemGroup->GetSupportedIconMode() == UNiagaraStackEntry::EIconMode::Text)
		{
			ContentBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 1, 1, 1)
				[
					SNew(SBox)
					.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
					.WidthOverride(IconSize.X)
					.HeightOverride(IconSize.Y)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(StackItemGroup->GetIconText())
						.ColorAndOpacity(IconColor)
					]
				];
		}

		// Name
		ContentBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1, 2, 0, 2)
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.GroupText")
				.Text_UObject(Item, &UNiagaraStackEntry::GetDisplayName)
				.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

		// Issue Icon
		ContentBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f, 0.0f)
			[
				SNew(SNiagaraStackIssueIcon, StackViewModel, StackItemGroup)
				.IconMode(SNiagaraStackIssueIcon::EIconMode::Compact)
				.Visibility_Static(&SNiagaraOverviewStack::GetIssueIconVisibility, Item)
			];

		// Secondary Icon
		if (StackItemGroup->SupportsSecondaryIcon())
		{
			ContentBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4, 1, 0, 1)
				[
					SNew(SBox)
					.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
					.WidthOverride(IconSize.X)
					.HeightOverride(IconSize.Y)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image_UObject(StackItemGroup, &UNiagaraStackItemGroup::GetSecondaryIconBrush)
					]
				];
		}

		// Spacer
		ContentBox->AddSlot()
			.Padding(0, 0, 2, 0)
			[
				SNew(SSpacer)
			];

		// Perf 
		ContentBox->AddSlot()
			.AutoWidth()
			.Padding(6, 0, 5, 0)
			[
				SNew(SNiagaraStackRowPerfWidget, Item, ENiagaraPerformanceStateScopeMode::Basic)
			];

		// Inheritance Icon
		if (StackItemGroup->SupportsInheritance())
		{
			ContentBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 0, 0, 0)
				[
					SNew(SNiagaraStackInheritanceIcon, Item)
				];
		}

		// Delete button
		if (StackItemGroup->SupportsDelete())
		{
			ContentBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 0, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.IsFocusable(false)
					.ToolTipText(this, &SNiagaraOverviewStack::GetItemGroupDeleteButtonToolTip, StackItemGroup)
					.OnClicked(this, &SNiagaraOverviewStack::OnItemGroupDeleteClicked, StackItemGroup)
					.Visibility(this, &SNiagaraOverviewStack::GetItemGroupDeleteButtonVisibility, StackItemGroup)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.DesiredSizeOverride(FVector2D(14.0f, 14.0f))
					]
				];
		}

		// Enabled checkbox
		if (StackItemGroup->SupportsChangeEnabled())
		{
			ContentBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(2, 1, 1, 1)
				[
					SNew(SCheckBox)
					.ForegroundColor(FSlateColor::UseSubduedForeground())
					.ToolTipText(LOCTEXT("StackItemGroupEnableDisableTooltip", "Enable or disable this item."))
					.IsChecked(this, &SNiagaraOverviewStack::ItemGroupCheckEnabledStatus, StackItemGroup)
					.OnCheckStateChanged(this, &SNiagaraOverviewStack::OnItemGroupEnabledStateChanged, StackItemGroup)
					.IsEnabled(this, &SNiagaraOverviewStack::GetItemGroupEnabledCheckboxEnabled, StackItemGroup)
				];
		}

		if (StackItemGroup->GetAddUtilities() != nullptr)
		{
			ContentBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0)
			[
				SNew(SNiagaraStackItemGroupAddButton, StackItemGroup, StackItemGroup->GetAddUtilities())
				.Width(22)
			];
		}

		Content = ContentBox;
		RowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.SystemOverview.TableViewRow.Group");
	}
	else
	{
		Content = SNullWidget::NullWidget;
	}

	return SNew(SNiagaraSystemOverviewEntryListRow, StackViewModel, Item, StackCommandContext.ToSharedRef(), OwnerTable)
		.Style(RowStyle)
		.OnDragDetected(this, &SNiagaraOverviewStack::OnRowDragDetected, TWeakObjectPtr<UNiagaraStackEntry>(Item))
		.OnDragLeave(this, &SNiagaraOverviewStack::OnRowDragLeave)
		.OnCanAcceptDrop(this, &SNiagaraOverviewStack::OnRowCanAcceptDrop)
		.OnAcceptDrop(this, &SNiagaraOverviewStack::OnRowAcceptDrop)
	[
		Content.ToSharedRef()
	];
}

EVisibility SNiagaraOverviewStack::GetEnabledCheckBoxVisibility(UNiagaraStackItem* Item) const
{
	return Item->SupportsChangeEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SNiagaraOverviewStack::GetShouldDebugDrawStatusVisibility(UNiagaraStackItem* Item) const
{
	return IsModuleDebugDrawEnabled(Item) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SNiagaraOverviewStack::IsModuleDebugDrawEnabled(UNiagaraStackItem* Item) const
{
	return Item->GetIsEnabled() && Item->IsDebugDrawEnabled();
}

const FSlateBrush* SNiagaraOverviewStack::GetDebugIconBrush(UNiagaraStackItem* Item) const
{
	return IsModuleDebugDrawEnabled(Item)? 
		FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.Overview.DebugActive")) :
		FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.Overview.DebugInactive"));
}

FReply SNiagaraOverviewStack::ToggleModuleDebugDraw(UNiagaraStackItem* Item)
{
	if (Item->SupportsDebugDraw())
	{
		Item->SetDebugDrawEnabled(!Item->IsDebugDrawEnabled());
	}

	return FReply::Handled();
}

void SNiagaraOverviewStack::OnSelectionChanged(UNiagaraStackEntry* InNewSelection, ESelectInfo::Type SelectInfo)
{
	if (bUpdatingStackSelectionFromOverviewSelection == false)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingOverviewSelectionFromStackSelection, true);
		TArray<UNiagaraStackEntry*> SelectedEntries;
		EntryListView->GetSelectedItems(SelectedEntries);

		TArray<UNiagaraStackEntry*> DeselectedEntries;

		for (TWeakObjectPtr<UNiagaraStackEntry> PreviousSelectedEntry : PreviousSelection)
		{
			if (PreviousSelectedEntry.IsValid() && SelectedEntries.Contains(PreviousSelectedEntry.Get()) == false)
			{
				DeselectedEntries.Add(PreviousSelectedEntry.Get());
			}
		}

		bool bClearCurrentSelection = FSlateApplication::Get().GetModifierKeys().IsControlDown() == false;

		// todo (ME) remove this special case when we remove the user parameters group from the stack in favor of the new user param tab
		if(InNewSelection && InNewSelection->IsA<UNiagaraStackSystemUserParametersGroup>())
		{
			InNewSelection->GetSystemViewModel()->FocusTab(TEXT("NiagaraSystemEditor_UserParameters"), true);
		}
		else
		{
			OverviewSelectionViewModel->UpdateSelectedEntries(SelectedEntries, DeselectedEntries, bClearCurrentSelection);
		}

		PreviousSelection.Empty();
		for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
		{
			PreviousSelection.Add(SelectedEntry);
		}

		UpdateCommandContextSelection();
	}
}

void SNiagaraOverviewStack::SystemSelectionChanged()
{
	if (bUpdatingOverviewSelectionFromStackSelection == false)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingStackSelectionFromOverviewSelection, true);

		TArray<UNiagaraStackEntry*> SelectedListViewStackEntries;
		EntryListView->GetSelectedItems(SelectedListViewStackEntries);
		TArray<UNiagaraStackEntry*> SelectedOverviewEntries;
		OverviewSelectionViewModel->GetSelectedEntries(SelectedOverviewEntries);

		TArray<UNiagaraStackEntry*> EntriesToDeselect;
		for (UNiagaraStackEntry* SelectedListViewStackEntry : SelectedListViewStackEntries)
		{
			if (SelectedOverviewEntries.Contains(SelectedListViewStackEntry) == false)
			{
				EntriesToDeselect.Add(SelectedListViewStackEntry);
			}
		}

		TArray<UNiagaraStackEntry*> EntriesToSelect;
		RefreshEntryList();
		for (UNiagaraStackEntry* SelectedOverviewEntry : SelectedOverviewEntries)
		{
			if (FlattenedEntryList.Contains(SelectedOverviewEntry))
			{
				EntriesToSelect.Add(SelectedOverviewEntry);
			}
		}

		for (UNiagaraStackEntry* EntryToDeselect : EntriesToDeselect)
		{
			EntryListView->SetItemSelection(EntryToDeselect, false);
		}

		for (UNiagaraStackEntry* EntryToSelect : EntriesToSelect)
		{
			EntryListView->SetItemSelection(EntryToSelect, true);
		}

		UpdateCommandContextSelection();
	}
}

void SNiagaraOverviewStack::UpdateCommandContextSelection()
{
	TArray<UNiagaraStackEntry*> SelectedListViewStackEntries;
	EntryListView->GetSelectedItems(SelectedListViewStackEntries);
	StackCommandContext->SetSelectedEntries(SelectedListViewStackEntries);
}

FReply SNiagaraOverviewStack::OnRowDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, TWeakObjectPtr<UNiagaraStackEntry> InStackEntryWeak)
{
	UNiagaraStackEntry* StackEntry = InStackEntryWeak.Get();
	if (StackEntry != nullptr && StackEntry->CanDrag())
	{
		TArray<UNiagaraStackEntry*> EntriesToDrag;
		StackEntry->GetSystemViewModel()->GetSelectionViewModel()->GetSelectedEntries(EntriesToDrag);
		EntriesToDrag.AddUnique(StackEntry);
		return FReply::Handled().BeginDragDrop(FNiagaraStackEditorWidgetsUtilities::ConstructDragDropOperationForStackEntries(EntriesToDrag));
	}
	return FReply::Unhandled();
}

void SNiagaraOverviewStack::OnRowDragLeave(const FDragDropEvent& InDragDropEvent)
{
	FNiagaraStackEditorWidgetsUtilities::HandleDragLeave(InDragDropEvent);
}

TOptional<EItemDropZone> SNiagaraOverviewStack::OnRowCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry)
{
	return FNiagaraStackEditorWidgetsUtilities::RequestDropForStackEntry(InDragDropEvent, InDropZone, InTargetEntry, UNiagaraStackEntry::EDropOptions::Overview);
}

FReply SNiagaraOverviewStack::OnRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry)
{
	bool bHandled = FNiagaraStackEditorWidgetsUtilities::HandleDropForStackEntry(InDragDropEvent, InDropZone, InTargetEntry, UNiagaraStackEntry::EDropOptions::Overview);
	return bHandled ? FReply::Handled() : FReply::Unhandled();
}

void SNiagaraOverviewStack::OnItemGroupEnabledStateChanged(ECheckBoxState InCheckState, UNiagaraStackItemGroup* Group)
{
	Group->SetIsEnabled(InCheckState == ECheckBoxState::Checked);
}

ECheckBoxState SNiagaraOverviewStack::ItemGroupCheckEnabledStatus(UNiagaraStackItemGroup* Group) const
{
	return Group->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool SNiagaraOverviewStack::GetItemGroupEnabledCheckboxEnabled(UNiagaraStackItemGroup* Group) const
{
	return Group->GetOwnerIsEnabled();
}

FText SNiagaraOverviewStack::GetItemGroupDeleteButtonToolTip(UNiagaraStackItemGroup* Group) const
{
	FText Message;
	Group->TestCanDeleteWithMessage(Message);
	return Message;
}

EVisibility SNiagaraOverviewStack::GetItemGroupDeleteButtonVisibility(UNiagaraStackItemGroup* Group) const
{
	FText UnusedMessage;
	return Group->SupportsDelete() && Group->TestCanDeleteWithMessage(UnusedMessage)
		? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SNiagaraOverviewStack::OnItemGroupDeleteClicked(UNiagaraStackItemGroup* Group)
{
	Group->Delete();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
