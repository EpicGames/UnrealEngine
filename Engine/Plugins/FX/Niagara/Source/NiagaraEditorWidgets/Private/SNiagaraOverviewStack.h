// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "UObject/ObjectKey.h"
#include "Framework/Commands/UICommandList.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"

class UNiagaraStackEntry;
class UNiagaraStackItem;
class UNiagaraStackViewModel;
class UNiagaraSystemSelectionViewModel;
class FNiagaraStackCommandContext;
class UNiagaraStackItemGroup;

class SNiagaraOverviewStack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraOverviewStack)
		: _AllowedClasses({UNiagaraStackItem::StaticClass(), UNiagaraStackItemGroup::StaticClass()})
	{}
		SLATE_ARGUMENT(TArray<UClass*>, AllowedClasses)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackViewModel& InStackViewModel, UNiagaraSystemSelectionViewModel& InOverviewSelectionViewModel);

	virtual	~SNiagaraOverviewStack() override;

	virtual bool SupportsKeyboardFocus() const override;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	static EVisibility GetIssueIconVisibility(UNiagaraStackEntry* StackEntry);
	static EVisibility GetUsageIconVisibility(UNiagaraStackEntry* StackEntry);
	static const FSlateBrush* GetUsageIcon(UNiagaraStackEntry* StackEntry);
	static int32 GetUsageIconWidth(UNiagaraStackEntry* StackEntry);
	static int32 GetUsageIconHeight(UNiagaraStackEntry* StackEntry);
	static FText GetUsageTooltip(UNiagaraStackEntry* StackEntry);

private:
	void AddEntriesRecursive(UNiagaraStackEntry& EntryToAdd, TArray<UNiagaraStackEntry*>& EntryList, const TArray<UClass*>& AcceptableClasses, TArray<UNiagaraStackEntry*> ParentChain);

	void RefreshEntryList();

	void EntryExpansionChanged();
	void EntryExpansionInOverviewChanged();
	void EntryStructureChanged(ENiagaraStructureChangedFlags Flags);

	TSharedRef<ITableRow> OnGenerateRowForEntry(UNiagaraStackEntry* Item, const TSharedRef<STableViewBase>& OwnerTable);

	EVisibility GetEnabledCheckBoxVisibility(UNiagaraStackItem* Item) const;

	EVisibility GetShouldDebugDrawStatusVisibility(UNiagaraStackItem* Item) const;

	bool IsModuleDebugDrawEnabled(UNiagaraStackItem* Item) const;

	const FSlateBrush* GetDebugIconBrush(UNiagaraStackItem* Item) const;

	FReply ToggleModuleDebugDraw(UNiagaraStackItem* Item);

	void OnSelectionChanged(UNiagaraStackEntry* InNewSelection, ESelectInfo::Type SelectInfo);

	void SystemSelectionChanged();

	void UpdateCommandContextSelection();

	FReply OnRowDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, TWeakObjectPtr<UNiagaraStackEntry> InStackEntryWeak);

	void OnRowDragLeave(const FDragDropEvent& InDragDropEvent);

	TOptional<EItemDropZone> OnRowCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry);

	FReply OnRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry);

	void OnItemGroupEnabledStateChanged(ECheckBoxState InCheckState, UNiagaraStackItemGroup* Group);
	ECheckBoxState ItemGroupCheckEnabledStatus(UNiagaraStackItemGroup* Group) const;
	bool GetItemGroupEnabledCheckboxEnabled(UNiagaraStackItemGroup* Group) const;

	FText GetItemGroupDeleteButtonToolTip(UNiagaraStackItemGroup* Group) const;
	bool GetItemGroupDeleteButtonIsEnabled(UNiagaraStackItemGroup* Group) const;
	EVisibility GetItemGroupDeleteButtonVisibility(UNiagaraStackItemGroup* Group) const;
	FReply OnItemGroupDeleteClicked(UNiagaraStackItemGroup* Group);

private:
	UNiagaraStackViewModel* StackViewModel;
	UNiagaraSystemSelectionViewModel* OverviewSelectionViewModel;

	TArray<UNiagaraStackEntry*> FlattenedEntryList;
	TMap<FObjectKey, TArray<UNiagaraStackEntry*>> EntryObjectKeyToParentChain;
	TSharedPtr<SListView<UNiagaraStackEntry*>> EntryListView;

	TArray<TWeakObjectPtr<UNiagaraStackEntry>> PreviousSelection;

	TSharedPtr<FNiagaraStackCommandContext> StackCommandContext;

	TArray<UClass*> AllowedClasses;

	bool bRefreshEntryListPending;
	bool bUpdatingOverviewSelectionFromStackSelection;
	bool bUpdatingStackSelectionFromOverviewSelection;
};