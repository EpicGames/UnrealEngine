// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraOverviewStackNode.h"

#include "EditorFontGlyphs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraSettings.h"
#include "SGraphPanel.h"
#include "SLevelOfDetailBranchNode.h"
#include "SNiagaraOverviewStack.h"
#include "Stack/SNiagaraDeterminismToggle.h"
#include "Stack/SNiagaraLocalSpaceToggle.h"
#include "Stack/SNiagaraSummaryViewToggle.h"
#include "Stack/SNiagaraSimTargetToggle.h"
#include "Stack/SNiagaraStackIssueIcon.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "AssetThumbnail.h"
#include "SNiagaraEmitterRendererThumbnails.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"

#define LOCTEXT_NAMESPACE "NiagaraOverviewStackNode"

constexpr float EmitterThumbnailSize = 200.f;
constexpr float NodeTitleMaxSize = 125.f;

void SNiagaraOverviewStackNode::Construct(const FArguments& InArgs, UNiagaraOverviewNode* InNode)
{
	GraphNode = InNode;
	OverviewStackNode = InNode;
	StackViewModel = nullptr;
	OverviewSelectionViewModel = nullptr;
	CurrentIssueIndex = -1;
	// we are reducing the left margin to better place our widgets
	TitleBorderMargin = FMargin(2.f, 5.f, 30.f, 3.f);

	EmitterHandleViewModelWeak.Reset();
	
	if (OverviewStackNode->GetOwningSystem() != nullptr)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FNiagaraEditorModule::Get();
		TSharedPtr<FNiagaraSystemViewModel> OwningSystemViewModel = NiagaraEditorModule.GetExistingViewModelForSystem(OverviewStackNode->GetOwningSystem());
		if (OwningSystemViewModel.IsValid())
		{
			// if the emitter handle view models have updates, make sure we rebind the editor delegates. The previous bindings might have become invalid due to a merge
			OwningSystemViewModel->OnEmitterHandleViewModelsChanged().AddSP(this, &SNiagaraOverviewStackNode::BindEditorDataDelegates);
			if (OverviewStackNode->GetEmitterHandleGuid().IsValid() == false)
			{
				StackViewModel = OwningSystemViewModel->GetSystemStackViewModel();
			}
			else
			{
				EmitterHandleViewModelWeak = OwningSystemViewModel->GetEmitterHandleViewModelById(OverviewStackNode->GetEmitterHandleGuid());
				if (EmitterHandleViewModelWeak.IsValid())
				{
					StackViewModel = EmitterHandleViewModelWeak.Pin()->GetEmitterStackViewModel();
				}
			}
			
			OverviewSelectionViewModel = OwningSystemViewModel->GetSelectionViewModel();
			ScalabilityViewModel = OwningSystemViewModel->GetScalabilityViewModel();

			if(ScalabilityViewModel.IsValid())
			{
				bScalabilityModeActive = ScalabilityViewModel->IsActive();
				ScalabilityViewModel->OnScalabilityModeChanged().AddSP(this, &SNiagaraOverviewStackNode::OnScalabilityModeChanged);
			}
		}
	}

	BindEditorDataDelegates();

	UpdateGraphNode();
}

SNiagaraOverviewStackNode::~SNiagaraOverviewStackNode()
{
	if(ScalabilityViewModel.IsValid())
	{
		ScalabilityViewModel->OnScalabilityModeChanged().RemoveAll(this);
	}

	UnbindEditorDataDelegates();
}

void SNiagaraOverviewStackNode::BindEditorDataDelegates()
{
	UnbindEditorDataDelegates();

	if(EmitterHandleViewModelWeak.IsValid())
	{
		EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().OnPersistentDataChanged().AddSP(this, &SNiagaraOverviewStackNode::RefreshEmitterThumbnailPreview);
		EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().OnSummaryViewStateChanged().AddSP(this, &SNiagaraOverviewStackNode::UpdateGraphNode);
	}
}

void SNiagaraOverviewStackNode::UnbindEditorDataDelegates() const
{
	if(EmitterHandleViewModelWeak.IsValid() && EmitterHandleViewModelWeak.Pin()->GetEmitterHandle() != nullptr)
	{
		EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().OnPersistentDataChanged().RemoveAll(this);
		EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().OnSummaryViewStateChanged().RemoveAll(this);
	}
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
{
	switch(DisplayMode)
	{
	case EDisplayMode::Default:
		return CreateTitleWidget_Default(NodeTitle);
	case EDisplayMode::Summary:
		return CreateTitleWidget_Summary(NodeTitle);
	default:
		return CreateTitleWidget_Default(NodeTitle);
	}
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTitleRightWidget()
{
	switch(DisplayMode)
	{
	case EDisplayMode::Default:
		return CreateTitleRightWidget_Default();
	case EDisplayMode::Summary:
		return CreateTitleRightWidget_Summary();
	default:
		return CreateTitleRightWidget_Default();
	}
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateNodeContentArea()
{
	switch(DisplayMode)
	{
	case EDisplayMode::Default:
		return CreateNodeContentArea_Default();
	case EDisplayMode::Summary:
		return CreateNodeContentArea_Summary();
	default:
		return CreateNodeContentArea_Default();
	}
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTopContentBar()
{
	switch(DisplayMode)
	{
	case EDisplayMode::Default:
		return CreateTopContentBar_Default();
	case EDisplayMode::Summary:
		return CreateTopContentBar_Summary();
	default:
		return CreateTopContentBar_Default();
	}
}

FText SNiagaraOverviewStackNode::GetSpawnCountScaleText() const
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		if(FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModelWeak.Pin()->GetEmitterHandle()->GetInstance().GetEmitterData())
		{
			if(EmitterData->GetScalabilitySettings().bScaleSpawnCount)
			{
				return FText::FromString(FString::SanitizeFloat(EmitterData->GetScalabilitySettings().SpawnCountScale));
			}
		}
	}

	return FText::GetEmpty();
}

FText SNiagaraOverviewStackNode::GetSpawnCountScaleTooltip() const
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		if(FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModelWeak.Pin()->GetEmitterHandle()->GetInstance().GetEmitterData())
		{
			if(EmitterData->GetScalabilitySettings().bScaleSpawnCount)
			{
				return FText::FormatOrdered(LOCTEXT("EmitterSpawnCountScaleInfoTooltip", "This emitter currently uses a Spawn Count Scale of {0}.\nThis affects the number of spawned particles. Enter Scalability Mode to view & edit."),
					FText::FromString(FString::SanitizeFloat(EmitterData->GetScalabilitySettings().SpawnCountScale)));
			}
		}
	}

	return FText::GetEmpty();
}

EVisibility SNiagaraOverviewStackNode::GetSpawnCountScaleTextVisibility() const
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		if(FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModelWeak.Pin()->GetEmitterHandle()->GetInstance().GetEmitterData())
		{
			if(EmitterData->GetScalabilitySettings().bScaleSpawnCount && EmitterData->GetScalabilitySettings().SpawnCountScale != 1.f)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

ECheckBoxState SNiagaraOverviewStackNode::IsScalabilityModeActive() const
{
	TSharedPtr<FNiagaraSystemViewModel> NiagaraSystemViewModel;
	
	if(ScalabilityViewModel.IsValid())
	{
		NiagaraSystemViewModel = ScalabilityViewModel->GetSystemViewModel().Pin();
	}
	
	if(NiagaraSystemViewModel.IsValid())
	{
		return NiagaraSystemViewModel->GetWorkflowMode().IsEqual(FNiagaraSystemToolkitModeBase::ScalabilityModeName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}

void SNiagaraOverviewStackNode::OnScalabilityModeStateChanged(ECheckBoxState CheckBoxState)
{
	TSharedPtr<FNiagaraSystemViewModel> NiagaraSystemViewModel;
	
	if(ScalabilityViewModel.IsValid())
	{
		NiagaraSystemViewModel = ScalabilityViewModel->GetSystemViewModel().Pin();
	}

	if(NiagaraSystemViewModel.IsValid())
	{
		if(NiagaraSystemViewModel->GetWorkflowMode().IsEqual(FNiagaraSystemToolkitModeBase::ScalabilityModeName))
		{
			NiagaraSystemViewModel->SetWorkflowMode(FNiagaraSystemToolkitModeBase::DefaultModeName);
		}
		else
		{
			NiagaraSystemViewModel->SetWorkflowMode(FNiagaraSystemToolkitModeBase::ScalabilityModeName);
		}
	}
}

FReply SNiagaraOverviewStackNode::OnCycleThroughIssues()
{
	const TArray<UNiagaraStackEntry*>& ChildrenWithIssues = StackViewModel->GetRootEntry()->GetAllChildrenWithIssues();
	if (ChildrenWithIssues.Num() > 0)
	{
		++CurrentIssueIndex;

		if (CurrentIssueIndex >= ChildrenWithIssues.Num())
		{
			CurrentIssueIndex = 0;
		}

		if (ChildrenWithIssues.IsValidIndex(CurrentIssueIndex))
		{
			UNiagaraStackEntry* ChildIssue = ChildrenWithIssues[CurrentIssueIndex];
			UNiagaraStackEntry* ChildToSelect = Cast<UNiagaraStackModuleItem>(ChildIssue);
			if (ChildToSelect == nullptr)
			{
				ChildToSelect = ChildIssue->GetTypedOuter<UNiagaraStackModuleItem>();
			}

			if (ChildToSelect == nullptr)
			{
				ChildToSelect = Cast<UNiagaraStackItemGroup>(ChildIssue);
			}
			
			if (ChildToSelect != nullptr)
			{
				OverviewSelectionViewModel->UpdateSelectedEntries(TArray<UNiagaraStackEntry*> { ChildToSelect }, TArray<UNiagaraStackEntry*>(), true /* bClearCurrentSelection */ );
			}
		}
	}

	return FReply::Handled();
}

FReply SNiagaraOverviewStackNode::OnCaptureThumbnailButtonClicked()
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		EmitterHandleViewModelWeak.Pin()->RequestCaptureThumbnail();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SNiagaraOverviewStackNode::OnRendererThumbnailClicked(UNiagaraStackRendererItem* StackRendererItem)
{
	TArray<UNiagaraStackEntry*> SelectedEntries;
	SelectedEntries.Add(StackRendererItem);
	TArray<UNiagaraStackEntry*> DeselectedEntries;
	OverviewSelectionViewModel->UpdateSelectedEntries(SelectedEntries, DeselectedEntries, true);

	CurrentIssueIndex = -1;
	return FReply::Handled();
}

FReply SNiagaraOverviewStackNode::OnPropertiesButtonClicked() const
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		TArray<UNiagaraStackEmitterPropertiesGroup*> EmitterProperties;
		EmitterHandleViewModelWeak.Pin()->GetEmitterStackViewModel()->GetRootEntry()->GetUnfilteredChildrenOfType(EmitterProperties);

		ensure(EmitterProperties.Num() == 1);
		EmitterHandleViewModelWeak.Pin()->GetOwningSystemViewModel()->GetSelectionViewModel()->UpdateSelectedEntries({EmitterProperties[0]}, {}, true);
	}

	return FReply::Handled();
}

void SNiagaraOverviewStackNode::RefreshEmitterThumbnailPreview()
{
	if(EmitterHandleViewModelWeak.IsValid() && ThumbnailContainer.IsValid())
	{
		if(EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().GetThumbnail() != nullptr)
		{
			PreviewThumbnail = MakeShared<FAssetThumbnail>(EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().GetThumbnail(), 200.f, 200.f, UThumbnailManager::Get().GetSharedThumbnailPool());
			ThumbnailContainer->SetContent(PreviewThumbnail->MakeThumbnailWidget());
		}
		else
		{
			PreviewThumbnail = MakeShared<FAssetThumbnail>(EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEmitter().Emitter, 200.f, 200.f, UThumbnailManager::Get().GetSharedThumbnailPool());
			ThumbnailContainer->SetContent(PreviewThumbnail->MakeThumbnailWidget());
		}
	}
}

void SNiagaraOverviewStackNode::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (OverviewStackNode != nullptr)
	{
		// To support drag & drop indicators, we mark the prepass as dirty. This will cause the invalidation widget to render using the slow path, updating the drag & drop indicators properly
		if(FSlateApplication::Get().IsDragDropping())
		{
			if (ContentAreaWidget)
			{
				ContentAreaWidget->MarkPrepassAsDirty();
			}
		}
		
		if (OverviewStackNode->IsRenamePending() && !SGraphNode::IsRenamePending())
		{
			SGraphNode::RequestRename();
			OverviewStackNode->RenameStarted();
		}

		if (ScalabilityWrapper != nullptr && !UseLowDetailNodeContent())
		{
			LastHighDetailSize = ScalabilityWrapper->GetTickSpaceGeometry().Size;
			if (GeometryTickForSize > 0)
			{
				// the stack needs a few tick to fully initialize, so we wait to bit to grab the right low detail geomtry size
				GeometryTickForSize--;
			}
		}
	}
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTitleWidget_Default(TSharedPtr<SNodeTitle> NodeTitle)
{
	TSharedRef<SWidget> DefaultTitle = SGraphNode::CreateTitleWidget(NodeTitle);
	DefaultTitle->SetToolTipText(TAttribute<FText>::CreateSP(NodeTitle.Get(), &SNodeTitle::GetHeadTitle));

	if (StackViewModel == nullptr)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 0, 5, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InvalidNode", "INVALID"))
			]
			+ SHorizontalBox::Slot()
			[
				DefaultTitle
			];
	}

	TSharedPtr<SWidget> TitleWidget = SNew(SHorizontalBox)
	// Summary View Controls
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(2.f)
	[
		CreateSummaryViewToggle()
	]
	// Enabled checkbox
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(5.f, 2.f, 2.f, 2.f)
	[
		CreateEnabledCheckbox()
	]
	// Name
	+ SHorizontalBox::Slot()
	.Padding(3, 0, 0, 0)
	.AutoWidth()
	.MaxWidth(NodeTitleMaxSize)
	[
		DefaultTitle
	];

	return TitleWidget.ToSharedRef();
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTitleRightWidget_Default()
{
	if (StackViewModel == nullptr)
	{
		return SNullWidget::NullWidget;
	}
	return SNew(SHorizontalBox)
	// open parent button
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(0, 0, 1, 0)
	[
		CreateOpenParentButton()
	]

	// version selector
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Right)
	.AutoWidth()
	.Padding(1, 0, 2, 0)
	[
		CreateVersionSelectorButton()	
	]

	// scalability controls
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Right)
	.AutoWidth()
	[
		CreateScalabilityControls()
	]

	// issue/error icon
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(0, 0, 1, 0)
	[
		SNew(SNiagaraStackIssueIcon, StackViewModel, StackViewModel->GetRootEntry())
		.Visibility(this, &SNiagaraOverviewStackNode::GetIssueIconVisibility)
		.OnClicked(this, &SNiagaraOverviewStackNode::OnCycleThroughIssues)
	];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateNodeContentArea_Default()
{
	TSharedPtr<SWidget> ContentWidget;
	if (StackViewModel != nullptr && OverviewSelectionViewModel != nullptr)
	{
		ContentWidget = SNew(SNiagaraOverviewStack, *StackViewModel, *OverviewSelectionViewModel)
		.AllowedClasses({UNiagaraStackItemGroup::StaticClass(), UNiagaraStackItem::StaticClass()});
	}
	else
	{
		ContentWidget = SNullWidget::NullWidget;
	}
	
	TSharedPtr<SVerticalBox> NodeBox;
	
	// NODE CONTENT AREA
	TSharedRef<SWidget> NodeWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(2, 2, 2, 4))
		[
			SAssignNew(NodeBox, SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				CreateTopContentBar()
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(0.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					// LEFT
					SAssignNew(LeftNodeBox, SVerticalBox)
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				[
					SNew(SBorder)
					.BorderImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.SystemOverview.NodeBackgroundBorder"))
					.BorderBackgroundColor(FStyleColors::Panel)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(0, 0, 0, 4))
					[
						ContentWidget.ToSharedRef()
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		];

	TSharedRef<SWidget> DetailedContent = SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &SNiagaraOverviewStackNode::UseLowDetailNodeContent)
		.LowDetail()
		[
			SNew(SBox)
			.WidthOverride(this, &SNiagaraOverviewStackNode::GetLowDetailDesiredWidth)
			.HeightOverride(this, &SNiagaraOverviewStackNode::GetLowDetailDesiredHeight)
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraOverviewStackNode::GetLowDetailNodeTitle)
				.TextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.SystemOverview.ZoomedOutNodeFont"))
				.Visibility(EVisibility::HitTestInvisible)
				.Clipping(EWidgetClipping::Inherit)
			]
		]
		.HighDetail()
		[
			NodeWidget
		];

	ScalabilityWrapper = SNew(SOverlay)
	+ SOverlay::Slot()
	[
		DetailedContent
	]
	+ SOverlay::Slot()
	.Padding(0, 1)
	[
		SNew(SBorder)
		.Visibility(this, &SNiagaraOverviewStackNode::ShowExcludedOverlay)
		.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.SystemOverview.ExcludedFromScalability.NodeBody"))
		.BorderBackgroundColor(TAttribute<FSlateColor>(this, &SNiagaraOverviewStackNode::GetScalabilityTintAlpha))
	];

	ContentAreaWidget = SNew(SInvalidationPanel)
	[
		ScalabilityWrapper.ToSharedRef()
	];

	return ContentAreaWidget.ToSharedRef();
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTopContentBar_Default()
{
	TSharedRef<SHorizontalBox> TopContentBar = SNew(SHorizontalBox);

	if (EmitterHandleViewModelWeak.IsValid())
	{		
		// Isolate toggle button
		TopContentBar->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2, 0, 0, 0)
		[
			CreateIsolateButton()
		];

		// Bounds toggle button
		TopContentBar->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2, 0, 0, 0)
		[
			CreateBoundsButton()
		];

		// Renderer thumbnails
		TopContentBar->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.MaxWidth(300.f)
		[
			CreateEmitterRendererThumbnails()
		];
	}

	return TopContentBar;
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTitleWidget_Summary(TSharedPtr<SNodeTitle> NodeTitle)
{
	// We don't need to differ from the default currently
	return CreateTitleWidget_Default(NodeTitle);
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTitleRightWidget_Summary()
{
	if (StackViewModel == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
	// issue/error icon
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(0, 0, 1, 0)
	[
		SNew(SNiagaraStackIssueIcon, StackViewModel, StackViewModel->GetRootEntry())
		.Visibility(this, &SNiagaraOverviewStackNode::GetIssueIconVisibility)
		.OnClicked(this, &SNiagaraOverviewStackNode::OnCycleThroughIssues)
	];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateNodeContentArea_Summary()
{
	return SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		CreateTopContentBar()
	]
	+ SVerticalBox::Slot()
	.Padding(3.f)
	[
		CreateEmitterThumbnail()
	];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateTopContentBar_Summary()
{
	TSharedRef<SHorizontalBox> TopContentBar = SNew(SHorizontalBox);
	
	TopContentBar->AddSlot()
	.AutoWidth()
	.Padding(4.f, 5.f)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		CreateIsolateButton()
	];

	TopContentBar->AddSlot()
	[
		SNew(SSpacer)
	];

	TopContentBar->AddSlot()
	.AutoWidth()
	.Padding(2.f, 5.f)
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	[
		CreateLocalSpaceToggle()
	];
	
	TopContentBar->AddSlot()
	.AutoWidth()
	.Padding(2.f, 5.f)
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	[
		CreateDeterminismToggle()
	];
	
	TopContentBar->AddSlot()
	.AutoWidth()
	.Padding(2.f, 5.f, 4.f, 5.f)
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	[
		CreateSimTargetToggle()
	];

	// TopContentBar->AddSlot()
	// .AutoWidth()
	// .Padding(10.f, 5.f)
	// .HAlign(HAlign_Right)
	// .VAlign(VAlign_Center)
	// [
	// 	CreatePropertiesButton()
	// ];

	return TopContentBar;
}

void SNiagaraOverviewStackNode::UpdateGraphNode()
{
	DisplayMode = EDisplayMode::Default;
	if(EmitterHandleViewModelWeak.IsValid())
	{
		DisplayMode = EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().ShouldShowSummaryView() ? EDisplayMode::Summary : DisplayMode;
	}
	
	SGraphNode::UpdateGraphNode();
}

TOptional<ETextOverflowPolicy> SNiagaraOverviewStackNode::GetNameOverflowPolicy() const
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		if(DisplayMode == EDisplayMode::Summary)
		{
			return ETextOverflowPolicy::Ellipsis;
		}
	}

	return {};
}

EVisibility SNiagaraOverviewStackNode::GetScalabilityIndicatorVisibility() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();

	if(EmitterHandleViewModel.IsValid())
	{
		FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		const FNiagaraPlatformSet* EmitterPlatformSet = nullptr;
		const FNiagaraEmitterScalabilityOverrides* EmitterScalabilityOverrides = nullptr;

		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Standard)
		{
			if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle->GetInstance().GetEmitterData())
			{
				EmitterPlatformSet = &EmitterData->Platforms;
				EmitterScalabilityOverrides = &EmitterData->ScalabilityOverrides;
			}
		}
		else
		{
			if (UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle->GetStatelessEmitter())
			{
				EmitterPlatformSet = &StatelessEmitter->GetPlatformSet();
				EmitterScalabilityOverrides = &StatelessEmitter->GetScalabilityOverrides();
			}
		}

		const bool bIsQualityLevelMaskSetup = EmitterPlatformSet && EmitterPlatformSet->QualityLevelMask != INDEX_NONE && EmitterPlatformSet->QualityLevelMask != FNiagaraPlatformSet::GetFullQualityLevelMask(GetDefault<UNiagaraSettings>()->QualityLevels.Num());
		const bool bIsScalabilitySetup = EmitterScalabilityOverrides && EmitterScalabilityOverrides->Overrides.Num() != 0;
		return (bIsQualityLevelMaskSetup || bIsScalabilitySetup) ? EVisibility::Visible : DisplayMode == EDisplayMode::Summary ? EVisibility::Hidden : EVisibility::Collapsed;
	}

	if(UNiagaraSystem* System = OverviewStackNode->GetOwningSystem())
	{
		bool bIsQualityLevelMaskSetup = System->GetScalabilityPlatformSet().QualityLevelMask != INDEX_NONE;
		bool bIsScalabilityOverridden = System->GetOverrideScalabilitySettings();
		bool bIsScalabilitySetup = bIsScalabilityOverridden || (System->GetScalabilityOverrides().Overrides.Num() != 0 || (bIsQualityLevelMaskSetup && System->GetScalabilityPlatformSet().QualityLevelMask != FNiagaraPlatformSet::GetFullQualityLevelMask(GetDefault<UNiagaraSettings>()->QualityLevels.Num())));
		return bIsScalabilitySetup ? EVisibility::Visible : EVisibility::Collapsed; 
	}
	
	return EVisibility::Collapsed;
}

bool SNiagaraOverviewStackNode::UseLowDetailNodeContent() const
{
	const UNiagaraEditorSettings* NiagaraSettings = GetDefault<UNiagaraEditorSettings>();
	if (LastHighDetailSize.IsNearlyZero() || GeometryTickForSize > 0 || !NiagaraSettings->bSimplifyStackNodesAtLowResolution)
	{
		return false;
	}
	
	if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
	{
		return (MyOwnerPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowDetail);
	}
	return false;
}

FVector2D SNiagaraOverviewStackNode::GetLowDetailDesiredSize() const
{
	return LastHighDetailSize;
}

FOptionalSize SNiagaraOverviewStackNode::GetLowDetailDesiredWidth() const
{
	return LastHighDetailSize.X;
}

FOptionalSize SNiagaraOverviewStackNode::GetLowDetailDesiredHeight() const
{
	return LastHighDetailSize.Y;
}

namespace NiagaraOverviewStackNode
{
	struct StringRange
	{
		int32 Start = 0;
		int32 End = 0;
	};
	
	bool IsUpperAlphaNumeric(const TCHAR& c)
	{
		return (c >= 'A' && c <= 'Z') || (c >= '0' && c <= '9');
	}

	// this method splits long emitter names for the low details view by inserting new lines
	// and abbreviating large pieces when necessary
	FString PrettySplitString(const FString& Input)
	{
		TArray<FStringView, TInlineAllocator<16>> Parts;

		// split at convenient places
		int32 Start = 0;
		for (int i = 0; i < Input.Len() - 1; i++)
		{
			TCHAR currentChar = Input[i];
			TCHAR nextChar = Input[i + 1];

			bool bIsLowerAlpha = currentChar >= 'a' && currentChar <= 'z';

			// skip '_' at the beginning or in sequence
			if (currentChar == '_')
			{
				Start = i + 1;
			}
			// replace '_' in the middle of the name with a newline
			else if (nextChar == '_')
			{
				Parts.Add(FStringView(&Input[Start], i - Start + 1));
				i++;
				Start = i + 1;
			}
			// split when changing from lowercase to uppercase or number
			else if (bIsLowerAlpha && IsUpperAlphaNumeric(nextChar))
			{
				Parts.Add(FStringView(&Input[Start], i - Start + 1));
				Start = i + 1;
			}
		}
		// add the end piece
		if (Start < Input.Len())
		{
			Parts.Add(FStringView(&Input[Start], Input.Len() - Start));
		}

		// assemble the pieces
		const UNiagaraEditorSettings* NiagaraSettings = GetDefault<UNiagaraEditorSettings>();
		int32 MaxLength = FMath::Max(3, NiagaraSettings->LowResolutionNodeMaxNameChars);
		TStringBuilder<128> SplitNameBuilder;
		int32 RunningLength = 0;
		for (const FStringView& Split : Parts)
		{
			// merge small parts together, only insert a newline between big parts
			if (Split.Len() + RunningLength > MaxLength && SplitNameBuilder.Len() > 0)
			{
				SplitNameBuilder.AppendChar('\n');
				RunningLength = 0;
			}
			
			// if the name is too long, abbreviate with ...
			if (Split.Len() > MaxLength)
			{
				SplitNameBuilder.Append(Split.Left(MaxLength - 2));
				SplitNameBuilder.Append(TEXT("…"));
			}
			else
			{
				SplitNameBuilder.Append(Split);
			}
			RunningLength += Split.Len();
		}

		return SplitNameBuilder.ToString();
	}
}

FText SNiagaraOverviewStackNode::GetLowDetailNodeTitle() const
{
	if (FString Title = GetEditableNodeTitle(); LowDetailTitleCache.Key != Title)
	{
		LowDetailTitleCache.Key = Title;
		LowDetailTitleCache.Value = FText::FromString(NiagaraOverviewStackNode::PrettySplitString(Title));
	}
	return LowDetailTitleCache.Value;
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateSummaryViewToggle()
{
	return SNew(SNiagaraSummaryViewToggle, EmitterHandleViewModelWeak.Pin() ? EmitterHandleViewModelWeak.Pin() : nullptr);
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateLocalSpaceToggle()
{
	return SNew(SNiagaraLocalSpaceToggle, EmitterHandleViewModelWeak.Pin() ? EmitterHandleViewModelWeak.Pin() : nullptr);
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateEmitterThumbnail()
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();

		TSharedRef<SOverlay> PreviewOverlay = SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[		
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFill)
			[
				SAssignNew(ThumbnailContainer, SBox)
				.WidthOverride(EmitterThumbnailSize)
				.HeightOverride(EmitterThumbnailSize)
			]
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Bottom)
		.Padding(3.f)
		[
			CreateCaptureThumbnailButton()
		];

		RefreshEmitterThumbnailPreview();

		return PreviewOverlay;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateEmitterRendererThumbnails()
{
	if(ensureMsgf(EmitterHandleViewModelWeak.IsValid(), TEXT("This should only ever be called on emitter nodes")))
	{
		return SNew(SNiagaraEmitterRendererThumbnails, EmitterHandleViewModelWeak.Pin().ToSharedRef())
			.OnRendererThumbnailClicked(this, &SNiagaraOverviewStackNode::OnRendererThumbnailClicked);
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateEnabledCheckbox()
{
	return SNew(SCheckBox)
	.Visibility(this, &SNiagaraOverviewStackNode::GetEnabledCheckBoxVisibility)
	.IsChecked(this, &SNiagaraOverviewStackNode::GetEnabledCheckState)
	.OnCheckStateChanged(this, &SNiagaraOverviewStackNode::OnEnabledCheckStateChanged);
}

TSharedRef<SButton> SNiagaraOverviewStackNode::CreateIsolateButton()
{
	return SNew(SButton)
	.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	.HAlign(HAlign_Center)
	.ContentPadding(1)
	.ToolTipText(this, &SNiagaraOverviewStackNode::GetToggleIsolateToolTip)
	.OnClicked(this, &SNiagaraOverviewStackNode::OnToggleIsolateButtonClicked)
	.Visibility(this, &SNiagaraOverviewStackNode::GetToggleIsolateVisibility)
	.IsFocusable(false)
	.Content()
	[
		SNew(SImage)
		.Image(this, &SNiagaraOverviewStackNode::GetToggleIsolateImage)
		.ColorAndOpacity(this, &SNiagaraOverviewStackNode::GetToggleIsolateImageColor)
	];
}

TSharedRef<SButton> SNiagaraOverviewStackNode::CreateBoundsButton()
{
	return SNew(SButton)
	.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	.HAlign(HAlign_Center)
	.ContentPadding(1)
	.ToolTipText(this, &SNiagaraOverviewStackNode::GetToggleBoundsToolTip)
	.OnClicked(this, &SNiagaraOverviewStackNode::OnToggleBoundsButtonClicked)
	.Visibility(this, &SNiagaraOverviewStackNode::GetToggleBoundsVisibility)
	.IsFocusable(false)
	.Content()
	[
		SNew(SImage)
		.Image(this, &SNiagaraOverviewStackNode::GetToggleBoundsImage)
		.ColorAndOpacity(this, &SNiagaraOverviewStackNode::GetToggleBoundsImageColor)
	];
}

TSharedRef<SButton> SNiagaraOverviewStackNode::CreateCaptureThumbnailButton()
{
	return SNew(SButton)
	.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	.HAlign(HAlign_Center)
	.ContentPadding(1)
	.OnClicked(this, &SNiagaraOverviewStackNode::OnCaptureThumbnailButtonClicked)
	.IsFocusable(false)
	.ToolTipText(LOCTEXT("CaptureNewEmitterThumbnailButtonTooltip", "Capture a new thumbnail for this emitter based on the Niagara viewport"))
	.Content()
	[
		SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("AssetEditor.SaveThumbnail"))
	];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateOpenParentButton()
{
	return SNew(SButton)
	.IsFocusable(false)
	.ToolTipText(this, &SNiagaraOverviewStackNode::OpenParentEmitterTooltip)
	.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	.ContentPadding(2)
	.OnClicked(this, &SNiagaraOverviewStackNode::OpenParentEmitter)
	.Visibility(this, &SNiagaraOverviewStackNode::GetOpenParentEmitterVisibility)
	.DesiredSizeScale(FVector2D(14.0f / 30.0f, 14.0f / 30.0f)) // GoToSourceIcon is 30x30, scale down
	.Content()
	[
		SNew(SImage)
		.Image(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.GoToSourceIcon"))
		.ColorAndOpacity(FSlateColor::UseForeground())
	];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateVersionSelectorButton()
{
	return SNew(SComboButton)
	.HasDownArrow(false)
	.ToolTipText(LOCTEXT("ChangeEmitterVersionToolTip", "Change the parent emitter version"))
	.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	.ForegroundColor(FSlateColor::UseForeground())
	.OnGetMenuContent(this, &SNiagaraOverviewStackNode::GetVersionSelectorDropdownMenu)
	.ContentPadding(FMargin(2))
	.Visibility(this, &SNiagaraOverviewStackNode::GetVersionSelectorVisibility)
	.ButtonContent()
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
		.ColorAndOpacity(this, &SNiagaraOverviewStackNode::GetVersionSelectorColor)
		.Text(FEditorFontGlyphs::Random)
	];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateScalabilityControls()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(2.f, 4.f)
		[
			// Toggle Button to enter & exit scalability mode
			SNew(SCheckBox)
			.Style(&FAppStyle::GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.IsChecked(this, &SNiagaraOverviewStackNode::IsScalabilityModeActive)
			.Padding(2.f)
			.OnCheckStateChanged(this, &SNiagaraOverviewStackNode::OnScalabilityModeStateChanged)
			[
				SNew(SBox)
				.WidthOverride(16.f)
				.HeightOverride(16.f)
				[
					SNew(SImage)
					.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Scalability"))
					.Visibility(this, &SNiagaraOverviewStackNode::GetScalabilityIndicatorVisibility)
					.ToolTipText(FText::FormatOrdered(LOCTEXT("ScalabilityIndicatorToolTip",
						"This {0} has scalability set up. Inspecting and editing scalability is accessible by entering Scalability Mode by clicking this or the button in the toolbar.."), EmitterHandleViewModelWeak.IsValid() ? FText::FromString("emitter") : FText::FromString("system")))
				]
			]
		]
		// Spawn Count Scale Info
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(this, &SNiagaraOverviewStackNode::GetSpawnCountScaleText)
			.ToolTipText(this, &SNiagaraOverviewStackNode::GetSpawnCountScaleTooltip)
			.Visibility(this, &SNiagaraOverviewStackNode::GetSpawnCountScaleTextVisibility)
		];
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateSimTargetToggle()
{
	return SNew(SNiagaraSimTargetToggle, EmitterHandleViewModelWeak.IsValid() ? EmitterHandleViewModelWeak.Pin() : nullptr);
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreateDeterminismToggle()
{
	return SNew(SNiagaraDeterminismToggle, EmitterHandleViewModelWeak.IsValid() ? EmitterHandleViewModelWeak.Pin() : nullptr);
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::CreatePropertiesButton()
{
	return SNew(SButton)
	.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	.HAlign(HAlign_Center)
	.ContentPadding(1)
	.OnClicked(this, &SNiagaraOverviewStackNode::OnPropertiesButtonClicked)
	.IsFocusable(false)
	.ToolTipText(LOCTEXT("SelectEmitterPropertiesButtonTooltip", "Select this emitter's properties"))
	.Content()
	[
		SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("Icons.Details"))
	];
}

EVisibility SNiagaraOverviewStackNode::GetIssueIconVisibility() const
{
	return StackViewModel->HasIssues() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SNiagaraOverviewStackNode::GetEnabledCheckBoxVisibility() const
{
	return EmitterHandleViewModelWeak.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SNiagaraOverviewStackNode::GetEnabledCheckState() const
{
	return EmitterHandleViewModelWeak.IsValid() && EmitterHandleViewModelWeak.Pin()->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

const FSlateBrush* SNiagaraOverviewStackNode::GetEnabledImage() const
{
	return EmitterHandleViewModelWeak.IsValid() && EmitterHandleViewModelWeak.Pin()->GetIsEnabled()
		? FAppStyle::GetBrush("Icons.Success")
		: FAppStyle::GetBrush("Icons.MinusCircle");
}

void SNiagaraOverviewStackNode::OnEnabledCheckStateChanged(ECheckBoxState InCheckState)
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	if (EmitterHandleViewModel.IsValid())
	{
		EmitterHandleViewModel->SetIsEnabled(InCheckState == ECheckBoxState::Checked);
	}
}

EVisibility SNiagaraOverviewStackNode::GetShouldShowSummaryControls() const
{
	if(EmitterHandleViewModelWeak.IsValid() && EmitterHandleViewModelWeak.Pin()->GetEmitterViewModel()->GetEditorData().GetSummaryRoot()->GetChildren().Num() > 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FReply SNiagaraOverviewStackNode::OnToggleIsolateButtonClicked()
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	if (EmitterHandleViewModel.IsValid())
	{
		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			// Shift+click: toggle isolation only on this emitter, leaving other emitters' isolation state untouched.
			EmitterHandleViewModel->ToggleIsIsolated();
		}
		else
		{
			EmitterHandleViewModel->GetOwningSystemViewModel()->ToggleIsolateEmitterAndSelectedEmitters(EmitterHandleViewModel->GetId());
		}
	}

	return FReply::Handled();
}

FText SNiagaraOverviewStackNode::GetToggleIsolateToolTip() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	return EmitterHandleViewModel.IsValid() && EmitterHandleViewModel->GetIsIsolated()
		? LOCTEXT("TurnOffEmitterIsolation", "Disable emitter isolation.\nShift+click: toggle only this emitter without affecting other emitters' isolation state.")
		: LOCTEXT("IsolateThisEmitter", "Enable isolation for this emitter.\nShift+click: toggle only this emitter without affecting other emitters' isolation state.");
}

EVisibility SNiagaraOverviewStackNode::GetToggleIsolateVisibility() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	return EmitterHandleViewModel.IsValid() &&
		EmitterHandleViewModel->GetOwningSystemEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset 
		? EVisibility::Visible 
		: EVisibility::Collapsed;
}

const FSlateBrush* SNiagaraOverviewStackNode::GetToggleIsolateImage() const
{
	return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Isolate");
}

FSlateColor SNiagaraOverviewStackNode::GetToggleIsolateImageColor() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	return EmitterHandleViewModel.IsValid() && 
		EmitterHandleViewModel->GetIsIsolated()
		? FAppStyle::GetSlateColor("SelectionColor")
		: FLinearColor::Gray;
}

FText SNiagaraOverviewStackNode::GetToggleBoundsToolTip() const
{
	return LOCTEXT("ToggleBoundsTooltip", "Toggle debug emitter bounds visibility on / off");
}

FReply SNiagaraOverviewStackNode::OnToggleBoundsButtonClicked()
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel.IsValid() ? EmitterHandleViewModel->GetEmitterHandle() : nullptr;
	if (EmitterHandle)
	{
		EmitterHandle->SetDebugShowBounds(!EmitterHandle->GetDebugShowBounds());
	}
	return FReply::Handled();
}

EVisibility SNiagaraOverviewStackNode::GetToggleBoundsVisibility() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	return EmitterHandleViewModel.IsValid() &&
		EmitterHandleViewModel->GetOwningSystemEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

const FSlateBrush* SNiagaraOverviewStackNode::GetToggleBoundsImage() const
{
	return FAppStyle::Get().GetBrush("AssetEditor.ToggleShowBounds");
}

FSlateColor SNiagaraOverviewStackNode::GetToggleBoundsImageColor() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel.IsValid() ? EmitterHandleViewModel->GetEmitterHandle() : nullptr;
	return EmitterHandle && EmitterHandle->GetDebugShowBounds() ? FAppStyle::GetSlateColor("SelectionColor") : FLinearColor::Gray;
}

FSlateColor SNiagaraOverviewStackNode::GetScalabilityTintAlpha() const
{
	if(EmitterHandleViewModelWeak.IsValid())
	{
		float ScalabilityBaseAlpha = FNiagaraEditorUtilities::GetScalabilityTintAlpha(EmitterHandleViewModelWeak.Pin()->GetEmitterHandle());
		return FLinearColor(1, 1, 1, ScalabilityBaseAlpha * GetGraphZoomDistanceAlphaMultiplier());
	}

	return FLinearColor(1, 1, 1, 1);
}

void SNiagaraOverviewStackNode::OnScalabilityModeChanged(bool bActive)
{
	bScalabilityModeActive = bActive;
}

EVisibility SNiagaraOverviewStackNode::ShowExcludedOverlay() const
{
	// we only want actual results in scalability mode and for nodes representing emitters (not system nodes)
	if(bScalabilityModeActive)
	{
		if (UNiagaraSystemScalabilityViewModel* ScalabilityVM = ScalabilityViewModel.Get())
		{
			const TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = ScalabilityVM->GetSystemViewModel().Pin();
			if (SystemViewModel && !SystemViewModel->GetSystem().IsAllowedByScalability())
			{
				return EVisibility::HitTestInvisible;
			}
		}

		if (const TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin())
		{
			if (!EmitterHandleViewModel->GetEmitterHandle()->IsAllowedByScalability())
			{
				return EVisibility::HitTestInvisible;
			}
		}
	}
	
	return EVisibility::Hidden; 
}

float SNiagaraOverviewStackNode::GetGraphZoomDistanceAlphaMultiplier() const
{
	// we lower the alpha if the zoom amount is high
	float ZoomAmount = OwnerGraphPanelPtr.Pin()->GetZoomAmount();
	return FMath::Lerp(0.4f, 1.f, 1 - ZoomAmount);
}

FReply SNiagaraOverviewStackNode::OpenParentEmitter()
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	if (EmitterHandleViewModel.IsValid())
	{
		FNiagaraEditorUtilities::OpenParentEmitterForEdit(EmitterHandleViewModel->GetEmitterViewModel());
	}
	return FReply::Handled();
}

FText SNiagaraOverviewStackNode::OpenParentEmitterTooltip() const
{
	FString TooltipText = LOCTEXT("OpenAndFocusParentEmitterToolTip", "Open and Focus Parent Emitter").ToString();
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	
	if (EmitterHandleViewModel.IsValid() && EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter() && EmitterHandleViewModel->GetEmitterViewModel()->GetParentEmitter().Emitter->IsVersioningEnabled())
	{
		FVersionedNiagaraEmitter ParentEmitter = EmitterHandleViewModel->GetEmitterViewModel()->GetParentEmitter();
		FText ParentName = FText::FromString(ParentEmitter.Emitter->GetUniqueEmitterName());
		if (FVersionedNiagaraEmitterData* EmitterData = ParentEmitter.GetEmitterData())
		{
			TooltipText.Append(TEXT(":\n{0} - v{1}.{2}"));
			TooltipText = FText::Format(FText::FromString(TooltipText), ParentName, EmitterData->Version.MajorVersion, EmitterData->Version.MinorVersion).ToString();
		}
	}

	if(EmitterHandleViewModel.IsValid() && EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter())
	{
		TooltipText.Append(TEXT("\n{0}"));
		TooltipText = FText::Format(FText::FromString(TooltipText), EmitterHandleViewModel->GetEmitterViewModel()->GetParentPathNameText()).ToString();
	}
	
	return FText::FromString(TooltipText);
}

EVisibility SNiagaraOverviewStackNode::GetOpenParentEmitterVisibility() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	return EmitterHandleViewModel.IsValid() && 
		EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter()
		? EVisibility::Visible 
		: DisplayMode == EDisplayMode::Summary ? EVisibility::Hidden : EVisibility::Collapsed;
}

EVisibility SNiagaraOverviewStackNode::GetVersionSelectorVisibility() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	return EmitterHandleViewModel.IsValid() && 
		EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter() &&
		EmitterHandleViewModel->GetEmitterViewModel()->GetParentEmitter().Emitter->IsVersioningEnabled()
		? EVisibility::Visible 
		: DisplayMode == EDisplayMode::Summary ? EVisibility::Hidden : EVisibility::Collapsed;
}

FSlateColor SNiagaraOverviewStackNode::GetVersionSelectorColor() const
{
	if (TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin())
	{
		FVersionedNiagaraEmitter ParentEmitter = EmitterHandleViewModel->GetEmitterViewModel()->GetParentEmitter();
		if (ParentEmitter.Emitter && ParentEmitter.Emitter->IsVersioningEnabled())
		{
			FNiagaraAssetVersion ExposedVersion = ParentEmitter.Emitter->GetExposedVersion();
			if (ParentEmitter.GetEmitterData() == nullptr || ParentEmitter.GetEmitterData()->Version < ExposedVersion)
			{
				return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.IconColor.VersionUpgrade");
			}
		}
	}
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.FlatButtonColor");
}

TSharedRef<SWidget> SNiagaraOverviewStackNode::GetVersionSelectorDropdownMenu()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	if (TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin())
	{
		FVersionedNiagaraEmitter ParentEmitter = EmitterHandleViewModel->GetEmitterViewModel()->GetParentEmitter();
		TArray<FNiagaraAssetVersion> AssetVersions = ParentEmitter.Emitter->GetAllAvailableVersions();
		for (FNiagaraAssetVersion& Version : AssetVersions)
		{
			if (!Version.bIsVisibleInVersionSelector)
			{
				continue;
			}
			FVersionedNiagaraEmitterData* EmitterData = ParentEmitter.Emitter->GetEmitterData(Version.VersionGuid);
			bool bIsSelected = ParentEmitter.Version == Version.VersionGuid;
		
			FText Tooltip = LOCTEXT("NiagaraSelectVersion_Tooltip", "Select this version to use for the emitter");
			if (EmitterData && !EmitterData->VersionChangeDescription.IsEmpty())
			{
				Tooltip = FText::Format(LOCTEXT("NiagaraSelectVersionChangelist_Tooltip", "Select this version to use for the emitter. Change description for this version:\n{0}"), EmitterData->VersionChangeDescription);
			}
		
			FUIAction UIAction(FExecuteAction::CreateSP(this, &SNiagaraOverviewStackNode::SwitchToVersion, Version),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([bIsSelected]() { return bIsSelected; }));
			FText Format = (Version == ParentEmitter.Emitter->GetExposedVersion()) ? FText::FromString("{0}.{1}*") : FText::FromString("{0}.{1}");
			FText Label = FText::Format(Format, Version.MajorVersion, Version.MinorVersion);
			MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);	
		}
	}

	return MenuBuilder.MakeWidget();
}

// this switches the referenced parent version, for the version selector in the toolbar see FNiagaraSystemViewModel::ChangeEmitterVersion
void SNiagaraOverviewStackNode::SwitchToVersion(FNiagaraAssetVersion Version)
{
	if (TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin())
	{
		FNiagaraEditorUtilities::SwitchParentEmitterVersion(EmitterHandleViewModel->GetEmitterViewModel(), EmitterHandleViewModel->GetOwningSystemViewModel(), Version.VersionGuid);
	}
}

#undef LOCTEXT_NAMESPACE
