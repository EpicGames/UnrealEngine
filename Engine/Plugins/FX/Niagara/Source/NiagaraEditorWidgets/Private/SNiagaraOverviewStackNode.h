// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SInvalidationPanel.h"
#include "SNiagaraOverviewStack.h"
#include "ViewModels/NiagaraSystemScalabilityViewModel.h"

class UNiagaraOverviewNode;
class UNiagaraStackViewModel;
class UNiagaraSystemSelectionViewModel;
class FNiagaraEmitterHandleViewModel;
class FAssetThumbnail;
class UNiagaraStackEntry;

class SNiagaraOverviewStackNode : public SGraphNode
{
public:
	enum class EDisplayMode : uint8
	{
		Default,
		Summary
	};
	
	SLATE_BEGIN_ARGS(SNiagaraOverviewStackNode) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, UNiagaraOverviewNode* InNode);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	virtual ~SNiagaraOverviewStackNode() override;
protected:
	void BindEditorDataDelegates();
	void UnbindEditorDataDelegates() const;
	
	virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle) override;
	virtual TSharedRef<SWidget> CreateTitleRightWidget() override;
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	TSharedRef<SWidget> CreateTopContentBar();

	TSharedRef<SWidget> CreateTitleWidget_Default(TSharedPtr<SNodeTitle> NodeTitle);
	TSharedRef<SWidget> CreateTitleRightWidget_Default();
	TSharedRef<SWidget> CreateNodeContentArea_Default();
	TSharedRef<SWidget> CreateTopContentBar_Default();
	
	TSharedRef<SWidget> CreateTitleWidget_Summary(TSharedPtr<SNodeTitle> NodeTitle);
	TSharedRef<SWidget> CreateTitleRightWidget_Summary();
	TSharedRef<SWidget> CreateNodeContentArea_Summary();
	TSharedRef<SWidget> CreateTopContentBar_Summary();

	virtual bool IsHidingPinWidgets() const override { return UseLowDetailNodeContent(); }
	
	bool UseLowDetailNodeContent() const;
	FVector2D GetLowDetailDesiredSize() const;
	FOptionalSize GetLowDetailDesiredWidth() const;
	FOptionalSize GetLowDetailDesiredHeight() const;
	FText GetLowDetailNodeTitle() const;

	TSharedRef<SWidget> CreateEnabledCheckbox();
	TSharedRef<SButton> CreateIsolateButton();
	TSharedRef<SButton> CreateBoundsButton();
	TSharedRef<SWidget> CreateEmitterThumbnail();
	TSharedRef<SWidget> CreateEmitterRendererThumbnails();
	TSharedRef<SWidget> CreateSummaryViewToggle();
	TSharedRef<SWidget> CreateLocalSpaceToggle();
	TSharedRef<SButton> CreateCaptureThumbnailButton();
	TSharedRef<SWidget> CreateOpenParentButton();
	TSharedRef<SWidget> CreateVersionSelectorButton();
	TSharedRef<SWidget> CreateScalabilityControls();
	TSharedRef<SWidget> CreateSimTargetToggle();
	TSharedRef<SWidget> CreateDeterminismToggle();
	TSharedRef<SWidget> CreatePropertiesButton();
	
	virtual void UpdateGraphNode() override;
private:
	void RefreshEmitterThumbnailPreview();

	virtual TOptional<ETextOverflowPolicy> GetNameOverflowPolicy() const override;
	EVisibility GetIssueIconVisibility() const;
	
	EVisibility GetEnabledCheckBoxVisibility() const;
	ECheckBoxState GetEnabledCheckState() const;
	const FSlateBrush* GetEnabledImage() const;
	void OnEnabledCheckStateChanged(ECheckBoxState InCheckState);
	EVisibility GetShouldShowSummaryControls() const;
	FReply OnCaptureThumbnailButtonClicked();
	FReply OnRendererThumbnailClicked(class UNiagaraStackRendererItem* StackRendererItem);
	FReply OnPropertiesButtonClicked() const;
	
	FText GetToggleIsolateToolTip() const;
	FReply OnToggleIsolateButtonClicked();
	EVisibility GetToggleIsolateVisibility() const;
	const FSlateBrush* GetToggleIsolateImage() const;
	FSlateColor GetToggleIsolateImageColor() const;

	FText GetToggleBoundsToolTip() const;
	FReply OnToggleBoundsButtonClicked();
	EVisibility GetToggleBoundsVisibility() const;
	const FSlateBrush* GetToggleBoundsImage() const;
	FSlateColor GetToggleBoundsImageColor() const;

	EVisibility GetScalabilityIndicatorVisibility() const;
	FSlateColor GetScalabilityTintAlpha() const;
	void OnScalabilityModeChanged(bool bActive);
	EVisibility ShowExcludedOverlay() const;
	float GetGraphZoomDistanceAlphaMultiplier() const;

	FText GetSpawnCountScaleText() const;
	FText GetSpawnCountScaleTooltip() const;
	EVisibility GetSpawnCountScaleTextVisibility() const;
	ECheckBoxState IsScalabilityModeActive() const;
	void OnScalabilityModeStateChanged(ECheckBoxState CheckBoxState);

	FReply OnCycleThroughIssues();
	FReply OpenParentEmitter();
	FText OpenParentEmitterTooltip() const;
	EVisibility GetOpenParentEmitterVisibility() const;
	EVisibility GetVersionSelectorVisibility() const;
	FSlateColor GetVersionSelectorColor() const;
	TSharedRef<SWidget> GetVersionSelectorDropdownMenu();
	void SwitchToVersion(FNiagaraAssetVersion Version);

	UNiagaraOverviewNode* OverviewStackNode = nullptr;
	UNiagaraStackViewModel* StackViewModel = nullptr;
	UNiagaraSystemSelectionViewModel* OverviewSelectionViewModel = nullptr;
	TWeakObjectPtr<UNiagaraSystemScalabilityViewModel> ScalabilityViewModel;
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelWeak;
	TSharedPtr<SOverlay> ScalabilityWrapper;
	TSharedPtr<SInvalidationPanel> ContentAreaWidget;

	TArray<UNiagaraStackEntry*> RendererPreviewStackEntries;
	int32 CurrentIssueIndex = 0;
	bool bScalabilityModeActive = false;

	EDisplayMode DisplayMode = EDisplayMode::Default;
	
	/** Cached size from when we last drew at high detail */
	FVector2D LastHighDetailSize = FVector2D::ZeroVector;
	int32 GeometryTickForSize = 3;
	mutable TPair<FString, FText> LowDetailTitleCache;

	TSharedPtr<SBox> ThumbnailContainer;
	TSharedPtr<FAssetThumbnail> PreviewThumbnail;
};