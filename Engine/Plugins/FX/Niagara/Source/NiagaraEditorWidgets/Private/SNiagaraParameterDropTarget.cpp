// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraParameterDropTarget.h"

#include "NiagaraActions.h"
#include "NiagaraEditorUtilities.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterDropTarget"

void SNiagaraParameterDropTarget::Construct(const FArguments& InArgs)
{
	SDropTarget::Construct(InArgs._DropTargetArgs);

	TargetParameter = InArgs._TargetParameter;
	TypeToTestAgainst = InArgs._TypeToTestAgainst;
	ExecutionCategory = InArgs._ExecutionCategory;
}

FReply SNiagaraParameterDropTarget::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FNiagaraParameterDragOperation> ParameterDragOperation = DragDropEvent.GetOperationAs<FNiagaraParameterDragOperation>();

	if(ParameterDragOperation.IsValid())
	{
		ensure(TargetParameter.IsSet() || TypeToTestAgainst.IsSet());
		
		FText TargetText = TargetParameter.IsSet() ? FText::FromName(TargetParameter->GetName()) : TypeToTestAgainst.IsSet() ? TypeToTestAgainst->GetNameText() : FText::GetEmpty();
		
		if(TargetParameter.IsSet())
		{
			TypeToTestAgainst = TargetParameter->GetType();
		}
		
		if(!TargetText.IsEmpty() && ExecutionCategory->IsValid())
		{
			TSharedPtr<FNiagaraParameterAction> Action = StaticCastSharedPtr<FNiagaraParameterAction>(ParameterDragOperation->GetSourceAction());
			bool bAllowedInExecutionCategory = FNiagaraStackGraphUtilities::ParameterAllowedInExecutionCategory(Action->GetParameter().GetName(), ExecutionCategory.GetValue());
			FNiagaraTypeDefinition DropType = Action->GetParameter().GetType();

			if (bAllowedInExecutionCategory)
			{
				// check if we can simply link the input directly
				if(FNiagaraEditorUtilities::AreTypesAssignable(DropType, TypeToTestAgainst.GetValue()))
				{
					// if we can directly connect, we don't need any additional text
					ParameterDragOperation->SetAdditionalText(FText::GetEmpty()); 
				}
				// check if we can use a conversion script
				else if(UNiagaraStackFunctionInput::GetPossibleConversionScripts(DropType, TypeToTestAgainst.GetValue()).Num() > 0)
				{
					UNiagaraScript* ConversionScriptToUse = UNiagaraStackFunctionInput::GetPossibleConversionScripts(DropType, TypeToTestAgainst.GetValue())[0];
					FText ConversionScriptDescription = ConversionScriptToUse->GetLatestScriptData()->Description;
					FText ModifiedConversionScriptDescription = ConversionScriptDescription.IsEmpty() ? FText::GetEmpty() : FText::FromString("\n" + ConversionScriptDescription.ToString()); 
					if(FNiagaraTypeDefinition::IsLossyConversion(Action->GetParameter().GetType(), TypeToTestAgainst.GetValue()))
					{
						FText Format = LOCTEXT("RequiresLossyAdapterDragDropInfo", "A conversion script will be inserted to adapt {0} to {1}.\nThis is a potentially lossy conversion.{2}");
						FText HoverTooltipText = FText::FormatOrdered(Format, Action->GetParameter().GetType().GetNameText(), TypeToTestAgainst.GetValue().GetNameText(), ModifiedConversionScriptDescription);
						ParameterDragOperation->SetAdditionalText(HoverTooltipText); 
					}
					else
					{
						FText Format = LOCTEXT("RequiresAdapterDragDropInfo", "A conversion script will be inserted to adapt {0} to {1}.{2}");
						FText HoverTooltipText = FText::FormatOrdered(Format, Action->GetParameter().GetType().GetNameText(), TypeToTestAgainst.GetValue().GetNameText(), ModifiedConversionScriptDescription);
						ParameterDragOperation->SetAdditionalText(HoverTooltipText); 
					}
				}
			}
			else
			{
				FText Format = LOCTEXT("ExecutionCategoryNotAllowed", "This parameter is not allowed in execution category {0}.");
				FText HoverTooltipText = FText::FormatOrdered(Format, FText::FromName(ExecutionCategory.GetValue()));
				ParameterDragOperation->SetAdditionalText(HoverTooltipText); 
			}
		}
	}
	
	AllowDrop(DragDropEvent.GetOperation());
	// we always handle the drag because we don't want another widget to interfere with the feedback we are giving
	return FReply::Handled();
}

void SNiagaraParameterDropTarget::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SDropTarget::OnDragLeave(DragDropEvent);

	TSharedPtr<FNiagaraParameterDragOperation> ParameterDragOperation = DragDropEvent.GetOperationAs<FNiagaraParameterDragOperation>();

	if(ParameterDragOperation.IsValid())
	{
		ParameterDragOperation->ResetToDefaultToolTip();
	}
}

#undef LOCTEXT_NAMESPACE
