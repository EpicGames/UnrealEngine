// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SDropTarget.h"
#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/StyleColors.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"

#define UE_API NIAGARAEDITORWIDGETS_API

class SNiagaraParameterDropTarget : public SDropTarget
{
public:
    SLATE_BEGIN_ARGS(SNiagaraParameterDropTarget)
	    : _DropTargetArgs(SDropTarget::FArguments())
		, _TargetParameter(TOptional<FNiagaraVariable>())
		, _TypeToTestAgainst(TOptional<FNiagaraTypeDefinition>())
		, _ExecutionCategory(TOptional<FName>())
	{ }
		SLATE_ARGUMENT(SDropTarget::FArguments, DropTargetArgs)
		SLATE_ARGUMENT(TOptional<FNiagaraVariable>, TargetParameter)
		SLATE_ARGUMENT(TOptional<FNiagaraTypeDefinition>, TypeToTestAgainst)
		SLATE_ARGUMENT(TOptional<FName>, ExecutionCategory)
    SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
protected:
    UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
private:
	TOptional<FName> ExecutionCategory;
	TOptional<FNiagaraVariable> TargetParameter;
	TOptional<FNiagaraTypeDefinition> TypeToTestAgainst;
};

#undef UE_API
