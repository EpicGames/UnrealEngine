// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraScalabilityPreviewSettings.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "ISinglePropertyView.h"
#include "NiagaraDeviceProfileViewModel.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraSettings.h"
#include "PlatformInfo.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "NiagaraScalabilityPreviewSettings"

void SNiagaraScalabilityPreviewSettings::Construct(const FArguments& InArgs, UNiagaraSystemScalabilityViewModel& InScalabilityViewModel)
{
	ScalabilityViewModel = &InScalabilityViewModel;

	ChildSlot
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(2.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ScalabilityPreviewSettingsLabel", "Preview"))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.Padding(2.f)
		.AutoWidth()
		[
			CreatePreviewQualityLevelWidgets()
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
			.Size(FVector2D(1, 1))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SAssignNew(PlatformMenuAnchor, SMenuAnchor)
			.OnGetMenuContent(this, &SNiagaraScalabilityPreviewSettings::GenerateDeviceProfileTreeWidget)
			[				
				CreatePreviewPlatformWidgets()
			]
		]
	];
}

TSharedRef<SWidget> SNiagaraScalabilityPreviewSettings::CreatePreviewQualityLevelWidgets()
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	TSharedRef<SWrapBox> QualityLevelWidgetsBox = SNew(SWrapBox)
	.PreferredSize(600.f)
	.UseAllottedSize(true);
	
	int32 NumQualityLevels = Settings->QualityLevels.Num();

	for (int32 QualityLevel = 0; QualityLevel < NumQualityLevels; ++QualityLevel)
	{
		bool First = QualityLevel == 0;
		bool Last = QualityLevel == (NumQualityLevels - 1);
		
		QualityLevelWidgetsBox->AddSlot()
		.Padding(0, 0, 1, 0)
		[
			SNew(SBox)
			//.WidthOverride(80.f)
			.VAlign(VAlign_Top)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SCheckBox)
					.ToolTipText(GetQualityButtonTooltip(QualityLevel))
					.Style(FNiagaraEditorStyle::Get(), First ? "NiagaraEditor.PlatformSet.StartButton" : 
						(Last ? "NiagaraEditor.PlatformSet.EndButton" : "NiagaraEditor.PlatformSet.MiddleButton"))
					.IsChecked(this, &SNiagaraScalabilityPreviewSettings::IsQLChecked, QualityLevel)
					.OnCheckStateChanged(this, &SNiagaraScalabilityPreviewSettings::QLCheckStateChanged, QualityLevel)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Fill)
						.Padding(6,2,6,4)
						[
							SNew(STextBlock)
							.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.PlatformSet.ButtonText")
							.Text(FNiagaraPlatformSet::GetQualityLevelText(QualityLevel))
							.ColorAndOpacity(this, &SNiagaraScalabilityPreviewSettings::GetQualityLevelButtonTextColor, QualityLevel)
							.ShadowOffset(FVector2D(1, 1))
						]
					]
				]
			]
		];
	}

	return QualityLevelWidgetsBox;
}

TSharedRef<SWidget> SNiagaraScalabilityPreviewSettings::CreatePreviewPlatformWidgets()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SNiagaraScalabilityPreviewSettings::OnResetPreviewPlatformClicked)
			.ToolTipText(LOCTEXT("ResetPreviewPlatformToolTip", "Reset the platform to preview."))
			.Visibility_Lambda([this]()
			{
				return ScalabilityViewModel->GetPreviewDeviceProfile().IsSet() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			[
				SNew(SImage)
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Scalability.Preview.ResetPlatform"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
	       .ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
	       .ForegroundColor(FSlateColor::UseForeground())
	       .ToolTipText(LOCTEXT("ChoosePreviewPlatform", "Choose a platform to preview. This is cosmetic only."))
	       .OnClicked(this, &SNiagaraScalabilityPreviewSettings::TogglePlatformMenuOpen)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.f)
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(16.f)
					.HeightOverride(16.f)
					.Visibility_Lambda([this]()
				    {
						return GetActivePreviewPlatformImage() != nullptr
							? EVisibility::Visible
					        : EVisibility::Collapsed;
				    })
					[
						SNew(SImage)
						.Image(this, &SNiagaraScalabilityPreviewSettings::GetActivePreviewPlatformImage)
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(2.f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &SNiagaraScalabilityPreviewSettings::GetActivePreviewPlatformName)
				]
				+ SHorizontalBox::Slot()
				.Padding(2.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(8.f)
					.HeightOverride(8.f)
					[
						SNew(SImage)
						.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.DropdownButton"))
					]
				]
			]
		];
}

ECheckBoxState SNiagaraScalabilityPreviewSettings::IsQLChecked(int32 QualityLevel)const
{	
	return ScalabilityViewModel->IsViewModeQualityEnabled(QualityLevel) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;	
}

void SNiagaraScalabilityPreviewSettings::QLCheckStateChanged(ECheckBoxState CheckState, int32 QualityLevel)
{
	ScalabilityViewModel->UpdatePreviewQualityLevel(QualityLevel);
}

FSlateColor SNiagaraScalabilityPreviewSettings::GetQualityLevelButtonTextColor(int32 QualityLevel) const
{
	return ScalabilityViewModel->IsViewModeQualityEnabled(QualityLevel) ? 
		FSlateColor(FLinearColor(0.95f, 0.95f, 0.95f)) :
		FSlateColor::UseForeground();
}

const FSlateBrush* SNiagaraScalabilityPreviewSettings::GetActivePreviewPlatformImage() const
{
	if (UDeviceProfile* DeviceProfile = ScalabilityViewModel->GetPreviewDeviceProfile().Get(nullptr))
	{
		FName PlatformName = *DeviceProfile->DeviceType;
		if (const PlatformInfo::FTargetPlatformInfo* Info = PlatformInfo::FindPlatformInfo(PlatformName))
		{
			const FSlateBrush* DeviceProfileTypeIcon = FAppStyle::GetBrush(Info->GetIconStyleName(EPlatformIconSize::Normal));
			if (DeviceProfileTypeIcon != FAppStyle::Get().GetDefaultBrush())
			{
				return DeviceProfileTypeIcon;
			}
		}
	}
	
	return nullptr;
}

FText SNiagaraScalabilityPreviewSettings::GetActivePreviewPlatformName() const
{
	if(ScalabilityViewModel->GetPreviewDeviceProfile().IsSet())
	{
		return FText::FromString(ScalabilityViewModel->GetPreviewDeviceProfile().GetValue()->GetName());
	}
	else
	{
		return LOCTEXT("Scalability_NoDeviceProfileSet", "No Preview Device Set");
	}
}

FText SNiagaraScalabilityPreviewSettings::GetQualityButtonTooltip(int32 QualityLevel) const
{
	FText QualityLevelText = FNiagaraPlatformSet::GetQualityLevelText(QualityLevel);
	return FText::FormatOrdered(LOCTEXT("QualityButtonTooltip", "Activates {0} preview. This is cosmetic only.\nEmitters and renderers excluded from preview will show a red overlay and will not be active in the viewport and PIE."), QualityLevelText);
}

void SNiagaraScalabilityPreviewSettings::CreateDeviceProfileTree()
{
	//Pull device profiles out by their hierarchy depth.
	TArray<TArray<UDeviceProfile*>> DeviceProfilesByHierarchyLevel;
	for (UObject* Profile : UDeviceProfileManager::Get().Profiles)
	{
		UDeviceProfile* DeviceProfile = CastChecked<UDeviceProfile>(Profile);

		if (DeviceProfile && DeviceProfile->IsVisibleForAssets())
		{
			TFunction<void(int32&, UDeviceProfile*)> FindDepth;
			FindDepth = [&](int32& Depth, UDeviceProfile* CurrProfile)
			{
				if (CurrProfile->Parent)
				{
					FindDepth(++Depth, Cast<UDeviceProfile>(CurrProfile->Parent));
				}
			};

			int32 ProfileDepth = 0;
			FindDepth(ProfileDepth, DeviceProfile);
			DeviceProfilesByHierarchyLevel.SetNum(FMath::Max(ProfileDepth + 1, DeviceProfilesByHierarchyLevel.Num()));
			DeviceProfilesByHierarchyLevel[ProfileDepth].Add(DeviceProfile);
		}
	}
	
	FullDeviceProfileTree.Reset(DeviceProfilesByHierarchyLevel[0].Num());
	for (int32 RootProfileIdx = 0; RootProfileIdx < DeviceProfilesByHierarchyLevel[0].Num(); ++RootProfileIdx)
	{
		TFunction<void(FNiagaraDeviceProfileViewModel*, int32)> BuildProfileTree;
		BuildProfileTree = [&](FNiagaraDeviceProfileViewModel* CurrRoot, int32 CurrLevel)
		{
			int32 NextLevel = CurrLevel + 1;
			if (NextLevel < DeviceProfilesByHierarchyLevel.Num())
			{
				for (UDeviceProfile* PossibleChild : DeviceProfilesByHierarchyLevel[NextLevel])
				{
					if (PossibleChild->Parent == CurrRoot->Profile)
					{
						TSharedPtr<FNiagaraDeviceProfileViewModel>& NewChild = CurrRoot->Children.Add_GetRef(MakeShared<FNiagaraDeviceProfileViewModel>());
						NewChild->Profile = PossibleChild;
						BuildProfileTree(NewChild.Get(), NextLevel);
					}
				}
			}
		};

		//Add all root nodes and build their trees.
		TSharedPtr<FNiagaraDeviceProfileViewModel> CurrRoot = MakeShared<FNiagaraDeviceProfileViewModel>();
		CurrRoot->Profile = DeviceProfilesByHierarchyLevel[0][RootProfileIdx];
		BuildProfileTree(CurrRoot.Get(), 0);
		FullDeviceProfileTree.Add(CurrRoot);
	}

	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	int32 NumQualityLevels = Settings->QualityLevels.Num();
	FilteredDeviceProfileTrees.SetNum(NumQualityLevels);
	
	for (TSharedPtr<FNiagaraDeviceProfileViewModel>& FullDeviceRoot : FullDeviceProfileTree)
	{
		for (int32 QualityLevel = 0; QualityLevel < NumQualityLevels; ++QualityLevel)
		{			
			TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>& FilteredRoots = FilteredDeviceProfileTrees[QualityLevel];

			TSharedPtr<FNiagaraDeviceProfileViewModel>& FilteredRoot = FilteredRoots.Add_GetRef(MakeShared<FNiagaraDeviceProfileViewModel>());
			FilteredRoot->Profile = FullDeviceRoot->Profile;
		}
	}
}

TSharedRef<SWidget> SNiagaraScalabilityPreviewSettings::GenerateDeviceProfileTreeWidget()
{
	if (FullDeviceProfileTree.Num() == 0)
	{
		CreateDeviceProfileTree();	
	}

	TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>* TreeToUse = &FullDeviceProfileTree;

	TSharedPtr<FNiagaraDeviceProfileViewModel> ClearViewModel = nullptr;
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		[
			SAssignNew(DeviceProfileTreeWidget, STreeView<TSharedPtr<FNiagaraDeviceProfileViewModel>>)
			.TreeItemsSource(TreeToUse)
			.OnGenerateRow(this, &SNiagaraScalabilityPreviewSettings::OnGenerateDeviceProfileTreeRow)
			.OnGetChildren(this, &SNiagaraScalabilityPreviewSettings::OnGetDeviceProfileTreeChildren)
			.SelectionMode(ESelectionMode::None)
		];
}

TSharedRef<ITableRow> SNiagaraScalabilityPreviewSettings::OnGenerateDeviceProfileTreeRow(
	TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SHorizontalBox> RowContainer;
	SAssignNew(RowContainer, SHorizontalBox);

	int32 ProfileMask = ScalabilityViewModel->GetPreviewPlatformSet()->GetActiveQualityMaskForDeviceProfile(InItem->Profile);
	FText NameTooltip = FText::Format(LOCTEXT("ProfileQLTooltipFmt", "Effects Quality: {0}"), FNiagaraPlatformSet::GetQualityLevelMaskText(ProfileMask));
	
	//Top level profile. Look for a platform icon.
	if (InItem->Profile->Parent == nullptr)
	{
		if (const PlatformInfo::FTargetPlatformInfo* Info = PlatformInfo::FindPlatformInfo(*InItem->Profile->DeviceType))
		{
			const FSlateBrush* DeviceProfileTypeIcon = FAppStyle::GetBrush(Info->GetIconStyleName(EPlatformIconSize::Normal));
			if (DeviceProfileTypeIcon != FAppStyle::Get().GetDefaultBrush())
			{
				RowContainer->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(4, 0, 0, 0)
					[
						SNew(SBox)
						.WidthOverride(16)
						.HeightOverride(16)
						[
							SNew(SImage)
							.Image(DeviceProfileTypeIcon)
						]
					];
			}
		}
	}

	FName TextStyleName("NormalText");
	FSlateColor TextColor(FSlateColor::UseForeground());

	RowContainer->AddSlot()
		.Padding(4, 2, 0, 2)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(this, &SNiagaraScalabilityPreviewSettings::OnProfileMenuButtonClicked, InItem)
			.ForegroundColor(TextColor)
			.ToolTipText(NameTooltip)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TextStyleName)
				.Text(FText::FromString(InItem->Profile->GetName()))
			]
		];

	return SNew(STableRow<TSharedPtr<FNiagaraDeviceProfileViewModel>>, OwnerTable)
		.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.PlatformSet.TreeView")
		[
			RowContainer.ToSharedRef()
		];
}

void SNiagaraScalabilityPreviewSettings::OnGetDeviceProfileTreeChildren(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem,
	TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>& OutChildren)
{
	OutChildren = InItem->Children;
}

FReply SNiagaraScalabilityPreviewSettings::TogglePlatformMenuOpen()
{
	PlatformMenuAnchor->SetIsOpen(!PlatformMenuAnchor->IsOpen());
	return FReply::Handled();
}

FReply SNiagaraScalabilityPreviewSettings::OnProfileMenuButtonClicked(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem)
{
	ScalabilityViewModel->UpdatePreviewDeviceProfile(InItem->Profile);
	TogglePlatformMenuOpen();
	
	return FReply::Handled();
}

FReply SNiagaraScalabilityPreviewSettings::OnResetPreviewPlatformClicked()
{
	ScalabilityViewModel->UpdatePreviewDeviceProfile(nullptr);	
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
