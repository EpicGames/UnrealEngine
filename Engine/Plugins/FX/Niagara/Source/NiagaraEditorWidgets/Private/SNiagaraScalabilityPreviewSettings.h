// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraSystemScalabilityViewModel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "ViewModels/NiagaraSystemViewModel.h"

struct FNiagaraDeviceProfileViewModel;
class SNiagaraScalabilityPreviewSettings : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SNiagaraScalabilityPreviewSettings)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraSystemScalabilityViewModel& InScalabilityViewModel);

private:
	TSharedRef<SWidget> CreatePreviewQualityLevelWidgets();
	TSharedRef<SWidget> CreatePreviewPlatformWidgets();

	ECheckBoxState IsQLChecked(int32 QualityLevel) const;
	void QLCheckStateChanged(ECheckBoxState CheckState, int32 QualityLevel);
	FSlateColor GetQualityLevelButtonTextColor(int32 QualityLevel) const;
	const FSlateBrush* GetActivePreviewPlatformImage() const;
	FText GetActivePreviewPlatformName() const;
	FText GetQualityButtonTooltip(int32 QualityLevel) const;

	void CreateDeviceProfileTree();
	TSharedRef<SWidget> GenerateDeviceProfileTreeWidget();
	TSharedRef<ITableRow> OnGenerateDeviceProfileTreeRow(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetDeviceProfileTreeChildren(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, TArray< TSharedPtr<FNiagaraDeviceProfileViewModel> >& OutChildren);
	FReply TogglePlatformMenuOpen();

	FReply OnProfileMenuButtonClicked(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem);
	FReply OnResetPreviewPlatformClicked();
	
	TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>> FullDeviceProfileTree;
	TArray<TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>> FilteredDeviceProfileTrees;
	TSharedPtr<STreeView<TSharedPtr<FNiagaraDeviceProfileViewModel>>> DeviceProfileTreeWidget;

private:
	TWeakObjectPtr<UNiagaraSystemScalabilityViewModel> ScalabilityViewModel;
	TSharedPtr<SMenuAnchor> PlatformMenuAnchor;
};
