// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraScratchPadScriptManager.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "Widgets/SDynamicLayoutBox.h"
#include "Widgets/SItemSelector.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "Widgets/SNiagaraScriptGraph.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraScratchPadCommandContext.h"
#include "Widgets/SNiagaraParameterPanel.h"

#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "SResizeBox.h"
#include "EditorStyleSet.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "EditorFontGlyphs.h"
#include "NiagaraConstants.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraScratchPad"

FName SNiagaraScratchPadScriptManager::ScriptSelectorName = "ScriptSelector";
FName SNiagaraScratchPadScriptManager::ScriptParameterPanelName = "ScriptParameterPanel";
FName SNiagaraScratchPadScriptManager::ScriptEditorName = "ScriptEditor";
FName SNiagaraScratchPadScriptManager::SelectionEditorName = "SelectionEditor";
FName SNiagaraScratchPadScriptManager::WideLayoutName = "Wide";
FName SNiagaraScratchPadScriptManager::NarrowLayoutName = "Narrow";

class SNiagaraScratchPadScriptRow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadScriptRow) {}
		SLATE_ATTRIBUTE(bool, IsSelected);
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		UNiagaraScratchPadViewModel* InScratchPadViewModel,
		TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel,
		TSharedPtr<FNiagaraScratchPadCommandContext> InCommandContext)
	{
		ScratchPadViewModel = InScratchPadViewModel;
		ScriptViewModel = InScriptViewModel;
		CommandContext = InCommandContext;
		IsSelected = InArgs._IsSelected;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(3, 0, 0, 0)
			[
				SAssignNew(NameEditableText, SInlineEditableTextBlock)
				.Text(this, &SNiagaraScratchPadScriptRow::GetNameText)
				.ToolTipText_Static(&SNiagaraScratchPadScriptRow::GetTooltipTextForSelection, ScriptViewModel.ToSharedRef())
				.IsSelected(this, &SNiagaraScratchPadScriptRow::GetIsSelected)
				.OnVerifyTextChanged(this, &SNiagaraScratchPadScriptRow::VerifyNameTextChange)
				.OnTextCommitted(this, &SNiagaraScratchPadScriptRow::OnNameTextCommitted)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			[
				SNew(STextBlock)
				.Visibility(this, &SNiagaraScratchPadScriptRow::GetUnappliedChangesVisibility)
				.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.ScratchPad.EditorHeaderText")
				.Text(FText::FromString(TEXT("*")))
				.ToolTipText(ScriptViewModel.ToSharedRef(), &FNiagaraScratchPadScriptViewModel::GetToolTip)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]
		];
	}

	static FText GetTooltipTextForSelection(TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel)
	{
		return FText::Format(LOCTEXT("ScratchTooltip", "Script: {0}\r\n{1}\r\n\r\nDouble click to edit script or drag & drop onto the stack.\r\nAn \"*\" indicates that the script has been edited but not applied yet."), InScriptViewModel->GetDisplayName(), InScriptViewModel->GetToolTip());
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (ScriptViewModel->GetIsPendingRename())
		{
			ScriptViewModel->SetIsPendingRename(false);
			NameEditableText->EnterEditingMode();
		}
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			return FReply::Handled()
				.CaptureMouse(SharedThis(this));
		}
		return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Set this script to be the active one.
			ScratchPadViewModel->SetActiveScriptViewModel(ScriptViewModel.ToSharedRef());

			FMenuBuilder MenuBuilder(true, CommandContext->GetCommands());
			CommandContext->AddMenuItems(MenuBuilder);

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			return FReply::Handled().ReleaseMouseCapture();
		}
		return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

private:
	FText GetNameText() const
	{
		return ScriptViewModel->GetDisplayName();
	}

	bool GetIsSelected() const
	{
		return IsSelected.Get(false);
	}

	bool VerifyNameTextChange(const FText& InNewNameText, FText& OutErrorMessage)
	{
		if (InNewNameText.IsEmpty())
		{
			OutErrorMessage = NSLOCTEXT("NiagaraScratchPadScriptName", "EmptyNameErrorMessage", "Script name can not be empty.");
			return false;
		}
		if (InNewNameText.ToString().Len() > FNiagaraConstants::MaxScriptNameLength)
		{
			OutErrorMessage = FText::Format(NSLOCTEXT("NiagaraScratchPadScriptName", "NameTooLongErrorFormat", "The name entered is too long.\nThe maximum script name length is {0}."), FText::AsNumber(FNiagaraConstants::MaxScriptNameLength));
			return false;
		}

		return true;
	}

	void OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
	{
		ScriptViewModel->SetScriptName(InText);
	}

	bool IsActive() const
	{
		return IsHovered();
	}

	EVisibility GetUnappliedChangesVisibility() const
	{
		return ScriptViewModel->HasUnappliedChanges() ? EVisibility::Visible : EVisibility::Collapsed;
	}

private:
	UNiagaraScratchPadViewModel* ScratchPadViewModel;
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel;
	TSharedPtr<FNiagaraScratchPadCommandContext> CommandContext;
	TAttribute<bool> IsSelected;
	TSharedPtr<SInlineEditableTextBlock> NameEditableText;
};

class SNiagaraScratchPadScriptSelector : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadScriptSelector) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraScratchPadViewModel* InViewModel, TSharedPtr<FNiagaraScratchPadCommandContext> InCommandContext)
	{
		ViewModel = InViewModel;
		CommandContext = InCommandContext;
		ViewModel->OnScriptViewModelsChanged().AddSP(this, &SNiagaraScratchPadScriptSelector::ScriptViewModelsChanged);
		ViewModel->OnActiveScriptChanged().AddSP(this, &SNiagaraScratchPadScriptSelector::ActiveScriptChanged);
		bIsUpdatingSelection = false;

		ChildSlot
		[
			SAssignNew(ScriptSelector, SNiagaraScriptViewModelSelector)
			.ClickActivateMode(EItemSelectorClickActivateMode::DoubleClick)
			.CategoryRowStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.ScratchPad.CategoryRow")
			.ClearSelectionOnClick(false)
			.Items(ViewModel->GetScriptViewModels())
			.DefaultCategories(ViewModel->GetAvailableUsages())
			.OnGetCategoriesForItem(this, &SNiagaraScratchPadScriptSelector::OnGetCategoriesForItem)
			.OnCompareCategoriesForEquality(this, &SNiagaraScratchPadScriptSelector::OnCompareCategoriesForEquality)
			.OnCompareCategoriesForSorting(this, &SNiagaraScratchPadScriptSelector::OnCompareCategoriesForSorting)
			.OnCompareItemsForEquality(this, &SNiagaraScratchPadScriptSelector::OnCompareItemsForEquality)
			.OnCompareItemsForSorting(this, &SNiagaraScratchPadScriptSelector::OnCompareItemsForSorting)
			.OnDoesItemMatchFilterText(this, &SNiagaraScratchPadScriptSelector::OnDoesItemMatchFilterText)
			.OnGenerateWidgetForCategory(this, &SNiagaraScratchPadScriptSelector::OnGenerateWidgetForCategory)
			.OnGenerateWidgetForItem(this, &SNiagaraScratchPadScriptSelector::OnGenerateWidgetForItem)
			.OnItemActivated(this, &SNiagaraScratchPadScriptSelector::OnScriptActivated)
			.OnItemsDragged(this, &SNiagaraScratchPadScriptSelector::OnScriptDragged)
			.OnSelectionChanged(this, &SNiagaraScratchPadScriptSelector::OnSelectionChanged)

		];

		if (ViewModel->GetActiveScriptViewModel().IsValid())
		{
			TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> SelectedViewModels;
			SelectedViewModels.Add(ViewModel->GetActiveScriptViewModel().ToSharedRef());
			ScriptSelector->SetSelectedItems(SelectedViewModels);
		}
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (CommandContext->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			return FReply::Handled()
				.CaptureMouse(SharedThis(this));
		}
		return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			FMenuBuilder MenuBuilder(true, CommandContext->GetCommands());
			CommandContext->AddMenuItems(MenuBuilder);

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			return FReply::Handled().ReleaseMouseCapture();
		}
		return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

private:
	void ScriptViewModelsChanged()
	{
		if (ScriptSelector.IsValid())
		{
			ScriptSelector->RefreshItemsAndDefaultCategories(ViewModel->GetScriptViewModels(), ViewModel->GetAvailableUsages());
		}
	}

	void ActiveScriptChanged()
	{
		if (bIsUpdatingSelection == false)
		{
			TGuardValue<bool> UpdateGuard(bIsUpdatingSelection, true);
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScriptViewModel = ViewModel->GetActiveScriptViewModel();
			if (ActiveScriptViewModel.IsValid())
			{
				TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> SelectedItems;
				SelectedItems.Add(ActiveScriptViewModel.ToSharedRef());
				ScriptSelector->SetSelectedItems(SelectedItems);
			}
			else
			{
				ScriptSelector->ClearSelectedItems();
			}
		}
	}

	TArray<ENiagaraScriptUsage> OnGetCategoriesForItem(const TSharedRef<FNiagaraScratchPadScriptViewModel>& Item)
	{
		TArray<ENiagaraScriptUsage> Categories;
		Categories.Add(Item->GetScripts()[0].Script->GetUsage());
		return Categories;
	}

	bool OnCompareCategoriesForEquality(const ENiagaraScriptUsage& CategoryA, const ENiagaraScriptUsage& CategoryB) const
	{
		return CategoryA == CategoryB;
	}

	bool OnCompareCategoriesForSorting(const ENiagaraScriptUsage& CategoryA, const ENiagaraScriptUsage& CategoryB) const
	{
		return ((int32)CategoryA) < ((int32)CategoryB);
	}

	bool OnCompareItemsForEquality(const TSharedRef<FNiagaraScratchPadScriptViewModel>& ItemA, const TSharedRef<FNiagaraScratchPadScriptViewModel>& ItemB) const
	{
		return ItemA == ItemB;
	}

	bool OnCompareItemsForSorting(const TSharedRef<FNiagaraScratchPadScriptViewModel>& ItemA, const TSharedRef<FNiagaraScratchPadScriptViewModel>& ItemB) const
	{
		return ItemA->GetDisplayName().CompareTo(ItemB->GetDisplayName()) < 0;
	}

	bool OnDoesItemMatchFilterText(const FText& FilterText, const TSharedRef<FNiagaraScratchPadScriptViewModel>& Item)
	{
		return Item->GetDisplayName().ToString().Find(FilterText.ToString(), ESearchCase::IgnoreCase) != INDEX_NONE;
	}

	TSharedRef<SWidget> OnGenerateWidgetForCategory(const ENiagaraScriptUsage& Category)
	{
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(3, 0, 0, 0)
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.GroupText")
			.Text(ViewModel->GetDisplayNameForUsage(Category))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 3.0f, 2.0f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraScratchPadScriptSelector::ScriptSelectorAddButtonClicked, Category)
			.ContentPadding(FMargin(3.0f, 2.0f, 2.0f, 2.0f))
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	TSharedRef<SWidget> OnGenerateWidgetForItem(const TSharedRef<FNiagaraScratchPadScriptViewModel>& Item)
	{
		return SNew(SNiagaraScratchPadScriptRow, ViewModel, Item, CommandContext)
			.IsSelected(this, &SNiagaraScratchPadScriptSelector::GetItemIsSelected, TWeakPtr<FNiagaraScratchPadScriptViewModel>(Item));
	}

	void OnScriptActivated(const TSharedRef<FNiagaraScratchPadScriptViewModel>& ActivatedScript)
	{
		if (bIsUpdatingSelection == false)
		{
			TGuardValue<bool> UpdateGuard(bIsUpdatingSelection, true);
			ViewModel->SetActiveScriptViewModel(ActivatedScript);
			ViewModel->OpenEditorForActive();
		}
	}

	FReply OnScriptDragged(const TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>>& DraggedItems, const FPointerEvent& MouseEvent) const
	{
		if (DraggedItems.Num() == 1)
		{
			TSharedPtr< FNiagaraScratchPadScriptViewModel> ScriptViewModel = DraggedItems[0];
			if (ScriptViewModel.IsValid())
			{
				TSharedRef<FNiagaraScriptDragOperation> DragDropOp = MakeShared<FNiagaraScriptDragOperation>(ScriptViewModel->GetOriginalScript(), ScriptViewModel->GetOriginalScript()->VersionToOpenInEditor, ScriptViewModel->GetDisplayName());
				DragDropOp->CurrentHoverText = ScriptViewModel->GetDisplayName();
				DragDropOp->CurrentIconBrush = FNiagaraEditorStyle::Get().GetBrush("Tab.ScratchPad");
				DragDropOp->SetupDefaults();
				DragDropOp->Construct();
				return FReply::Handled().BeginDragDrop(DragDropOp);
			}
		}
		return FReply::Unhandled();
	}


	void OnSelectionChanged()
	{
		if (bIsUpdatingSelection == false)
		{
			TGuardValue<bool> UpdateGuard(bIsUpdatingSelection, true);
			TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> SelectedScripts = ScriptSelector->GetSelectedItems();
			if (SelectedScripts.Num() == 0)
			{
				ViewModel->ResetActiveScriptViewModel();
			}
			else if (SelectedScripts.Num())
			{
				ViewModel->SetActiveScriptViewModel(SelectedScripts[0]);
			}
		}
	}

	FReply ScriptSelectorAddButtonClicked(ENiagaraScriptUsage Usage)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("MakeScratchpad", "Make new scratchpad"));
		TSharedPtr<FNiagaraScratchPadScriptViewModel> NewScriptViewModel = ViewModel->CreateNewScript(Usage, ENiagaraScriptUsage::ParticleUpdateScript, FNiagaraTypeDefinition());
		if (NewScriptViewModel.IsValid())
		{
			ViewModel->SetActiveScriptViewModel(NewScriptViewModel.ToSharedRef());
			NewScriptViewModel->SetIsPendingRename(true);
		}
		return FReply::Handled();
	}

	bool GetItemIsSelected(TWeakPtr<FNiagaraScratchPadScriptViewModel> ItemWeak) const
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> Item = ItemWeak.Pin();
		return Item.IsValid() && ViewModel->GetActiveScriptViewModel() == Item;
	}

private:
	TSharedPtr<SNiagaraScriptViewModelSelector> ScriptSelector;
	UNiagaraScratchPadViewModel* ViewModel;
	TSharedPtr<FNiagaraScratchPadCommandContext> CommandContext;
	bool bIsUpdatingSelection;
};


class SNiagaraScratchPadScriptEditorList : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadScriptEditorList) {}

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraScratchPadViewModel* InViewModel)
	{
		ViewModel = InViewModel;
		ViewModel->OnScriptViewModelsChanged().AddSP(this, &SNiagaraScratchPadScriptEditorList::ScriptViewModelsChanged);
		ViewModel->OnEditScriptViewModelsChanged().AddSP(this, &SNiagaraScratchPadScriptEditorList::EditScriptViewModelsChanged);
		ViewModel->OnActiveScriptChanged().AddSP(this, &SNiagaraScratchPadScriptEditorList::ActiveScriptChanged);
		bIsUpdatingSelection = false;

		ChildSlot
		[
			SAssignNew(ContentBorder, SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.DarkGroupBorder"))
		];
		UpdateContentFromEditScriptViewModels();
	}

private:
	void ScriptViewModelsChanged()
	{
		ScriptViewModelWidgetPairs.RemoveAll([](const FScriptViewModelWidgetPair& ScriptViewModelWidgetPair)
		{
			return ScriptViewModelWidgetPair.ViewModel.IsValid() == false || ScriptViewModelWidgetPair.Widget.IsValid() == false;
		});
	}

	void EditScriptViewModelsChanged()
	{
		UpdateContentFromEditScriptViewModels();
	}

	TSharedRef<SWidget> FindScriptEditor(TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel)
	{
		FScriptViewModelWidgetPair* ExistingPair = ScriptViewModelWidgetPairs.FindByPredicate([ScriptViewModel](FScriptViewModelWidgetPair& ScriptViewModelWidgetPair)
		{ 
			return ScriptViewModelWidgetPair.ViewModel == ScriptViewModel && ScriptViewModelWidgetPair.Widget.IsValid();
		});

		if (ExistingPair != nullptr)
		{
			return ExistingPair->Widget.ToSharedRef();
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}

	void UpdateContentFromEditScriptViewModels()
	{
		TSharedPtr<SWidget> NewContent;
		if (ViewModel->GetEditScriptViewModels().Num() == 0)
		{
			NewContent = SNew(SBox)
				.HAlign(HAlign_Center)
				.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoScriptToEdit", "No script selected"))
				];
			ScriptEditorList.Reset();
		}
		else if(ViewModel->GetEditScriptViewModels().Num() == 1)
		{
			NewContent = FindScriptEditor(ViewModel->GetEditScriptViewModels()[0]);
			ScriptEditorList.Reset();
		}
		else 
		{
			if (ScriptEditorList.IsValid())
			{
				ScriptEditorList->RequestListRefresh();
			}
			else 
			{
				ScriptEditorList = SNew(SListView<TSharedRef<FNiagaraScratchPadScriptViewModel>>)
					.ListItemsSource(&ViewModel->GetEditScriptViewModels())
					.OnGenerateRow(this, &SNiagaraScratchPadScriptEditorList::OnGenerateScriptEditorRow)
					.OnSelectionChanged(this, &SNiagaraScratchPadScriptEditorList::OnSelectionChanged);
			}
			NewContent = ScriptEditorList;
		}

		ContentBorder->SetContent(NewContent.ToSharedRef());
	}

	void ActiveScriptChanged()
	{
		if (ScriptEditorList.IsValid() && bIsUpdatingSelection == false)
		{
			TGuardValue<bool> UpdateGuard(bIsUpdatingSelection, true);
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScriptViewModel = ViewModel->GetActiveScriptViewModel();
			if (ActiveScriptViewModel.IsValid())
			{
				ScriptEditorList->SetSelection(ActiveScriptViewModel.ToSharedRef());
			}
			else
			{
				ScriptEditorList->ClearSelection();
			}
		}
	}

	TSharedRef<ITableRow> OnGenerateScriptEditorRow(TSharedRef<FNiagaraScratchPadScriptViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STableRow<TSharedRef<FNiagaraScratchPadScriptViewModel>>, OwnerTable)
		[
			SNew(SVerticalResizeBox)
			.ContentHeight(Item, &FNiagaraScratchPadScriptViewModel::GetEditorHeight)
			.ContentHeightChanged(Item, &FNiagaraScratchPadScriptViewModel::SetEditorHeight)
			[
				FindScriptEditor(Item)
			]
		];
	}

	void OnSelectionChanged(TSharedPtr<FNiagaraScratchPadScriptViewModel> InNewSelection, ESelectInfo::Type SelectInfo)
	{
		if (bIsUpdatingSelection == false)
		{
			TGuardValue<bool> UpdateGuard(bIsUpdatingSelection, true);
			TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> SelectedScripts;
			ScriptEditorList->GetSelectedItems(SelectedScripts);
			if (SelectedScripts.Num() == 0)
			{
				ViewModel->ResetActiveScriptViewModel();
			}
			else if (SelectedScripts.Num())
			{
				ViewModel->SetActiveScriptViewModel(SelectedScripts[0]);
			}
		}
	}

private:
	struct FScriptViewModelWidgetPair
	{
		TWeakPtr<FNiagaraScratchPadScriptViewModel> ViewModel;
		TSharedPtr<SWidget> Widget;
	};

	UNiagaraScratchPadViewModel* ViewModel;
	TSharedPtr<SBorder> ContentBorder;
	TSharedPtr<SListView<TSharedRef<FNiagaraScratchPadScriptViewModel>>> ScriptEditorList;
	TArray<FScriptViewModelWidgetPair> ScriptViewModelWidgetPairs;
	bool bIsUpdatingSelection;
};

class SNiagaraScratchPadParameterPanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadParameterPanel) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraScratchPadViewModel* InViewModel)
	{
		ViewModel = InViewModel;
		ViewModel->OnActiveScriptChanged().AddSP(this, &SNiagaraScratchPadParameterPanel::ActiveScriptChanged);
		bool bForce = true;
		UpdateContent(bForce);
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel = ScriptViewModelWeak.Pin();
		if (ScriptViewModel.IsValid() && ScriptViewModel->GetParameterPanelCommands()->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

private:
	void UpdateContent(bool bForce)
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> OldScriptViewModel = ScriptViewModelWeak.Pin();
		TSharedPtr<FNiagaraScratchPadScriptViewModel> NewScriptViewModel = ViewModel->GetActiveScriptViewModel();
		if (NewScriptViewModel != OldScriptViewModel || bForce)
		{
			ScriptViewModelWeak = NewScriptViewModel;
			if (NewScriptViewModel.IsValid())
			{
				TSharedPtr<SWidget> ParameterPanelWidget;
				if (NewScriptViewModel->GetParameterPanelViewModel().IsValid())
				{
					ParameterPanelWidget = SNew(SNiagaraParameterPanel, NewScriptViewModel->GetParameterPanelViewModel(), NewScriptViewModel->GetParameterPanelCommands());
				}
				else
				{
					TSharedRef<FNiagaraObjectSelection> ScriptSelection = MakeShared<FNiagaraObjectSelection>();
					const FVersionedNiagaraScript& EditScript = NewScriptViewModel->GetEditScript();
					ScriptSelection->SetSelectedObject(EditScript.Script, &EditScript.Version);
					TArray<TSharedRef<FNiagaraObjectSelection>> ParameterSelections;
					ParameterSelections.Add(ScriptSelection);
					ParameterSelections.Add(NewScriptViewModel->GetVariableSelection());
				}
				ChildSlot
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f, 2.0f, 2.0f, 5.0f)
					[
						SNew(STextBlock)
						.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.ScratchPad.SubSectionHeaderText")
						.Text(this, &SNiagaraScratchPadParameterPanel::GetSubHeaderText)
					]
					+ SVerticalBox::Slot()
					[
						ParameterPanelWidget.ToSharedRef()
					]
				];
			}
			else
			{
				ChildSlot
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoScriptForParameters", "No script selected"))
					]
				];
			}
		}
	}

	void ActiveScriptChanged()
	{
		bool bForce = false;
		UpdateContent(bForce);
	}

	FText GetSubHeaderText() const
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel = ScriptViewModelWeak.Pin();
		return ScriptViewModel.IsValid() ? ScriptViewModel->GetDisplayName() : FText();
	}

private:
	UNiagaraScratchPadViewModel* ViewModel;
	TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak;
};

class SNiagaraScratchPadSectionBox : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadSectionBox) {}
		SLATE_ATTRIBUTE(FText, HeaderText)
		SLATE_DEFAULT_SLOT(FArguments, Content);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(1.0f, 5.0f, 0.0f, 4.0f)
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.GroupText")
				.Text(InArgs._HeaderText)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+ SVerticalBox::Slot()
			.Padding(5.0f, 0.0f, 3.0f, 5.0f)
			[
				InArgs._Content.Widget
			]
		];
	}
};

void SNiagaraScratchPadScriptManager::Construct(const FArguments& InArgs, UNiagaraScratchPadViewModel* InViewModel)
{
	ViewModel = InViewModel;
	ViewModel->GetObjectSelection()->OnSelectedObjectsChanged().AddSP(this, &SNiagaraScratchPadScriptManager::ObjectSelectionChanged);

	CommandContext = MakeShared<FNiagaraScratchPadCommandContext>(InViewModel);

	ChildSlot
	[
		SNew(SDynamicLayoutBox)
		.GenerateNamedWidget_Lambda([this](FName InWidgetName)
		{
			if (InWidgetName == ScriptSelectorName)
			{
				return ConstructScriptSelector();
			}
			else if (InWidgetName == ScriptParameterPanelName)
			{
				return ConstructParameterPanel();
			}
			else if (InWidgetName == SelectionEditorName)
			{
				return ConstructSelectionEditor();
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		})
		.GenerateNamedLayout_Lambda([this](FName InLayoutName, const SDynamicLayoutBox::FNamedWidgetProvider& InNamedWidgetProvider)
		{
				TSharedPtr<SWidget> Layout =
					SNew(SSplitter)
					.Style(FAppStyle::Get(), "SplitterDark")
					.Orientation(Orient_Vertical)
					.PhysicalSplitterHandleSize(4.0f)
					.HitDetectionSplitterHandleSize(6.0f)
					+ SSplitter::Slot()
					.Value(0.3f)
					[
						InNamedWidgetProvider.GetNamedWidget(ScriptSelectorName)
					];
			
			return Layout.ToSharedRef();
		})
		.ChooseLayout_Lambda([this]() 
		{ 
				return NarrowLayoutName;
		})
	];
}

void SNiagaraScratchPadScriptManager::ObjectSelectionChanged()
{
	int32 SelectionCount = ViewModel->GetObjectSelection()->GetSelectedObjects().Num();
	if (SelectionCount == 0)
	{
		ObjectSelectionSubHeaderText = FText();
	}
	else if (SelectionCount == 1)
	{
		UObject* SelectedObject = ViewModel->GetObjectSelection()->GetFirstSelectedObject();
		if (SelectedObject->IsA<UEdGraphNode>())
		{
			UEdGraphNode* SelectedGraphNode = CastChecked<UEdGraphNode>(SelectedObject);
			ObjectSelectionSubHeaderText = SelectedGraphNode->GetNodeTitle(ENodeTitleType::ListView);
		}
		else if (SelectedObject->IsA<UNiagaraScript>())
		{
			UNiagaraScript* SelectedScript = CastChecked<UNiagaraScript>(SelectedObject);
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = ViewModel->GetViewModelForEditScript(SelectedScript);
			if (ScratchPadScriptViewModel.IsValid())
			{
				ObjectSelectionSubHeaderText = ScratchPadScriptViewModel->GetDisplayName();
			}
			else
			{
				ObjectSelectionSubHeaderText = FText::FromString(SelectedScript->GetName());
			}
		}
		else
		{
			ObjectSelectionSubHeaderText = FText::FromString(SelectedObject->GetName());
		}
	}
	else
	{
		ObjectSelectionSubHeaderText = FText::Format(LOCTEXT("MultipleSelectionFormat", "{0} Objects Selected..."),
			FText::AsNumber(SelectionCount));
	}
}

TSharedRef<SWidget> SNiagaraScratchPadScriptManager::ConstructScriptSelector()
{
	return SNew(SNiagaraScratchPadSectionBox)
	.HeaderText(LOCTEXT("ScriptManager", "Scratch Script Manager"))
	[
		SNew(SNiagaraScratchPadScriptSelector, ViewModel.Get(), CommandContext)
	];
}

TSharedRef<SWidget> SNiagaraScratchPadScriptManager::ConstructParameterPanel()
{
	return SNew(SNiagaraScratchPadSectionBox)
	.HeaderText(LOCTEXT("ScratchScriptParameters", "Scratch Script Parameters"))
	[
		SNew(SNiagaraScratchPadParameterPanel, ViewModel.Get())
	];
}

TSharedRef<SWidget> SNiagaraScratchPadScriptManager::ConstructSelectionEditor()
{
	return SNew(SNiagaraScratchPadSectionBox)
	.HeaderText(LOCTEXT("ScratchPadSelection", "Scratch Pad Selection"))
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f, 2.0f, 2.0f, 5.0f)
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.ScratchPad.SubSectionHeaderText")
			.Visibility(this, &SNiagaraScratchPadScriptManager::GetObjectSelectionSubHeaderTextVisibility)
			.Text(this, &SNiagaraScratchPadScriptManager::GetObjectSelectionSubHeaderText)
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.AutoHeight()
		.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(this, &SNiagaraScratchPadScriptManager::GetObjectSelectionNoSelectionTextVisibility)
			.Text(LOCTEXT("NoObjectSelection", "No object selected"))
		]
		+ SVerticalBox::Slot()
		[
			SNew(SNiagaraSelectedObjectsDetails, ViewModel->GetObjectSelection())
		]
	];
}

EVisibility SNiagaraScratchPadScriptManager::GetObjectSelectionSubHeaderTextVisibility() const
{
	return ViewModel->GetObjectSelection()->GetSelectedObjects().Num() != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SNiagaraScratchPadScriptManager::GetObjectSelectionSubHeaderText() const
{
	return ObjectSelectionSubHeaderText;
}

EVisibility SNiagaraScratchPadScriptManager::GetObjectSelectionNoSelectionTextVisibility() const
{
	return ViewModel->GetObjectSelection()->GetSelectedObjects().Num() == 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
