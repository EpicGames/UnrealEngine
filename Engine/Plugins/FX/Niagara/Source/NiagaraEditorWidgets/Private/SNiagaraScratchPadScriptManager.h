// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Widgets/SItemSelector.h"
#include "Widgets/SCompoundWidget.h"

class UNiagaraScratchPadViewModel;
class FNiagaraScratchPadScriptViewModel;
class FNiagaraScratchPadCommandContext;

typedef SItemSelector<ENiagaraScriptUsage, TSharedRef<FNiagaraScratchPadScriptViewModel>> SNiagaraScriptViewModelSelector;

class SNiagaraScratchPadScriptManager : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadScriptManager) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraScratchPadViewModel* InViewModel);

private:
	void ObjectSelectionChanged();

	TSharedRef<SWidget> ConstructScriptSelector();

	TSharedRef<SWidget> ConstructParameterPanel();

	TSharedRef<SWidget> ConstructScriptEditor();

	TSharedRef<SWidget> ConstructSelectionEditor();

	EVisibility GetObjectSelectionSubHeaderTextVisibility() const;

	FText GetObjectSelectionSubHeaderText() const;

	EVisibility GetObjectSelectionNoSelectionTextVisibility() const;
	static FName ScriptSelectorName;
	static FName ScriptParameterPanelName;
	static FName ScriptEditorName;
	static FName SelectionEditorName;
	static FName WideLayoutName;
	static FName NarrowLayoutName;
private:
	FText ObjectSelectionSubHeaderText;

	TWeakObjectPtr<UNiagaraScratchPadViewModel> ViewModel;

	TSharedPtr<FNiagaraScratchPadCommandContext> CommandContext;

};