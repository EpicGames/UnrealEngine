// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStack.h"

#include "NiagaraEditorModule.h"
#include "NiagaraEditorCommands.h"
#include "Styling/AppStyle.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEmitterHandle.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackModuleItemOutputCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItemLinkedInputCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItemOutput.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h" 
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/Stack/NiagaraStackItemFooter.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackSystemSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"
#include "Framework/Commands/GenericCommands.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "NiagaraEmitter.h"
#include "IDetailTreeNode.h"
#include "Stack/NiagaraStackPropertyRowUtilities.h"
#include "Stack/SNiagaraStackFunctionInputName.h"
#include "Stack/SNiagaraStackValueCollection.h"
#include "Stack/SNiagaraStackFunctionInputValue.h"
#include "Stack/SNiagaraStackInlineDynamicInput.h"
#include "Stack/SNiagaraStackItem.h"
#include "Stack/SNiagaraStackItemFooter.h"
#include "Stack/SNiagaraStackItemGroup.h"
#include "Stack/SNiagaraStackModuleItem.h"
#include "Stack/SNiagaraStackTableRow.h"
#include "Stack/SNiagaraStackIssueIcon.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Stack/SNiagaraStackErrorItem.h"
#include "Widgets/Layout/SWrapBox.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "NiagaraStackCommandContext.h"
#include "NiagaraEditorUtilities.h"
#include "SNiagaraStackNote.h"
#include "Framework/Commands/UICommandList.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Styling/StyleColors.h"
#include "SResetToDefaultPropertyEditor.h"
#include "Stack/SNiagaraStackScriptHierarchyRoot.h"
#include "Widgets/Images/SLayeredImage.h"

#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackNote.h"

#define LOCTEXT_NAMESPACE "NiagaraStack"

class SNiagaraStackHeaderButtons : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackHeaderButtons) {}
		SLATE_ATTRIBUTE(EVisibility, IssueIconVisibility);
		SLATE_EVENT(FSimpleDelegate, OnCycleThroughIssues);
		SLATE_ARGUMENT(TSharedPtr<SNiagaraStack>, ParentStack)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackEntry* InRootEntry, UNiagaraStackViewModel* InStackViewModel)
	{
		StackViewModel = InStackViewModel;
		OnCycleThroughIssues = InArgs._OnCycleThroughIssues;
		ParentStackPtr = InArgs._ParentStack;
		TopLevelViewModel = StackViewModel->GetTopLevelViewModelForEntry(*InRootEntry);
		ChildSlot
		[
			SNew(SHorizontalBox)
			// Issue Icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(0, 0, 2, 0)
			[
				SNew(SNiagaraStackIssueIcon, StackViewModel, InRootEntry)
				.Visibility(InArgs._IssueIconVisibility)
				.OnClicked(this, &SNiagaraStackHeaderButtons::OnIssueIconClicked)
			]
			// Options button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(0, 0, 2, 0)
			[
				SAssignNew(SettingsAnchor, SMenuAnchor)
				.Placement(MenuPlacement_MenuLeft)
				.OnGetMenuContent(this, &SNiagaraStackHeaderButtons::OnGetContent)
				[
					SNew(SButton)
					.ContentPadding(0)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraStackHeaderButtons::OpenSubmenu)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Fill)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
					]
				]
			]
		];
	}

private:
	FReply OnIssueIconClicked() const
	{
		StackViewModel->OnCycleThroughIssues(TopLevelViewModel);
		OnCycleThroughIssues.ExecuteIfBound();
		return FReply::Handled();
	}

	FReply OpenSubmenu()
	{
		SettingsAnchor->SetIsOpen(!SettingsAnchor->IsOpen());
		return FReply::Handled();
	}

	TSharedRef<SWidget> OnGetContent() const
	{
		if (ParentStackPtr.IsValid())
		{
			return ParentStackPtr.Pin()->GenerateStackMenu(TopLevelViewModel).ToSharedRef();
		}
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModel;
	FSimpleDelegate OnCycleThroughIssues;
	UNiagaraStackViewModel* StackViewModel;
	TWeakPtr<SNiagaraStack> ParentStackPtr;
	TSharedPtr<SMenuAnchor> SettingsAnchor;
};

class SNiagaraStackEmitterHeader : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackEmitterHeader) {}
		SLATE_ATTRIBUTE(EVisibility, IssueIconVisibility);
		SLATE_EVENT(FSimpleDelegate, OnCycleThroughIssues);
		SLATE_ARGUMENT(TSharedPtr<SNiagaraStack>, ParentStack)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel, UNiagaraStackEntry* InRootEntry, UNiagaraStackViewModel* InStackViewModel)
	{
		EmitterHandleViewModel = InEmitterHandleViewModel;
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FStyleColors::Panel)
			[
				SNew(SVerticalBox)

				//~ Enable check box, view source emitter button, and external header controls.
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					//~ Enabled
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.ToolTipText(LOCTEXT("EnabledToolTip", "Toggles whether this emitter is enabled. Disabled emitters don't simulate or render."))
						.IsChecked(EmitterHandleViewModel.ToSharedRef(), &FNiagaraEmitterHandleViewModel::GetIsEnabledCheckState)
						.OnCheckStateChanged(EmitterHandleViewModel.ToSharedRef(), &FNiagaraEmitterHandleViewModel::OnIsEnabledCheckStateChanged)
					]
					+ SHorizontalBox::Slot()
					.Padding(2)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						// Name and Source Emitter Name
						SNew(SWrapBox)
						.Clipping(EWidgetClipping::ClipToBoundsAlways) 
						.UseAllottedSize(true)
						+ SWrapBox::Slot()
						[
				
							SAssignNew(EmitterNameTextBlock, SInlineEditableTextBlock)
							.ToolTipText(this, &SNiagaraStackEmitterHeader::GetEmitterNameToolTip)
							.Style(FAppStyle::Get(), "DetailsView.NameTextBlockStyle")
							.Clipping(EWidgetClipping::ClipToBoundsAlways)
							.Text(EmitterHandleViewModel.ToSharedRef(), &FNiagaraEmitterHandleViewModel::GetNameText)
							.OnTextCommitted(EmitterHandleViewModel.ToSharedRef(), &FNiagaraEmitterHandleViewModel::OnNameTextComitted)
							.OnVerifyTextChanged(EmitterHandleViewModel.ToSharedRef(), &FNiagaraEmitterHandleViewModel::VerifyNameTextChanged)
							.IsReadOnly(EmitterHandleViewModel->CanRenameEmitter() == false)
						]
						+ SWrapBox::Slot()
						.Padding(4, 0, 0, 0)
						[
							SNew(STextBlock)
							.ToolTipText(this, &SNiagaraStackEmitterHeader::GetEmitterNameToolTip)
							.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.HeadingTextBlockSubdued")
							.Clipping(EWidgetClipping::ClipToBoundsAlways)
							.Text(EmitterHandleViewModel->GetEmitterViewModel(), &FNiagaraEmitterViewModel::GetParentNameText)
							.Visibility(this, &SNiagaraStackEmitterHeader::GetSourceEmitterNameVisibility) 
						]
					]

					+ SHorizontalBox::Slot()
					[
						SNew(SNiagaraStackHeaderButtons, InRootEntry, InStackViewModel)
						.IssueIconVisibility(InArgs._IssueIconVisibility)
						.OnCycleThroughIssues(InArgs._OnCycleThroughIssues)
						.ParentStack(InArgs._ParentStack)
					]
				]

				//~ Stats
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(2)
				[
					SNew(STextBlock)
					.Text(EmitterHandleViewModel->GetEmitterViewModel(), &FNiagaraEmitterViewModel::GetStatsText)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		if (EmitterHandleViewModel->GetIsRenamePending())
		{
			EmitterHandleViewModel->SetIsRenamePending(false);
			EmitterNameTextBlock->EnterEditingMode();
		}
	}

private:
	FText GetEmitterNameToolTip() const
	{
		if (EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter())
		{
			// We are looking at this Emitter in a System Asset and it has a valid parent Emitter
			return FText::Format(LOCTEXT("EmitterNameAndPath", "{0}\nParent: {1}"), EmitterHandleViewModel->GetNameText(), EmitterHandleViewModel->GetEmitterViewModel()->GetParentPathNameText());
		}
		else
		{
			// We are looking at this Emitter in an Emitter Asset or we are looking at this Emitter in a System Asset and it does not have a valid parent Emitter
			return EmitterHandleViewModel->GetNameText();
		}
	}

	EVisibility GetSourceEmitterNameVisibility() const
	{
		bool bIsRenamed = false;
		if (EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter())
		{
			const FText CurrentNameText = EmitterHandleViewModel->GetNameText();
			const FText ParentNameText = EmitterHandleViewModel->GetEmitterViewModel()->GetParentNameText();
			bIsRenamed = CurrentNameText.EqualTo(ParentNameText) == false;
		}
		return bIsRenamed ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetOpenSourceEmitterVisibility() const
	{
		return EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter() ? EVisibility::Visible : EVisibility::Collapsed;
	}

private:
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;
	TSharedPtr<SInlineEditableTextBlock> EmitterNameTextBlock;
};

const float SpacerHeight = 6;

void SNiagaraStack::Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel)
{
	StackViewModel = InStackViewModel;
	StackViewModel->OnChangeSearchTextExternal().BindSP(this, &SNiagaraStack::UpdateSearchTextFromExternal);
	StackViewModel->OnStackEntrySelectionRequested().BindSP(this, &SNiagaraStack::SelectStackEntry);
	StackViewModel->OnStructureChanged().AddSP(this, &SNiagaraStack::StackStructureChanged);
	StackViewModel->OnExpansionChanged().AddSP(this, &SNiagaraStack::OnStackExpansionChanged);
	StackViewModel->OnSearchCompleted().AddSP(this, &SNiagaraStack::OnStackSearchComplete);
	NameColumnWidth = .3f;
	ContentColumnWidth = .7f;
	StackCommandContext = MakeShared<FNiagaraStackCommandContext>();
	bSynchronizeExpansionPending = true;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1, 1, 1, 4)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FStyleColors::Panel)
			.Padding(5)
			[
				ConstructHeaderWidget()
			]
		]
		+ SVerticalBox::Slot()
		.Padding(1)
		[
			SAssignNew(StackTree, STreeView<UNiagaraStackEntry*>)
			.OnGenerateRow(this, &SNiagaraStack::OnGenerateRowForStackItem)
			.OnGetChildren(this, &SNiagaraStack::OnGetChildren)
			.TreeItemsSource(&StackViewModel->GetRootEntryAsArray())
			.OnTreeViewScrolled(this, &SNiagaraStack::StackTreeScrolled)
			.OnSelectionChanged(this, &SNiagaraStack::StackTreeSelectionChanged)
			.SelectionMode(ESelectionMode::Type::Multi)
			.OnItemToString_Debug_Static(&FNiagaraStackEditorWidgetsUtilities::StackEntryToStringForListDebug)
		]
	];

	StackTree->SetScrollOffset(StackViewModel->GetLastScrollPosition());

	SynchronizeTreeExpansion();
}

void SNiagaraStack::SynchronizeTreeExpansion()
{
	bSynchronizeExpansionPending = false;
	TArray<UNiagaraStackEntry*> EntriesToProcess(StackViewModel->GetRootEntryAsArray());
	while (EntriesToProcess.Num() > 0)
	{
		UNiagaraStackEntry* EntryToProcess = EntriesToProcess[0];
		EntriesToProcess.RemoveAtSwap(0);

		if (EntryToProcess->GetIsExpanded())
		{
			StackTree->SetItemExpansion(EntryToProcess, true);
			EntryToProcess->GetFilteredChildren(EntriesToProcess);
		}
		else
		{
			StackTree->SetItemExpansion(EntryToProcess, false);
		}
	}
}

TSharedRef<SWidget> SNiagaraStack::ConstructHeaderWidget()
{
	TSharedPtr<SLayeredImage> ViewOptionsImage;
	
	TSharedRef<SWidget> Result = SNew(SVerticalBox)
		//~ Top level object list view
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SAssignNew(HeaderList, SListView<TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel>>)
			.ListItemsSource(&StackViewModel->GetTopLevelViewModels())
			.SelectionMode(ESelectionMode::None)
			.OnGenerateRow(this, &SNiagaraStack::OnGenerateRowForTopLevelObject)
		]
		
		//~ Search, view options
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(2, 4, 2, 4)
		[
			// Search box
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("StackSearchBoxHint", "Search the stack"))
				.SearchResultData(this, &SNiagaraStack::GetSearchResultData)
				.IsSearching(this, &SNiagaraStack::GetIsSearching)
				.OnTextChanged(this, &SNiagaraStack::OnSearchTextChanged)
				.DelayChangeNotificationsWhileTyping(true)
				.OnTextCommitted(this, &SNiagaraStack::OnSearchBoxTextCommitted)
				.OnSearch(this, &SNiagaraStack::OnSearchBoxSearch)
			]
			// View options
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 0, 0, 0)
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton") // Use the tool bar item style for this button
				.HasDownArrow(false)
				.ToolTipText(LOCTEXT("ViewOptionsToolTip", "View Options"))
				.OnGetMenuContent(this, &SNiagaraStack::GetViewOptionsMenu)
				.ContentPadding(1)
				.MenuPlacement(MenuPlacement_BelowRightAnchor)
				.ButtonContent()
				[
					SAssignNew(ViewOptionsImage, SLayeredImage)
					.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];

	ViewOptionsImage->AddLayer(TAttribute<const FSlateBrush*>(this, &SNiagaraStack::GetViewOptionsBadgeIcon));

	return Result;
}

void SNiagaraStack::UpdateSearchTextFromExternal(FText NewSearchText)
{
	SearchBox->SetText(NewSearchText);
}

void SNiagaraStack::OnSearchTextChanged(const FText& SearchText)
{
	StackViewModel->OnSearchTextChanged(SearchText);
}

static void ExpandAllInPath(const TArray<UNiagaraStackEntry*>& EntryPath)
{
	for (UNiagaraStackEntry* Entry : EntryPath)
	{
		if (!Entry->IsA<UNiagaraStackRoot>())
		{
			Entry->SetIsExpanded(true);
		}
	}
}

FReply SNiagaraStack::ScrollToNextMatch()
{
	int NextMatchIndex = StackViewModel->GetCurrentFocusedEntryIndex() + 1;
	TArray<UNiagaraStackViewModel::FSearchResult> CurrentSearchResults = StackViewModel->GetCurrentSearchResults();
	if (CurrentSearchResults.Num() != 0)
	{
		if (NextMatchIndex >= CurrentSearchResults.Num())
		{
			NextMatchIndex = 0;
		}

		ExpandAllInPath(CurrentSearchResults[NextMatchIndex].EntryPath);
	}

	AddSearchScrollOffset(1);
	return FReply::Handled();
}

FReply SNiagaraStack::ScrollToPreviousMatch()
{
	const int PreviousMatchIndex = StackViewModel->GetCurrentFocusedEntryIndex() - 1;
	TArray<UNiagaraStackViewModel::FSearchResult> CurrentSearchResults = StackViewModel->GetCurrentSearchResults();
	if (CurrentSearchResults.Num() != 0)
	{
		if (PreviousMatchIndex >= 0)
		{
			ExpandAllInPath(CurrentSearchResults[PreviousMatchIndex].EntryPath);
		}
		else
		{
			ExpandAllInPath(CurrentSearchResults.Last().EntryPath);
		}
	}

	// move current match to the previous one in the StackTree, wrap around
	AddSearchScrollOffset(-1);
	return FReply::Handled();
}

void SNiagaraStack::AddSearchScrollOffset(int NumberOfSteps)
{
	if (StackViewModel->IsSearching() || StackViewModel->GetCurrentSearchResults().Num() == 0 || NumberOfSteps == 0)
	{
		return;
	}

	StackViewModel->AddSearchScrollOffset(NumberOfSteps);
	ScrollFocusedEntryIntoView();
}

TOptional<SSearchBox::FSearchResultData> SNiagaraStack::GetSearchResultData() const
{
	if (StackViewModel->GetCurrentSearchText().IsEmpty())
	{
		return TOptional<SSearchBox::FSearchResultData>();
	}
	return TOptional<SSearchBox::FSearchResultData>({ StackViewModel->GetCurrentSearchResults().Num(), StackViewModel->GetCurrentFocusedEntryIndex() + 1 });
}

bool SNiagaraStack::GetIsSearching() const
{
	return StackViewModel->IsSearching();
}

bool SNiagaraStack::IsEntryFocusedInSearch(UNiagaraStackEntry* Entry) const
{
	if (StackViewModel && Entry && StackViewModel->GetCurrentFocusedEntry() == Entry)
	{
		return true;
	}
	return false;
}

void SNiagaraStack::SelectStackEntry(UNiagaraStackEntry* StackEntry, FNiagaraStackEntrySelectionConfig Config)
{
	if(StackViewModel && StackEntry->IsFinalized() == false)
	{
		if(Config.bScrollToAndExpand)
		{
			TArray<UNiagaraStackEntry*> StackEntryPath;
			StackViewModel->GetPathForEntry(StackEntry, StackEntryPath);
			ExpandAllInPath(StackEntryPath);
			
			StackTree->RequestScrollIntoView(StackEntry);
		}

		if(Config.bSelectRow)
		{
			StackTree->SetSelection(StackEntry, ESelectInfo::Type::Direct);
		}

		if(Config.bPulse)
		{
			
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([StackEntry](double CurrentTime, float DeltaTime)
			{
				StackEntry->RequestPulse();
				return EActiveTimerReturnType::Stop;
			}));
		}
	}
}

void SNiagaraStack::ShowEmitterInContentBrowser(TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelWeak)
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelWeak.Pin();
	if (EmitterHandleViewModel.IsValid())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		TArray<FAssetData> Assets;
		if (EmitterHandleViewModel->GetEmitterViewModel()->HasParentEmitter())
		{
			Assets.Add(FAssetData(EmitterHandleViewModel->GetEmitterViewModel()->GetParentEmitter().Emitter));
			ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
		}
	}
}

void CollapseEntriesRecursive(TArray<UNiagaraStackEntry*> Entries)
{
	for (UNiagaraStackEntry* Entry : Entries)
	{
		if (Entry->GetCanExpand())
		{
			Entry->SetIsExpanded(false);
		}
		
		TArray<UNiagaraStackEntry*> Children;
		Entry->GetUnfilteredChildren(Children);
		CollapseEntriesRecursive(Children);
	}
}

void SNiagaraStack::CollapseAll()
{
	CollapseEntriesRecursive(StackViewModel->GetRootEntryAsArray());
}

TSharedRef<SWidget> SNiagaraStack::GetViewOptionsMenu() const
{
	FMenuBuilder MenuBuilder(false, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowOnlyModifiedLabel", "Show Only Modified Properties"),
		LOCTEXT("ShowOnlyModifiedToolTip", "Displays only properties that have been changes from their default."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { StackViewModel->SetShowOnlyModified(!StackViewModel->GetShowOnlyModified()); }),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]() { return StackViewModel->GetShowOnlyModified() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
		NAME_None, EUserInterfaceActionType::Check);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowAllAdvancedLabel", "Show All Advanced"),
		LOCTEXT("ShowAllAdvancedToolTip", "Forces all advanced items to be showing in the stack."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { StackViewModel->SetShowAllAdvanced(!StackViewModel->GetShowAllAdvanced()); }),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]() { return StackViewModel->GetShowAllAdvanced() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
		NAME_None, EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowParameterReadsLabel", "Show Parameter Reads"),
		LOCTEXT("ShowParameterReadsToolTip", "Whether or not to show the parameters that a module reads from."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { StackViewModel->SetShowLinkedInputs(!StackViewModel->GetShowLinkedInputs()); }),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]() { return StackViewModel->GetShowLinkedInputs() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
		NAME_None, EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowParameterWritesLabel", "Show Parameter Writes"),
		LOCTEXT("ShowParameterWritesToolTip", "Whether or not to show parameters that a module writes to."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { StackViewModel->SetShowOutputs(!StackViewModel->GetShowOutputs()); }),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]() { return StackViewModel->GetShowOutputs() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
		NAME_None, EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowIssuesLabel", "Show Only Issues"),
		LOCTEXT("ShowIssuesToolTip", "Hides all modules except those that have unresolved issues. This action is only enabled when system and emitter nodes are selected in the graph, as it could potentially hide any selected modules."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { StackViewModel->SetShowOnlyIssues(!StackViewModel->GetShowOnlyIssues()); }),
			FCanExecuteAction::CreateLambda([this]()
			{
				if (UNiagaraStackEntry* RootEntry = StackViewModel->GetRootEntry())
				{
					if (UNiagaraSystemSelectionViewModel* SelectionViewModel = RootEntry->GetSystemViewModel()->GetSelectionViewModel())
					{
						// only allow the user to toggle this option when a system or emitter node is selected
						TArray<UNiagaraStackEntry*> SelectedEntries;
						SelectionViewModel->GetSelectedEntries(SelectedEntries);
						for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
						{
							if (!SelectedEntry->IsA<UNiagaraStackRoot>())
							{
								return false;
							}
						}
						return SelectedEntries.Num() > 0;
					}
				}
				return false;
			}),
			FGetActionCheckState::CreateLambda([this]() { return StackViewModel->GetShowOnlyIssues() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
		NAME_None, EUserInterfaceActionType::Check);

	return MenuBuilder.MakeWidget();
}

FReply SNiagaraStack::OnRowDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, UNiagaraStackEntry* InStackEntry)
{
	if (InStackEntry->CanDrag())
	{
		TArray<UNiagaraStackEntry*> DraggedEntries;
		DraggedEntries.Add(InStackEntry);
		return FReply::Handled().BeginDragDrop(FNiagaraStackEditorWidgetsUtilities::ConstructDragDropOperationForStackEntries(DraggedEntries));
	}
	return FReply::Unhandled();
}

void  SNiagaraStack::OnRowDragLeave(FDragDropEvent const& InDragDropEvent)
{
	FNiagaraStackEditorWidgetsUtilities::HandleDragLeave(InDragDropEvent);
}

TOptional<EItemDropZone> SNiagaraStack::OnRowCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry)
{
	return FNiagaraStackEditorWidgetsUtilities::RequestDropForStackEntry(InDragDropEvent, InDropZone, InTargetEntry, UNiagaraStackEntry::EDropOptions::None);
}

FReply SNiagaraStack::OnRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry)
{
	bool bHandled = FNiagaraStackEditorWidgetsUtilities::HandleDropForStackEntry(InDragDropEvent, InDropZone, InTargetEntry, UNiagaraStackEntry::EDropOptions::None);
	return bHandled ? FReply::Handled() : FReply::Unhandled();
}

void SNiagaraStack::OnStackSearchComplete(const FNiagaraStackSearchConfig& SearchConfig)
{
	ExpandSearchResults();

	if(SearchConfig.bScrollToResult)
	{
		ScrollFocusedEntryIntoView();
	}
}

void SNiagaraStack::ExpandSearchResults()
{
	for (auto SearchResult : StackViewModel->GetCurrentSearchResults())
	{
		ExpandAllInPath(SearchResult.EntryPath);
	}

	bSynchronizeExpansionPending = true;
}

void SNiagaraStack::OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter) // hasn't been autojumped yet or we hit enter
	{
		bool bIsShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		if(bIsShiftDown)
		{
			AddSearchScrollOffset(-1);
		}
		else
		{
			AddSearchScrollOffset(+1);
		}
	}
}

void SNiagaraStack::OnSearchBoxSearch(SSearchBox::SearchDirection Direction)
{
	if (Direction == SSearchBox::Next)
	{
		ScrollToNextMatch();
	}
	else if (Direction == SSearchBox::Previous)
	{
		ScrollToPreviousMatch();
	}
}

FSlateColor SNiagaraStack::GetTextColorForItem(UNiagaraStackEntry* Item) const
{
	if (IsEntryFocusedInSearch(Item))
	{
		return FSlateColor(FLinearColor(FColor::Orange));
	}
	return FSlateColor::UseForeground();
}

void SNiagaraStack::ScrollFocusedEntryIntoView()
{
	StackTree->RequestScrollIntoView(StackViewModel->GetCurrentFocusedEntry());
}

TSharedRef<ITableRow> SNiagaraStack::OnGenerateRowForStackItem(UNiagaraStackEntry* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SNiagaraStackTableRow> Container = ConstructContainerForItem(Item);
	FRowWidgets RowWidgets = ConstructNameAndValueWidgetsForItem(Item, Container);
	Container->SetNameAndValueContent(RowWidgets.NameWidget, RowWidgets.ValueWidget);
	return Container;
}

TSharedRef<ITableRow> SNiagaraStack::OnGenerateRowForTopLevelObject(TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SWidget> Content;
	if (Item->SystemViewModel.IsValid())
	{
		Content =
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FStyleColors::Panel)
			[
				SNew(SHorizontalBox)
				// System name
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "DetailsView.ConstantTextBlockStyle")
					.Text(Item->SystemViewModel.ToSharedRef(), &FNiagaraSystemViewModel::GetDisplayName)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					SNew(SNiagaraStackHeaderButtons, Item->RootEntry.Get(), StackViewModel)
					.IssueIconVisibility(this, &SNiagaraStack::GetIssueIconVisibility)
					.OnCycleThroughIssues(this, &SNiagaraStack::OnCycleThroughIssues)
					.ParentStack(SharedThis(this))
				]
			];
	}
	else if (Item->EmitterHandleViewModel.IsValid())
	{
		Content = SNew(SNiagaraStackEmitterHeader, Item->EmitterHandleViewModel.ToSharedRef(), Item->RootEntry.Get(), StackViewModel)
			.IssueIconVisibility(this, &SNiagaraStack::GetIssueIconVisibility)
			.OnCycleThroughIssues(this, &SNiagaraStack::OnCycleThroughIssues)
			.ParentStack(SharedThis(this));
	}

	return SNew(STableRow<TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel>>, OwnerTable)
		[
			Content.ToSharedRef()
		];
}

TSharedPtr<SWidget> SNiagaraStack::GenerateStackMenu(TWeakPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModelWeak)
{
	TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModel = TopLevelViewModelWeak.Pin();
	if (TopLevelViewModel.IsValid())
	{
		TSharedPtr<FUICommandList> GraphCommandList = TopLevelViewModel->RootEntry->GetSystemViewModel()->GetOverviewGraphViewModel()->GetCommands();
		FMenuBuilder MenuBuilder(true, GraphCommandList);

		FNiagaraEditorUtilities::AddEmitterContextMenuActions(MenuBuilder, TopLevelViewModel->EmitterHandleViewModel);

		{
			MenuBuilder.BeginSection("EmitterEditSection", LOCTEXT("Edit", "Edit"));

			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);

			MenuBuilder.EndSection();
		}
		MenuBuilder.BeginSection("StackActions", LOCTEXT("StackActions", "Stack Actions"));
		{
			if (StackViewModel->HasDismissedStackIssues())
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("UndismissIssues", "Undismiss All Stack Issues"),
					LOCTEXT("ShowAssetInContentBrowserToolTip", "Undismiss all issues that were previously dismissed for this stack, if any"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateUObject(StackViewModel, &UNiagaraStackViewModel::UndismissAllIssues)));
			}

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CollapseStack", "Collapse All"),
				LOCTEXT("CollapseStackToolTip", "Collapses every row in the stack."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStack::CollapseAll)));

			TSharedPtr<FUICommandInfo> CollapseToHeadersCommand = FNiagaraEditorModule::Get().Commands().CollapseStackToHeaders;
			MenuBuilder.AddMenuEntry(
				CollapseToHeadersCommand->GetLabel(),
				CollapseToHeadersCommand->GetDescription(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateUObject(StackViewModel, &UNiagaraStackViewModel::CollapseToHeaders)));
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}
	return SNullWidget::NullWidget;
}

TSharedRef<SNiagaraStackTableRow> SNiagaraStack::ConstructContainerForItem(UNiagaraStackEntry* Item)
{
	float LeftContentPadding = 0;
	float RightContentPadding = 0;
	float ItemContentVerticalPadding = 1;
	FMargin ContentPadding(LeftContentPadding, 0, RightContentPadding, 0);
	FSlateColor IndicatorColor = FLinearColor::Transparent;
	bool bIsCategoryIconHighlighted = false;
	bool bShowExecutionCategoryIcon = false;
	const FTableRowStyle* TableRowStyle = nullptr;
	switch (Item->GetStackRowStyle())
	{
	case UNiagaraStackEntry::EStackRowStyle::GroupHeader:
		ContentPadding = FMargin(LeftContentPadding, 3, 0, 3);
		bIsCategoryIconHighlighted = true;
		bShowExecutionCategoryIcon = true;
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.ItemHeader");
		break;
	case UNiagaraStackEntry::EStackRowStyle::ItemHeader:
		ContentPadding = FMargin(LeftContentPadding, 2, 2, 2);
		bShowExecutionCategoryIcon = true;
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.ItemHeader");
		break;
	case UNiagaraStackEntry::EStackRowStyle::ItemContent:
		ContentPadding = FMargin(LeftContentPadding, ItemContentVerticalPadding, RightContentPadding, ItemContentVerticalPadding);
		break;
	case UNiagaraStackEntry::EStackRowStyle::ItemContentAdvanced:
		ContentPadding = FMargin(LeftContentPadding, ItemContentVerticalPadding, RightContentPadding, ItemContentVerticalPadding);
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.ItemContentAdvanced");
		break;
	case UNiagaraStackEntry::EStackRowStyle::ItemContentNote:
		ContentPadding = FMargin(LeftContentPadding, ItemContentVerticalPadding, RightContentPadding, ItemContentVerticalPadding);
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.ItemContentNote");
		break;
	case UNiagaraStackEntry::EStackRowStyle::ItemCategory:
		ContentPadding = FMargin(LeftContentPadding, ItemContentVerticalPadding, RightContentPadding, ItemContentVerticalPadding);
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.ItemHeader");
		break;
	case UNiagaraStackEntry::EStackRowStyle::StackIssue:
		switch (Item->GetIssueSeverity())
		{
			case EStackIssueSeverity::Error:
				IndicatorColor = FStyleColors::Error;
				break;
			case EStackIssueSeverity::Warning:
				IndicatorColor = FStyleColors::Warning;
				break;
			case EStackIssueSeverity::Info:
				IndicatorColor = FStyleColors::Foreground;
				break;	
			default:
				checkf(false, TEXT("Issue severity not set for stack issue."));
		}
		ContentPadding = FMargin(LeftContentPadding, ItemContentVerticalPadding, RightContentPadding, ItemContentVerticalPadding);
		break;
	case UNiagaraStackEntry::EStackRowStyle::Spacer:
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.Spacer");
	}

	if (TableRowStyle == nullptr)
	{
		TableRowStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTableRowStyle>("NiagaraEditor.Stack.TableViewRow.ItemContent");
	}

	return SNew(SNiagaraStackTableRow, StackViewModel, Item, StackCommandContext.ToSharedRef(), StackTree.ToSharedRef())
		.Style(TableRowStyle)
		.ContentPadding(ContentPadding)
		.IndicatorColor(IndicatorColor)
		.IsCategoryIconHighlighted(bIsCategoryIconHighlighted)
		.ShowExecutionCategoryIcon(bShowExecutionCategoryIcon)
		.NameColumnWidth(this, &SNiagaraStack::GetNameColumnWidth)
		.OnNameColumnWidthChanged(this, &SNiagaraStack::OnNameColumnWidthChanged)
		.ValueColumnWidth(this, &SNiagaraStack::GetContentColumnWidth)
		.OnValueColumnWidthChanged(this, &SNiagaraStack::OnContentColumnWidthChanged)
		.OnDragDetected(this, &SNiagaraStack::OnRowDragDetected, Item)
		.OnDragLeave(this, &SNiagaraStack::OnRowDragLeave)
		.OnCanAcceptDrop(this, &SNiagaraStack::OnRowCanAcceptDrop)
		.OnAcceptDrop(this, &SNiagaraStack::OnRowAcceptDrop)
		.IssueIconVisibility(this, &SNiagaraStack::GetIssueIconVisibility);
}


FReply SNiagaraStack::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (StackCommandContext->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SNiagaraStack::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Update the stack view model and synchronize the expansion state before the parent tick to ensure that the state is up to date before ticking child widgets.
	if (StackViewModel)
	{
		StackViewModel->Tick();
	}
	if (bSynchronizeExpansionPending)
	{
		SynchronizeTreeExpansion();
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


SNiagaraStack::FRowWidgets SNiagaraStack::ConstructNameAndValueWidgetsForItem(UNiagaraStackEntry* Item, TSharedRef<SNiagaraStackTableRow> Container)
{
	if (Item->IsA<UNiagaraStackItemGroup>())
	{
		TSharedRef<SWidget> GroupWidget = SNew(SNiagaraStackItemGroup, *CastChecked<UNiagaraStackItemGroup>(Item), StackViewModel);
		if (Item->IsA<UNiagaraStackSimulationStageGroup>())
		{
			Container->AddFillRowContextMenuHandler(SNiagaraStackTableRow::FOnFillRowContextMenu::CreateStatic(
				&FNiagaraStackEditorWidgetsUtilities::AddStackSimulationStageGroupContextMenuActions, CastChecked<UNiagaraStackSimulationStageGroup>(Item), GroupWidget));
		}
		return FRowWidgets(GroupWidget);
	}
	else if (Item->IsA<UNiagaraStackModuleItem>())
	{
		TSharedRef<SNiagaraStackModuleItem> ModuleItemWidget = SNew(SNiagaraStackModuleItem, *CastChecked<UNiagaraStackModuleItem>(Item), StackViewModel);
		Container->AddFillRowContextMenuHandler(SNiagaraStackTableRow::FOnFillRowContextMenu::CreateSP(ModuleItemWidget, &SNiagaraStackModuleItem::FillRowContextMenu));
		return FRowWidgets(ModuleItemWidget);
	}
	else if (Item->IsA<UNiagaraStackFunctionInput>())
	{
		UNiagaraStackFunctionInput* FunctionInput = CastChecked<UNiagaraStackFunctionInput>(Item);

		TSharedRef<SNiagaraStackFunctionInputName> FunctionInputNameWidget =
			SNew(SNiagaraStackFunctionInputName, FunctionInput, StackViewModel)
			.IsSelected(Container, &SNiagaraStackTableRow::IsSelected);

		FunctionInputNameWidget->OnRequestReconstructRow().AddSP(this, &SNiagaraStack::ReconstructNameAndValueWidgetsForItem, Item);

		bool bShowInlineDynamicInput = 
			GbEnableExperimentalInlineDynamicInputs &&
			FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic &&
			FunctionInput->GetInlineDisplayMode() != ENiagaraStackEntryInlineDisplayMode::None;
		TSharedPtr<SWidget> FunctionInputValueWidget = nullptr;
		if (bShowInlineDynamicInput)
		{
			FunctionInputValueWidget = SNew(SNiagaraStackInlineDynamicInput, FunctionInput);
		}
		else
		{
			FunctionInputValueWidget = SNew(SNiagaraStackFunctionInputValue, FunctionInput);
		}

		Container->AddFillRowContextMenuHandler(SNiagaraStackTableRow::FOnFillRowContextMenu::CreateSP(FunctionInputNameWidget, &SNiagaraStackFunctionInputName::FillRowContextMenu));
		if (bShowInlineDynamicInput && 
			(FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHorizontal ||
			FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphVertical ||
			FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHybrid))
		{
			return FRowWidgets(
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					FunctionInputNameWidget
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					FunctionInputValueWidget.ToSharedRef()
				]);
		}
		else
		{
			return FRowWidgets(FunctionInputNameWidget, FunctionInputValueWidget.ToSharedRef());
		}
	}
	else if (Item->IsA<UNiagaraStackErrorItem>())
	{
		return FRowWidgets(SNew(SNiagaraStackErrorItem, CastChecked<UNiagaraStackErrorItem>(Item), StackViewModel));
	}
	else if (Item->IsA<UNiagaraStackErrorItemLongDescription>())
	{
		Container->SetOverrideNameAlignment(EHorizontalAlignment::HAlign_Fill, EVerticalAlignment::VAlign_Center);
		return FRowWidgets(SNew(STextBlock)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.ToolTipText_UObject(Item, &UNiagaraStackEntry::GetTooltipText)
			.Text_UObject(Item, &UNiagaraStackEntry::GetDisplayName)
			.ColorAndOpacity(this, &SNiagaraStack::GetTextColorForItem, Item)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.AutoWrapText(true));
	}
	else if (Item->IsA<UNiagaraStackErrorItemFix>())
	{
		return FRowWidgets(SNew(SNiagaraStackErrorItemFix, CastChecked<UNiagaraStackErrorItemFix>(Item), StackViewModel));
	}
	else if(UNiagaraStackNote* StackNote = Cast<UNiagaraStackNote>(Item))
	{
		TSharedRef<SNiagaraStackNote> ContentWidget = SNew(SNiagaraStackNote, *StackNote)
			.bShowEditTextButtons(Container, &SNiagaraStackTableRow::IsHovered);
		
		Container->AddFillRowContextMenuHandler(SNiagaraStackTableRow::FOnFillRowContextMenu::CreateSP(ContentWidget, &SNiagaraStackNote::FillRowContextMenu));
		return FRowWidgets(ContentWidget);
	}
	else if (Item->IsA<UNiagaraStackItemFooter>())
	{
		UNiagaraStackItemFooter* ItemExpander = CastChecked<UNiagaraStackItemFooter>(Item);
		return FRowWidgets(SNew(SNiagaraStackItemFooter, *ItemExpander));
	}
	else if (Item->IsA<UNiagaraStackCategory>())
	{
		Container->SetOverrideNameAlignment(HAlign_Left, VAlign_Center);
		return FRowWidgets(SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ParameterText")
			.ToolTipText_UObject(Item, &UNiagaraStackEntry::GetTooltipText)
			.Text_UObject(Item, &UNiagaraStackEntry::GetDisplayName)
			.ColorAndOpacity(this, &SNiagaraStack::GetTextColorForItem, Item)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetOwnerIsEnabled),
			SNullWidget::NullWidget);
	}
	else if (Item->IsA<UNiagaraStackModuleItemOutput>())
	{
		UNiagaraStackModuleItemOutput* ModuleItemOutput = CastChecked<UNiagaraStackModuleItemOutput>(Item);
		return FRowWidgets(
			SNew(SNiagaraParameterName)
				.ReadOnlyTextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemText")
				.ToolTipText_UObject(Item, &UNiagaraStackEntry::GetTooltipText)
				.ParameterName(ModuleItemOutput->GetOutputParameterHandle().GetParameterHandleString())
				.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
				.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetOwnerIsEnabled)
				.IsReadOnly(true)
			);
	}
	else if (Item->IsA<UNiagaraStackValueCollection>())
	{
		UNiagaraStackValueCollection* InputCollection = CastChecked<UNiagaraStackValueCollection>(Item);
		return FRowWidgets(SNew(SNiagaraStackValueCollection, InputCollection));
	}
	else if (Item->IsA<UNiagaraStackScriptHierarchyRoot>())
	{
		UNiagaraStackScriptHierarchyRoot* ModuleHierarchyRoot = CastChecked<UNiagaraStackScriptHierarchyRoot>(Item);
		return FRowWidgets(SNew(SNiagaraStackScriptHierarchyRoot, ModuleHierarchyRoot));
	}
	else if (Item->IsA<UNiagaraStackModuleItemOutputCollection>() ||
		Item->IsA<UNiagaraStackModuleItemLinkedInputCollection>())
	{
		return FRowWidgets(
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemText")
			.ToolTipText_UObject(Item, &UNiagaraStackEntry::GetTooltipText)
			.Text_UObject(Item, &UNiagaraStackEntry::GetDisplayName)
			.ColorAndOpacity(this, &SNiagaraStack::GetTextColorForItem, Item)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetOwnerIsEnabled),
			SNullWidget::NullWidget);
	}
	else if (Item->IsA<UNiagaraStackPropertyRow>())
	{
		UNiagaraStackPropertyRow* PropertyRow = CastChecked<UNiagaraStackPropertyRow>(Item);
		FNodeWidgets PropertyRowWidgets = PropertyRow->GetDetailTreeNode()->CreateNodeWidgets();

		TAttribute<bool> IsEnabled;
		IsEnabled.BindUObject(Item, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled);
		
		Container->AddFillRowContextMenuHandler(FNiagaraStackPropertyRowUtilities::CreateOnFillRowContextMenu(*PropertyRow, PropertyRowWidgets.Actions));

		if (PropertyRowWidgets.WholeRowWidget.IsValid())
		{
			Container->SetOverrideNameWidth(PropertyRowWidgets.WholeRowWidgetLayoutData.MinWidth, PropertyRowWidgets.WholeRowWidgetLayoutData.MaxWidth);
			Container->SetOverrideNameAlignment(PropertyRowWidgets.WholeRowWidgetLayoutData.HorizontalAlignment, PropertyRowWidgets.WholeRowWidgetLayoutData.VerticalAlignment);
			PropertyRowWidgets.WholeRowWidget->SetEnabled(IsEnabled);
			TSharedRef<SHorizontalBox> RowBox = SNew(SHorizontalBox);
			if (PropertyRowWidgets.EditConditionWidget.IsValid())
			{
				RowBox->AddSlot()
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					PropertyRowWidgets.EditConditionWidget.ToSharedRef()
				];
			}
			RowBox->AddSlot()
			[
				PropertyRowWidgets.WholeRowWidget.ToSharedRef()
			];

			// do not add reset widget if it is customized
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
			if (PropertyHandle.IsValid() && !(PropertyHandle->IsResetToDefaultCustomized() || PropertyHandle->HasMetaData(TEXT("NoResetToDefault"))))
			{
				RowBox->AddSlot()
				.AutoWidth()
				.Padding(3, 0, 0, 0)
				[
					SNew(SResetToDefaultPropertyEditor, PropertyHandle)
				];
			}

			return FRowWidgets(RowBox); 
		}
		else
		{
			TSharedPtr<const FNiagaraStackObjectPropertyCustomization> Customization = PropertyRow->GetCustomization();
			if(Customization)
			{
				TOptional<TSharedPtr<SWidget>> CustomizedNameWidget = Customization->GenerateNameWidget(PropertyRow);

				if(CustomizedNameWidget.IsSet())
				{
					PropertyRowWidgets.NameWidget = CustomizedNameWidget.GetValue();
				}
			}
			
			Container->SetOverrideNameWidth(PropertyRowWidgets.NameWidgetLayoutData.MinWidth, PropertyRowWidgets.NameWidgetLayoutData.MaxWidth);
			Container->SetOverrideValueWidth(PropertyRowWidgets.ValueWidgetLayoutData.MinWidth, PropertyRowWidgets.ValueWidgetLayoutData.MaxWidth);
			Container->SetOverrideNameAlignment(HAlign_Left, VAlign_Center);
			PropertyRowWidgets.NameWidget->SetEnabled(IsEnabled);
			PropertyRowWidgets.ValueWidget->SetEnabled(IsEnabled);

			
			TSharedPtr<SWidget> NameWidget;
			if (PropertyRowWidgets.EditConditionWidget.IsValid())
			{
				NameWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					PropertyRowWidgets.EditConditionWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				[
					 PropertyRowWidgets.NameWidget.ToSharedRef()
				];
			}
			else
			{
				NameWidget = PropertyRowWidgets.NameWidget;
			}

			TSharedRef<SHorizontalBox> ValueWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				PropertyRowWidgets.ValueWidget.ToSharedRef()
			];

			// do not add reset widget if it is customized
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
			if (PropertyHandle.IsValid() && !(PropertyHandle->IsResetToDefaultCustomized() || PropertyHandle->HasMetaData(TEXT("NoResetToDefault"))))
			{
				ValueWidget->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3, 0, 0, 0)
				[
					SNew(SResetToDefaultPropertyEditor, PropertyHandle)
				];
			}

			return FRowWidgets(NameWidget.ToSharedRef(), ValueWidget);
		}
	}
	else if (Item->IsA<UNiagaraStackItem>())
	{
		UNiagaraStackItem* StackItem = CastChecked<UNiagaraStackItem>(Item);
		return FRowWidgets(SNew(SNiagaraStackItem, *StackItem, StackViewModel));
	}
	else if (Item->IsA<UNiagaraStackItemTextContent>())
	{
		Container->SetContentPadding(FMargin(5));
		UNiagaraStackItemTextContent* ItemTextContent = CastChecked<UNiagaraStackItemTextContent>(Item);
		return FRowWidgets(SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.TextContentText")
			.Text(ItemTextContent->GetDisplayName())
			.AutoWrapText(true)
			.Justification(ETextJustify::Center));
	}
	else if (Item->IsA<UNiagaraStackSummaryCategory>())
	{
		Container->SetOverrideNameAlignment(HAlign_Left, VAlign_Center);
		return FRowWidgets(SNew(STextBlock)
			.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle"))
			.ToolTipText_UObject(Item, &UNiagaraStackEntry::GetTooltipText)
			.Text_UObject(Item, &UNiagaraStackEntry::GetDisplayName)
			.ColorAndOpacity(this, &SNiagaraStack::GetTextColorForItem, Item)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.IsEnabled_UObject(Item, &UNiagaraStackEntry::GetOwnerIsEnabled),
			SNullWidget::NullWidget);
	}
	else if (Item->IsA<UNiagaraStackSpacer>())
	{
		UNiagaraStackSpacer* Spacer = CastChecked<UNiagaraStackSpacer>(Item);
		return FRowWidgets(SNew(SBox)
			.HeightOverride(Spacer->GetSpacerHeight()));
	}
	else
	{
		return FRowWidgets(SNullWidget::NullWidget);
	}
}

void SNiagaraStack::ReconstructNameAndValueWidgetsForItem(UNiagaraStackEntry* Item)
{
	TSharedPtr<ITableRow> TableRow = StackTree->WidgetFromItem(Item);
	if (TableRow.IsValid())
	{
		TSharedRef<SNiagaraStackTableRow> Container = StaticCastSharedRef<SNiagaraStackTableRow>(TableRow.ToSharedRef());
		Container->Reset();
		FRowWidgets RowWidgets = ConstructNameAndValueWidgetsForItem(Item, Container);
		Container->SetNameAndValueContent(RowWidgets.NameWidget, RowWidgets.ValueWidget);
	}
}

void SNiagaraStack::OnGetChildren(UNiagaraStackEntry* Item, TArray<UNiagaraStackEntry*>& Children)
{
	Item->GetFilteredChildren(Children);
}

void SNiagaraStack::StackTreeScrolled(double ScrollValue)
{
	StackViewModel->SetLastScrollPosition(ScrollValue);
}

void SNiagaraStack::StackTreeSelectionChanged(UNiagaraStackEntry* InNewSelection, ESelectInfo::Type SelectInfo)
{
	TArray<UNiagaraStackEntry*> SelectedStackEntries;
	StackTree->GetSelectedItems(SelectedStackEntries);
	StackCommandContext->SetSelectedEntries(SelectedStackEntries);
}

float SNiagaraStack::GetNameColumnWidth() const
{
	return NameColumnWidth;
}

float SNiagaraStack::GetContentColumnWidth() const
{
	return ContentColumnWidth;
}

void SNiagaraStack::OnNameColumnWidthChanged(float Width)
{
	NameColumnWidth = Width;
}

void SNiagaraStack::OnContentColumnWidthChanged(float Width)
{
	ContentColumnWidth = Width;
}

void SNiagaraStack::OnStackExpansionChanged()
{
	bSynchronizeExpansionPending = true;
}

void SNiagaraStack::StackStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	bSynchronizeExpansionPending = true;
	StackTree->RequestTreeRefresh();
	HeaderList->RequestListRefresh();
}

EVisibility SNiagaraStack::GetIssueIconVisibility() const
{
	return StackViewModel->HasIssues() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SNiagaraStack::OnCycleThroughSystemIssues(TSharedPtr<FNiagaraSystemViewModel> SystemViewModel)
{
	UNiagaraStackEntry* RootEntry = SystemViewModel->GetSystemStackViewModel()->GetRootEntry();
	if (RootEntry != nullptr)
	{
		TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModel = StackViewModel->GetTopLevelViewModelForEntry(*RootEntry);
		StackViewModel->OnCycleThroughIssues(TopLevelViewModel);
		OnCycleThroughIssues();
	}

	return FReply::Handled();
}

void SNiagaraStack::OnCycleThroughIssues()
{
	UNiagaraStackEntry* StackEntry = StackViewModel->GetCurrentFocusedIssue();

	TArray<UNiagaraStackEntry*> EntryPath;
	StackViewModel->GetPathForEntry(StackEntry, EntryPath);
	EntryPath.Add(StackEntry);

	ExpandAllInPath(EntryPath);
	SynchronizeTreeExpansion();

	StackTree->RequestScrollIntoView(StackEntry);
}

const FSlateBrush* SNiagaraStack::GetViewOptionsBadgeIcon() const
{
	// Badge the icon if any view option that narrows down the results is checked
	bool bHasBadge = (StackViewModel->GetShowOnlyModified() || StackViewModel->GetShowOnlyIssues()
		|| StackViewModel->GetShowAllAdvanced() || StackViewModel->GetShowLinkedInputs()
		|| StackViewModel->GetShowOutputs());
	
	return bHasBadge ? FAppStyle::Get().GetBrush("Icons.BadgeModified") : nullptr;
}

#undef LOCTEXT_NAMESPACE
