// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

class UNiagaraStackViewModel;
class SNiagaraStackTableRow;
class SSearchBox;
class FReply;
class FNiagaraStackCommandContext;
class SWidget;
class SNiagaraStack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStack)
	{}

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel);

	TSharedPtr<SWidget> GenerateStackMenu(TWeakPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModelWeak);

private:
	struct FRowWidgets
	{
		FRowWidgets(TSharedRef<SWidget> InNameWidget, TSharedRef<SWidget> InValueWidget)
			: NameWidget(InNameWidget)
			, ValueWidget(InValueWidget)
		{
		}

		FRowWidgets(TSharedRef<SWidget> InWholeRowWidget)
			: NameWidget(InWholeRowWidget)
		{
		}

		TSharedRef<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
	};

	void SynchronizeTreeExpansion();

	TSharedRef<ITableRow> OnGenerateRowForStackItem(UNiagaraStackEntry* Item, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedRef<ITableRow> OnGenerateRowForTopLevelObject(TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable);

	

	TSharedRef<SNiagaraStackTableRow> ConstructContainerForItem(UNiagaraStackEntry* Item);

	FRowWidgets ConstructNameAndValueWidgetsForItem(UNiagaraStackEntry* Item, TSharedRef<SNiagaraStackTableRow> Container);

	void ReconstructNameAndValueWidgetsForItem(UNiagaraStackEntry* Item);
	
	void OnGetChildren(UNiagaraStackEntry* Item, TArray<UNiagaraStackEntry*>& Children);

	void StackTreeScrolled(double ScrollValue);

	void StackTreeSelectionChanged(UNiagaraStackEntry* InNewSelection, ESelectInfo::Type SelectInfo);

	float GetNameColumnWidth() const;
	float GetContentColumnWidth() const;

	void OnNameColumnWidthChanged(float Width);
	void OnContentColumnWidthChanged(float Width);

	void OnStackExpansionChanged();
	void StackStructureChanged(ENiagaraStructureChangedFlags Info);

	EVisibility GetVisibilityForItem(UNiagaraStackEntry* Item) const;


	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedRef<SWidget> ConstructHeaderWidget();

	// ~stack search stuff
	void UpdateSearchTextFromExternal(FText NewSearchText);
	void OnSearchTextChanged(const FText& SearchText);
	FReply ScrollToNextMatch();
	FReply ScrollToPreviousMatch();
	TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;
	bool GetIsSearching() const;
	void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);	
	void OnSearchBoxSearch(SSearchBox::SearchDirection Direction);
	FSlateColor GetTextColorForItem(UNiagaraStackEntry* Item) const;
	void ScrollFocusedEntryIntoView();
	void AddSearchScrollOffset(int NumberOfSteps);
	void OnStackSearchComplete(const FNiagaraStackSearchConfig& SearchConfig);
	void ExpandSearchResults();
	bool IsEntryFocusedInSearch(UNiagaraStackEntry* Entry) const;

	void SelectStackEntry(UNiagaraStackEntry* StackEntry, FNiagaraStackEntrySelectionConfig Config);

	// Inline menu commands
	void ShowEmitterInContentBrowser(TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelWeak);
	void NavigateTo(UNiagaraStackEntry* Item);
	void CollapseAll();

	TSharedRef<SWidget> GetViewOptionsMenu() const;

	// Drag/Drop
	FReply OnRowDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, UNiagaraStackEntry* InStackEntry);

	void OnRowDragLeave(FDragDropEvent const& InDragDropEvent);

	TOptional<EItemDropZone> OnRowCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry);

	FReply OnRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry);

	EVisibility GetIssueIconVisibility() const;

	FReply OnCycleThroughSystemIssues(TSharedPtr<FNiagaraSystemViewModel> SystemViewModel);
	void OnCycleThroughIssues();

	const FSlateBrush* GetViewOptionsBadgeIcon() const;
private:
	UNiagaraStackViewModel* StackViewModel;

	TSharedPtr<STreeView<UNiagaraStackEntry*>> StackTree;

	TSharedPtr<SListView<TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel>>> HeaderList;

	float NameColumnWidth;

	float ContentColumnWidth;

	// ~ search stuff
	TSharedPtr<SSearchBox> SearchBox;

	TSharedPtr<FNiagaraStackCommandContext> StackCommandContext;

	bool bSynchronizeExpansionPending;
};
