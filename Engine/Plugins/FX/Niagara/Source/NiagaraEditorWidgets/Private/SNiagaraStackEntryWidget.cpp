// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStackEntryWidget.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraStackEditorData.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SNiagaraStackEntryWidget"

void SNiagaraStackDisplayName::Construct(const FArguments& InArgs, UNiagaraStackEntry& InStackEntry, UNiagaraStackViewModel& InStackViewModel)
{
	StackEntryItem = &InStackEntry;
	StackViewModel = &InStackViewModel;

	NameStyle = InArgs._NameStyle;
 	EditableNameStyle = InArgs._EditableNameStyle;
 	OriginalNameStyle = InArgs._OriginalNameStyle;

	TAttribute<FText> EntryToolTipText;
	EntryToolTipText.Bind(this, &SNiagaraStackDisplayName::GetEntryToolTipText);
	SetToolTip(FSlateApplication::Get().MakeToolTip(EntryToolTipText));

	TAttribute<bool> EntryIsEnabled;
	EntryIsEnabled.Bind(this, &SNiagaraStackDisplayName::GetEntryIsEnabled);
	SetEnabled(EntryIsEnabled);

	StackViewModel->OnStructureChanged().AddSP(this, &SNiagaraStackDisplayName::StackViewModelStructureChanged);
	StackEntryItem->OnAlternateDisplayNameChanged().AddSP(this, &SNiagaraStackDisplayName::StackEntryItemAlternateNameChanged);

	ChildSlot
	[
		SAssignNew(Container, SBox)
		[
			ConstructChildren()
		]
	];
}

SNiagaraStackDisplayName::~SNiagaraStackDisplayName()
{
	StackViewModel->OnStructureChanged().RemoveAll(this);
	StackEntryItem->OnAlternateDisplayNameChanged().RemoveAll(this);
}

TSharedRef<SWidget> SNiagaraStackDisplayName::ConstructChildren()
{
	EditableTextBlock.Reset();
	TArray<TSharedRef<SWidget>> NameWidgets;

	// First check to see if we need to insert the emitter name.
	int32 NumTopLevelEmitters = 0;
	for (const TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel>& TopLevelViewModel : StackViewModel->GetTopLevelViewModels())
	{
		if (TopLevelViewModel->EmitterHandleViewModel.IsValid())
		{
			NumTopLevelEmitters++;
		}
	}

	if (NumTopLevelEmitters > 1)
	{
		TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModel = StackViewModel->GetTopLevelViewModelForEntry(*StackEntryItem);
		NameWidgets.Add(SNew(STextBlock)
			.TextStyle(NameStyle)
			.Text(this, &SNiagaraStackDisplayName::GetTopLevelDisplayName, TWeakPtr<UNiagaraStackViewModel::FTopLevelViewModel>(TopLevelViewModel))
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.ColorAndOpacity(this, &SNiagaraStackDisplayName::GetTextColorForSearch, FSlateColor::UseForeground()));
	}
	TopLevelViewModelCountAtLastConstruction = NumTopLevelEmitters;

	// Next add the main name widget which will be the alternate name if it's available, otherwise it's the regular display name.
	if (StackEntryItem->SupportsRename())
	{
		// If the entry can be renamed we need an editable text block.
		NameWidgets.Add(SAssignNew(EditableTextBlock, SInlineEditableTextBlock)
			.Style(EditableNameStyle)
			.Text(this, &SNiagaraStackDisplayName::GetEntryDisplayName)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.ColorAndOpacity(this, &SNiagaraStackEntryWidget::GetTextColorForSearch, FSlateColor::UseForeground())
			.OnTextCommitted(this, &SNiagaraStackDisplayName::EntryNameTextCommitted));
	}
	else
	{
		// Otherwise add a regular text block.
		NameWidgets.Add(SNew(STextBlock)
			.TextStyle(NameStyle)
			.Text(this, &SNiagaraStackDisplayName::GetEntryDisplayName)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.ColorAndOpacity(this, &SNiagaraStackDisplayName::GetTextColorForSearch, FSlateColor::UseForeground()));
	}

	// Finally add a subdued box for the regular display name if we're showing an alternate name.
	if(StackEntryItem->GetAlternateDisplayName().IsSet())
	{
		NameWidgets.Add(SNew(STextBlock)
			.TextStyle(OriginalNameStyle)
			.Text(this, &SNiagaraStackDisplayName::GetOriginalName)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.ColorAndOpacity(this, &SNiagaraStackDisplayName::GetTextColorForSearch, FSlateColor::UseSubduedForeground()));
	}

	// If there is more than one name, put them in a wrap box so that they flow correctly when the rows are narrow.
	if(NameWidgets.Num() > 1)
	{
		TSharedRef<SWrapBox> NamesWrapBox = SNew(SWrapBox)
			.UseAllottedSize(true);
		for (TSharedRef<SWidget> NameWidget : NameWidgets)
		{
			NamesWrapBox->AddSlot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0, 0, 5, 0))
				[
					NameWidget
				];
		}
		return NamesWrapBox;
	}
	else if (NameWidgets.Num() == 1)
	{
		return NameWidgets[0];
	}

	TopLevelViewModelCountAtLastConstruction = -1;
	return SNullWidget::NullWidget;
}

FText SNiagaraStackDisplayName::GetTopLevelDisplayName(TWeakPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModelWeak) const
{
	TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModel = TopLevelViewModelWeak.Pin();
	if (TopLevelViewModel.IsValid())
	{
		if (TopLevelViewModel->GetDisplayName().IdenticalTo(TopLevelDisplayNameCache) == false)
		{
			TopLevelDisplayNameCache = TopLevelViewModel->GetDisplayName();
			TopLevelDisplayNameFormattedCache = FText::Format(LOCTEXT("TopLevelDisplayNameFormat", "{0} -"), TopLevelDisplayNameCache);
		}
	}
	else
	{
		TopLevelDisplayNameFormattedCache = FText();
	}
	return TopLevelDisplayNameFormattedCache;
}

void SNiagaraStackDisplayName::StackViewModelStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	if (StackEntryItem->IsFinalized() == false && StackViewModel->GetTopLevelViewModels().Num() != TopLevelViewModelCountAtLastConstruction)
	{
		Container->SetContent(ConstructChildren());
	}
}

void SNiagaraStackDisplayName::StackEntryItemAlternateNameChanged()
{
	if (StackEntryItem->IsFinalized() == false)
	{
		Container->SetContent(ConstructChildren());
	}
}

FText SNiagaraStackDisplayName::GetEntryDisplayName() const 
{
	return StackEntryItem->GetAlternateDisplayName().IsSet() ? StackEntryItem->GetAlternateDisplayName().GetValue() : StackEntryItem->GetDisplayName();
}

FText SNiagaraStackDisplayName::GetOriginalName() const
{
	if (StackEntryItem->IsFinalized() == false)
	{
		return FText::Format(FTextFormat::FromString(TEXT("({0})")), StackEntryItem->GetDisplayName());
	}
	return FText::GetEmpty();
}

FText SNiagaraStackDisplayName::GetEntryToolTipText() const
{
	if (StackEntryItem->IsFinalized() == false)
	{
		return StackEntryItem->GetTooltipText();
	}
	return FText::GetEmpty();
}

bool SNiagaraStackDisplayName::GetEntryIsEnabled() const
{
	if (StackEntryItem->IsFinalized() == false)
	{
		return StackEntryItem->GetIsEnabledAndOwnerIsEnabled();
	}
	return false;
}

void SNiagaraStackDisplayName::EntryNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (StackEntryItem->IsFinalized() == false && CommitInfo != ETextCommit::OnCleared)
	{
		StackEntryItem->OnRenamed(InText);
	}
}

void SNiagaraStackDisplayName::StartRename()
{
	if (EditableTextBlock.IsValid())
	{
		EditableTextBlock->EnterEditingMode();
	}
}

FSlateColor SNiagaraStackEntryWidget::GetTextColorForSearch(FSlateColor DefaultColor) const
{
	if (IsCurrentSearchMatch())
	{
		return FStyleColors::Select;
	}
	return DefaultColor;
}

bool SNiagaraStackEntryWidget::IsCurrentSearchMatch() const
{
	UNiagaraStackEntry* FocusedEntry = StackViewModel->GetCurrentFocusedEntry();
	return StackEntryItem != nullptr && FocusedEntry == StackEntryItem;
}

FReply SNiagaraStackEntryWidget::ExpandEntry()
{
	StackEntryItem->SetIsExpanded(true);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
