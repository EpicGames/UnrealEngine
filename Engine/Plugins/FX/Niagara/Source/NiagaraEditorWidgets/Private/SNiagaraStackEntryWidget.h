// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Framework/SlateDelegates.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "Styling/AppStyle.h"

class UNiagaraStackViewModel;
class UNiagaraStackEntry;
class SBox;
class SInlineEditableTextBlock;

class SNiagaraStackEntryWidget : public SCompoundWidget
{
public:
	FReply ExpandEntry();
	
	FSlateColor GetTextColorForSearch(FSlateColor DefaultColor) const;

protected:
	bool IsCurrentSearchMatch() const;
	
protected:
	UNiagaraStackViewModel* StackViewModel;
	UNiagaraStackEntry* StackEntryItem;
};

class SNiagaraStackDisplayName : public SNiagaraStackEntryWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackDisplayName)
		: _NameStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		, _EditableNameStyle(&FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle"))
 		, _OriginalNameStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, NameStyle)
		SLATE_STYLE_ARGUMENT(FInlineEditableTextBlockStyle, EditableNameStyle)
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, OriginalNameStyle)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackEntry& InStackEntry, UNiagaraStackViewModel& InStackViewModel);

	~SNiagaraStackDisplayName();

	void StartRename();

private:
	void OnEndRename();

	TSharedRef<SWidget> ConstructChildren();

	FText GetTopLevelDisplayName(TWeakPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModelWeak) const;

	void StackViewModelStructureChanged(ENiagaraStructureChangedFlags Flags);

	void StackEntryItemAlternateNameChanged();

	FText GetEntryDisplayName() const;

	FText GetOriginalName() const;

	FText GetEntryToolTipText() const;

	bool GetEntryIsEnabled() const;

	void EntryNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

private:
	const FTextBlockStyle* NameStyle;
	const FInlineEditableTextBlockStyle* EditableNameStyle;
	const FTextBlockStyle* OriginalNameStyle;

	TSharedPtr<SBox> Container;

	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;
	FOnTextCommitted OnRenameCommitted;

	mutable FText TopLevelDisplayNameCache;
	mutable FText TopLevelDisplayNameFormattedCache;
	int32 TopLevelViewModelCountAtLastConstruction;
};
