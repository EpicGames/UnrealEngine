// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStackNote.h"

#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"

#define LOCTEXT_NAMESPACE "SNiagaraStackNote"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNiagaraStackInlineNote::Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry)
{
	StackEntry = InStackEntry;
	bInteractable = InArgs._bInteractable;

	TSharedRef<SWidget> ChildWidget = SNew(SImage)
		.Image(FAppStyle::GetBrush("Icons.Comment"))
		.ColorAndOpacity(this, &SNiagaraStackInlineNote::GetStackNoteColor);

	if(bInteractable)
	{
		ChildWidget = SNew(SButton)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		.OnClicked(this, &SNiagaraStackInlineNote::OnClicked)
		.Content()
		[
			ChildWidget	
		];
	}
	ChildSlot
	[
		ChildWidget
	];

	UpdateTooltip();
}

void SNiagaraStackInlineNote::UpdateTooltip()
{
	if(StackEntry.IsValid() && StackEntry->GetStackNote() != nullptr)
	{
		SetToolTip(FNiagaraEditorUtilities::Tooltips::CreateStackNoteTooltip(*StackEntry->GetStackNote()));
	}
}

FReply SNiagaraStackInlineNote::OnClicked() const
{
	StackEntry->GetStackNote()->ToggleInlineDisplay();
	return FReply::Handled();
}

FSlateColor SNiagaraStackInlineNote::GetStackNoteColor() const
{
	if(StackEntry.IsValid() && StackEntry->IsFinalized() == false && StackEntry->GetStackNote())
	{
		return StackEntry->GetStackNote()->GetColor();
	}

	return GetDefault<UNiagaraEditorSettings>()->GetDefaultNoteColor();
}

void SNiagaraStackInlineNote::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// this is a workaround. We force a regeneration of the tooltip whenever we enter it.
	// SetTooltip with a TAttribute bound to a function should work to create the tooltip whenever it's needed, but doesn't work right now.
	UpdateTooltip();
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
}

void SNiagaraStackNote::Construct(const FArguments& InArgs, UNiagaraStackNote& InStackNote)
{
	StackNote = &InStackNote;
	ShowEditTextButtons = InArgs._bShowEditTextButtons;
	
	StackNote->OnRequestEditHeader().BindSP(this, &SNiagaraStackNote::EditHeaderText);

	Rebuild();
}

SNiagaraStackNote::~SNiagaraStackNote()
{
	if(StackNote.IsValid())
	{
		StackNote->OnRequestEditHeader().Unbind();
	}
}

void SNiagaraStackNote::Rebuild()
{	
	ChildSlot
	[
		SNullWidget::NullWidget
	];

	TOptional<FNiagaraStackNoteData> MatchingStackNote = GetStackNoteData();
	if(MatchingStackNote.IsSet() == false)
	{
		return;
	}
	
	FNiagaraStackNoteData DisplayedStackNote = MatchingStackNote.GetValue();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(16.f)
			[
				SNew(SColorBlock)
				.Color(this, &SNiagaraStackNote::GetStackNoteColor)
				.OnMouseButtonDown(this, &SNiagaraStackNote::SummonColorPicker)
			]
		]
		+ SHorizontalBox::Slot()
		[
			SAssignNew(ExpandableArea, SExpandableArea)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(FMargin(20.f, 2.f, 5.f, 2.f))
			.HeaderContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
					.OnClicked(this, &SNiagaraStackNote::OnToggleInlineDisplayClicked)
					.ToolTipText(LOCTEXT("InlineNoteButtonTooltip", "Converts this note to an inline note."))
					.Content()
					[
						SNew(SBox)
						.HeightOverride(16.f)
						.WidthOverride(16.f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Comment"))
							.ColorAndOpacity(this, &SNiagaraStackNote::GetSlateStackNoteColor)
						]
					]
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SAssignNew(HeaderText, SInlineEditableTextBlock)
						.Text(this, &SNiagaraStackNote::GetStackNoteHeader)
						.OnTextCommitted(this, &SNiagaraStackNote::CommitStackNoteHeaderUpdate)
						.Style(&FNiagaraEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("NiagaraEditor.Stack.Note.HeaderEditableText"))
						.IsSelected(FIsSelected::CreateLambda([]() { return false; }))
						.AutoWrapNonEditText(true)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Bottom)
					.Padding(10.f, 2.f)
					[
						SNew(SButton)
						.OnClicked(this, &SNiagaraStackNote::OnEditHeaderButtonClicked)
						.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
						.Visibility(this, &SNiagaraStackNote::GetEditNoteHeaderButtonVisibility)
						.ToolTipText(LOCTEXT("EditHeaderButtonTooltip", "Edit the header of this note."))
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Edit"))
						]
					]
				]
			]
			.BodyContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SAssignNew(BodyText, SInlineEditableTextBlock)
					.Text(this, &SNiagaraStackNote::GetStackNoteBody)
					.OnTextCommitted(this, &SNiagaraStackNote::CommitStackNoteBodyUpdate)
					.AutoWrapNonEditText(true)
					.AutoWrapMultilineEditText(true)
					.MultiLine(true)
					.ModiferKeyForNewLine(EModifierKey::Shift)
					.IsSelected(FIsSelected::CreateLambda([]() { return false; }))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Bottom)
				.Padding(10.f, 2.f)
				[
					SNew(SButton)
					.OnClicked(this, &SNiagaraStackNote::OnEditBodyButtonClicked)
					.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
					.Visibility(this, &SNiagaraStackNote::GetEditNoteBodyButtonVisibility)
					.ToolTipText(LOCTEXT("EditMessageButtonTooltip", "Edit the message of this note."))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Edit"))
					]
				]
			]
		]
	];
}

void SNiagaraStackNote::FillRowContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("NoteActions", LOCTEXT("NoteActions", "Note Actions"));
	{		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("EditTitle", "Edit Title"),
			LOCTEXT("EditTitleTooltip", "Edit the title of this note."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackNote::EditHeaderText)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("EditBody", "Edit Message"),
			LOCTEXT("EditMessageTooltip", "Edit the message of this note."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackNote::EditBodyText)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleInline", "Toggle Inline Display"),
			LOCTEXT("ToggleInlineTooltip", "Toggle the Inline Display for this note.\nAn inlined note will show up in the row itself, saving on space."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Comment"),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackNote::ToggleInlineDisplay)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteNote", "Delete Note"),
			LOCTEXT("DeleteNoteTooltip", "Delete this note."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackNote::DeleteStackNote)));
	}
	MenuBuilder.EndSection();
}

void SNiagaraStackNote::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if(StackNote->GetIsRenamePending())
	{
		if(HeaderText.IsValid())
		{
			EditBodyText();
			StackNote->SetIsRenamePending(false);
		}
	}
}

TOptional<FNiagaraStackNoteData> SNiagaraStackNote::GetStackNoteData() const
{
	return StackNote->GetTargetStackNoteData();
}

void SNiagaraStackNote::EditHeaderText() const
{
	HeaderText->EnterEditingMode();
}

void SNiagaraStackNote::EditBodyText() const
{
	ExpandableArea->SetExpanded(true);
	BodyText->EnterEditingMode();
}

void SNiagaraStackNote::CommitStackNoteHeaderUpdate(const FText& Text, ETextCommit::Type Arg)
{
	FScopedTransaction Transaction(LOCTEXT("UpdateNoteHeaderTransaction", "Updated Note Header"));
	StackNote->GetStackEditorData().Modify();
	
	FNiagaraStackNoteData UpdatedMessage = GetStackNoteData().GetValue();
	UpdatedMessage.MessageHeader = Text;

	StackNote->GetStackEditorData().AddOrReplaceStackNote(StackNote->GetTargetStackEntryKey(), UpdatedMessage);
	Rebuild();
}

void SNiagaraStackNote::CommitStackNoteBodyUpdate(const FText& Text, ETextCommit::Type Arg)
{
	FScopedTransaction Transaction(LOCTEXT("UpdateNoteMessageTransaction", "Updated Note Message"));
	StackNote->GetStackEditorData().Modify();
	
	FNiagaraStackNoteData UpdatedMessage = GetStackNoteData().GetValue();
	UpdatedMessage.Message = Text;

	StackNote->GetStackEditorData().AddOrReplaceStackNote(StackNote->GetTargetStackEntryKey(), UpdatedMessage);
	Rebuild();
}

void SNiagaraStackNote::ToggleInlineDisplay() const
{
	StackNote->ToggleInlineDisplay();
}

void SNiagaraStackNote::DeleteStackNote() const
{
	StackNote->DeleteTargetStackNote();
}

FText SNiagaraStackNote::GetStackNoteHeader() const
{
	return GetStackNoteData().GetValue().MessageHeader;
}

FText SNiagaraStackNote::GetStackNoteBody() const
{
	return GetStackNoteData().GetValue().Message;
}

FLinearColor SNiagaraStackNote::GetStackNoteColor() const
{
	return StackNote->GetColor();
}

FSlateColor SNiagaraStackNote::GetSlateStackNoteColor() const
{
	return GetStackNoteColor();
}

FReply SNiagaraStackNote::OnToggleInlineDisplayClicked() const
{
	ToggleInlineDisplay();
	return FReply::Handled();
}

FReply SNiagaraStackNote::OnEditHeaderButtonClicked()
{
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, double)
	{
		EditHeaderText();
		return EActiveTimerReturnType::Stop;
	}));
	return FReply::Handled();
}

FReply SNiagaraStackNote::OnEditBodyButtonClicked()
{
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, double)
	{
		EditBodyText();
		return EActiveTimerReturnType::Stop;
	}));
	return FReply::Handled();
}

FReply SNiagaraStackNote::SummonColorPicker(const FGeometry& Geometry, const FPointerEvent& PointerEvent) const
{	
	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bOnlyRefreshOnOk = true;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SNiagaraStackNote::OnColorPickedCommitted);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SNiagaraStackNote::OnColorPickerCancelled);
		PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SNiagaraStackNote::OnColorPickerClosed);
		PickerArgs.bUseAlpha = false;
		PickerArgs.InitialColor = GetStackNoteColor();
		PickerArgs.bOnlyRefreshOnMouseUp = true;
		PickerArgs.bOnlyRefreshOnOk = false;
	}
	
	OpenColorPicker(PickerArgs);
	return FReply::Handled();
}

void SNiagaraStackNote::OnColorPickedCommitted(FLinearColor LinearColor) const
{
	StackNote->UpdateNoteColor(LinearColor);
}

void SNiagaraStackNote::OnColorPickerCancelled(FLinearColor LinearColor) const
{
	StackNote->UpdateNoteColor(LinearColor);
}

void SNiagaraStackNote::OnColorPickerClosed(const TSharedRef<SWindow>& Window) const
{
}

EVisibility SNiagaraStackNote::GetEditNoteHeaderButtonVisibility() const
{
	return ShowEditTextButtons.Get() ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility SNiagaraStackNote::GetEditNoteBodyButtonVisibility() const
{
	return ShowEditTextButtons.Get() ? EVisibility::Visible : EVisibility::Hidden;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
