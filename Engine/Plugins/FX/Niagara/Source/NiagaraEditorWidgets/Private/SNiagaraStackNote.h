// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewModels/Stack/NiagaraStackNote.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define UE_API NIAGARAEDITORWIDGETS_API

/** A possibly interactable widget showing a note. On hover, summons the note as a tooltip. On clicked, toggles between inline & full display */
class SNiagaraStackInlineNote : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackInlineNote)
		: _bInteractable(true)
		{}
	
		/** If true, a button will be added. If false, it will be a simple image.  */
		SLATE_ARGUMENT(bool, bInteractable)	
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry);

	UE_API void UpdateTooltip();
private:
	UE_API FReply OnClicked() const;

	UE_API FSlateColor GetStackNoteColor() const;

	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
private:
	TWeakObjectPtr<UNiagaraStackEntry> StackEntry;
	bool bInteractable = true;
};

/** The full stack note widget displayed in the stack. Features editable title, message & toggle inline display button. */
class SNiagaraStackNote : public SCompoundWidget
{
public:	
	SLATE_BEGIN_ARGS(SNiagaraStackNote)
		: _bShowEditTextButtons(true)
		{}
		SLATE_ATTRIBUTE(bool, bShowEditTextButtons)
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs, UNiagaraStackNote& StackNote);
	UE_API virtual ~SNiagaraStackNote() override;

	UE_API void Rebuild();

	UE_API void FillRowContextMenu(FMenuBuilder& MenuBuilder);
private:
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	UE_API TOptional<FNiagaraStackNoteData> GetStackNoteData() const;
	
	UE_API void EditHeaderText() const;
	UE_API void EditBodyText() const;
	
	UE_API void CommitStackNoteHeaderUpdate(const FText& Text, ETextCommit::Type Arg);
	UE_API void CommitStackNoteBodyUpdate(const FText& Text, ETextCommit::Type Arg);
	
	UE_API void ToggleInlineDisplay() const;
	UE_API void DeleteStackNote() const;
	
	UE_API FText GetStackNoteHeader() const;
	UE_API FText GetStackNoteBody() const;
	UE_API FLinearColor GetStackNoteColor() const;
	UE_API FSlateColor GetSlateStackNoteColor() const;

private:
	UE_API FReply OnToggleInlineDisplayClicked() const;

	UE_API FReply OnEditHeaderButtonClicked();
	UE_API FReply OnEditBodyButtonClicked();
	
	UE_API FReply SummonColorPicker(const FGeometry& Geometry, const FPointerEvent& PointerEvent) const;
	UE_API void OnColorPickedCommitted(FLinearColor LinearColor) const;
	UE_API void OnColorPickerCancelled(FLinearColor LinearColor) const;
	UE_API void OnColorPickerClosed(const TSharedRef<SWindow>& Window) const;

	UE_API EVisibility GetEditNoteHeaderButtonVisibility() const;
	UE_API EVisibility GetEditNoteBodyButtonVisibility() const;

private:
	TWeakObjectPtr<UNiagaraStackNote> StackNote;
private:
	TSharedPtr<SExpandableArea> ExpandableArea;
	TSharedPtr<SInlineEditableTextBlock> HeaderText;
	TSharedPtr<SInlineEditableTextBlock> BodyText;

	TAttribute<bool> ShowEditTextButtons;
};

#undef UE_API
