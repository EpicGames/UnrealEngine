// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSystemEffectTypeBar.h"
#include "ISinglePropertyView.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "Delegates/DelegateInstanceInterface.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEffectType.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemEffectType"

SNiagaraSystemEffectTypeBar::~SNiagaraSystemEffectTypeBar()
{
	System->OnScalabilityChanged().RemoveAll(this);
}

void SNiagaraSystemEffectTypeBar::Construct(const FArguments& InArgs, UNiagaraSystem& InSystem)
{
	System = &InSystem;
	EffectType = System->GetEffectType();
	
	EffectTypeValuesBox = SNew(SWrapBox).UseAllottedSize(true);
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FSinglePropertyParams EffectTypePropertyArgs;
	EffectTypePropertyArgs.bHideAssetThumbnail = true;
	
	TAttribute<int32> WidgetSwitcherIndexAttribute = TAttribute<int32>::CreateSP(this, &SNiagaraSystemEffectTypeBar::GetActiveDetailsWidgetIndex);
	ChildSlot
	.Padding(5.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			PropertyEditorModule.CreateSingleProperty(System.Get(), "EffectType", EffectTypePropertyArgs).ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(WidgetSwitcherIndexAttribute)
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoEffectTypeSelected", "No Effect Type assigned.\nEffect Types are used to provide default scalability settings that can partly be overridden."))
				.AutoWrapText(true)
			]
			+ SWidgetSwitcher::Slot()
			[
				EffectTypeValuesBox.ToSharedRef()
			]
		]
	];
	
	UpdateEffectTypeWidgets();
	System->OnScalabilityChanged().AddSP(this, &SNiagaraSystemEffectTypeBar::UpdateEffectType);
}

void SNiagaraSystemEffectTypeBar::UpdateEffectTypeWidgets()
{
	EffectTypeValuesBox->ClearChildren();
	
	for(TFieldIterator<FProperty> PropIt(UNiagaraEffectType::StaticClass(), EFieldIterationFlags::None); PropIt; ++PropIt)
	{
		// we only allow display of properties of effect type that are explicitly marked
		if(!PropIt->HasMetaData(TEXT("DisplayInSystemScalability")))
		{
			continue;
		}
		
		FName PropertyName = PropIt->GetFName();
		FText PropertyDisplayNameText = PropIt->GetDisplayNameText();
		FSinglePropertyParams PropertyParams;
		PropertyParams.NamePlacement = EPropertyNamePlacement::Hidden;

		if(PropIt->HasMetaData("ScalabilityBarDisplayName"))
		{
			PropertyDisplayNameText = PropIt->GetMetaDataText("ScalabilityBarDisplayName");
		}
		
		TSharedPtr<SWidget> PropertyValueWidget;
		PropertyValueWidget = CreatePropertyValueWidget(*PropIt);
		
		if(PropertyValueWidget.IsValid())
		{
			FName ImageName = FName(FString("NiagaraEditor.Scalability.EffectType.") + PropertyName.ToString());
			const FSlateBrush* ImageBrush = FNiagaraEditorWidgetsStyle::Get().GetBrush(ImageName);

			TSharedPtr<SWidget> PropertyIconWidget = SNew(SImage)
				.Image(ImageBrush)
				.ToolTipText(PropIt->GetToolTipText());			
			
			EffectTypeValuesBox->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(2.f, 2.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3.f)
				[
					PropertyIconWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(2.f, 1.f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(PropertyDisplayNameText)
						.ToolTipText(PropIt->GetToolTipText())
						.TextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.Scalability.EffectType.Property"))
					]
					+ SVerticalBox::Slot()
					.Padding(2.f, 1.f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						PropertyValueWidget.ToSharedRef()
					]
				]
			];
		}
	}
}

TSharedPtr<SWidget> SNiagaraSystemEffectTypeBar::CreatePropertyValueWidget(FProperty* Property)
{
	TSharedRef<STextBlock> ValueTextBlock = SNew(STextBlock)
		.TextStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.Scalability.EffectType.Property"));

	if(!EffectType.IsValid())
	{
		ValueTextBlock->SetText(FText::FromString("-"));
		ValueTextBlock->SetColorAndOpacity(FStyleColors::AccentGray);
	}
	else
	{
		ValueTextBlock->SetColorAndOpacity(FStyleColors::AccentBlue);

		if(FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			int32 EnumValue = *Property->ContainerPtrToValuePtr<int32>(EffectType.Get());
			ValueTextBlock->SetText(EnumProperty->GetEnum()->GetDisplayNameTextByValue(EnumValue));
		}

		if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			bool BoolValue = BoolProperty->GetPropertyValue_InContainer(EffectType.Get());
			ValueTextBlock->SetText(BoolValue ? FText::FromString("True") : FText::FromString("False"));
		}
				
		if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			const TObjectPtr<UObject>& ObjectPtr = ObjectProperty->GetPropertyValue_InContainer(EffectType.Get());
			
			if(ObjectPtr)
			{
				if(ObjectProperty->HasMetaData("DisplayClassDisplayName"))
				{
					ValueTextBlock->SetText(GetClassDisplayName(ObjectPtr.Get()));
				}
				else
				{
					ValueTextBlock->SetText(GetObjectName(ObjectPtr.Get()));
				}
			}
			else
			{
				UObject* NullObject = nullptr;
				ValueTextBlock->SetText(GetObjectName(NullObject));
				ValueTextBlock->SetColorAndOpacity(FStyleColors::AccentGray);
			}
		}
	}
	
	return ValueTextBlock;
}

FText SNiagaraSystemEffectTypeBar::GetObjectName(UObject* Object) const
{
	return IsValid(Object) ? FText::FromString(Object->GetName()) : LOCTEXT("NoObjectAssigned", "None");		
}

FText SNiagaraSystemEffectTypeBar::GetClassDisplayName(UObject* Object) const
{
	return IsValid(Object) ? Object->GetClass()->GetDisplayNameText() : LOCTEXT("NoClassAvailable", "None");
}

int32 SNiagaraSystemEffectTypeBar::GetActiveDetailsWidgetIndex() const
{
	if(System->GetEffectType())
	{
		return 1;
	}

	return 0;
}

void SNiagaraSystemEffectTypeBar::UpdateEffectType()
{
	EffectType = System->GetEffectType();
	UpdateEffectTypeWidgets();
}

#undef LOCTEXT_NAMESPACE
