// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SCompoundWidget.h"

class SNiagaraSystemEffectTypeBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSystemEffectTypeBar)
	{}
	SLATE_END_ARGS()

	virtual ~SNiagaraSystemEffectTypeBar();
	
	void Construct(const FArguments& InArgs, UNiagaraSystem& InSystem);
	void UpdateEffectTypeWidgets();
private:
	TSharedPtr<SWidget> CreatePropertyValueWidget(FProperty* Property);

private:
	FText GetObjectName(UObject* Object) const;
	FText GetClassDisplayName(UObject* Object) const;
	int32 GetActiveDetailsWidgetIndex() const;
	void UpdateEffectType();
private:
	TWeakObjectPtr<UNiagaraEffectType> EffectType;
	TWeakObjectPtr<UNiagaraSystem> System;
	TSharedPtr<SWrapBox> EffectTypeValuesBox;
};
