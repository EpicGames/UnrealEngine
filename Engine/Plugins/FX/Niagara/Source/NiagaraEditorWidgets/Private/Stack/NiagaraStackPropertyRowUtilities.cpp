// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackPropertyRowUtilities.h"

#include "Customizations/NiagaraStackObjectPropertyCustomization.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailTreeNode.h"
#include "PropertyHandle.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"

SNiagaraStackTableRow::FOnFillRowContextMenu FNiagaraStackPropertyRowUtilities::CreateOnFillRowContextMenu(UNiagaraStackPropertyRow& PropertyRow, const FNodeWidgetActions& GeneratedPropertyNodeWidgetActions)
{
	return SNiagaraStackTableRow::FOnFillRowContextMenu::CreateStatic(&OnFillPropertyRowContextMenu, TWeakObjectPtr<UNiagaraStackPropertyRow>(&PropertyRow), GeneratedPropertyNodeWidgetActions);
}

void FNiagaraStackPropertyRowUtilities::OnFillPropertyRowContextMenu(FMenuBuilder& MenuBuilder, TWeakObjectPtr<UNiagaraStackPropertyRow> PropertyRowWeak, FNodeWidgetActions PropertyNodeWidgetActions)
{
	FUIAction CopyAction;
	FUIAction PasteAction;

	UNiagaraStackPropertyRow* PropertyRow = PropertyRowWeak.Get();
	if (PropertyRow == nullptr)
	{
		return;
	}

	// Allow the property customization to contribute context menu actions regardless of copy/paste support.
	if (TSharedPtr<const FNiagaraStackObjectPropertyCustomization> Customization = PropertyRow->GetCustomization())
	{
		Customization->GenerateContextMenuActions(PropertyRow, MenuBuilder);
	}

	if (PropertyRow->SupportsCopy() && PropertyRow->SupportsPaste())
	{
		// If the row supports stack copy, don't add the actions from the property editor.
		return;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
	if (PropertyNodeWidgetActions.CopyMenuAction.ExecuteAction.IsBound() && PropertyNodeWidgetActions.PasteMenuAction.ExecuteAction.IsBound())
	{
		CopyAction = PropertyNodeWidgetActions.CopyMenuAction;
		PasteAction = PropertyNodeWidgetActions.PasteMenuAction;
	}
	else
	{
		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			PropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);
		}
	}

	bool bEditSectionAdded = false;
	if (CopyAction.ExecuteAction.IsBound() && PasteAction.ExecuteAction.IsBound())
	{
		MenuBuilder.BeginSection(NAME_None, NSLOCTEXT("NiagaraStackPropertyRowUtilities", "PropertyRowEditMenuLabel", "Edit"));
		{
			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("NiagaraStackPropertyRowUtilities", "CopyProperty", "Copy"),
				NSLOCTEXT("NiagaraStackPropertyRowUtilities", "CopyPropertyToolTip", "Copy this property value"),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				CopyAction);

			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("NiagaraStackPropertyRowUtilities", "PasteProperty", "Paste"),
				NSLOCTEXT("NiagaraStackPropertyRowUtilities", "PastePropertyToolTip", "Paste the copied value here"),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
				PasteAction);
		}
		MenuBuilder.EndSection();
		bEditSectionAdded = true;
	}

	if (bEditSectionAdded && PropertyNodeWidgetActions.CustomMenuItems.Num() > 0)
	{
		MenuBuilder.AddSeparator();
	}

	for (const FNodeWidgetActionsCustomMenuData& CustomMenuData : PropertyNodeWidgetActions.CustomMenuItems)
	{
		MenuBuilder.AddMenuEntry(CustomMenuData.Name, CustomMenuData.Tooltip, CustomMenuData.SlateIcon, CustomMenuData.Action);
	}
}