// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailTreeNode.h"
#include "Stack/SNiagaraStackTableRow.h"
#include "Templates/SharedPointer.h"

class FMenuBuilder;
class UNiagaraStackPropertyRow;

class FNiagaraStackPropertyRowUtilities
{
public:
	static SNiagaraStackTableRow::FOnFillRowContextMenu CreateOnFillRowContextMenu(UNiagaraStackPropertyRow& PropertyRow, const FNodeWidgetActions& GeneratedPropertyNodeWidgetActions);

private:
	static void OnFillPropertyRowContextMenu(FMenuBuilder& MenuBuilder, TWeakObjectPtr<UNiagaraStackPropertyRow> PropertyRowWeak, FNodeWidgetActions PropertyNodeWidgetActions);
};