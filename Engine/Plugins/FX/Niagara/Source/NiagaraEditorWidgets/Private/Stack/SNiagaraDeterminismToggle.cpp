// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraDeterminismToggle.h"

#include "NiagaraEditorStyle.h"
#include "NiagaraSystem.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "Widgets/Text/STextBlock.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SNiagaraDeterminismToggle"

void SNiagaraDeterminismToggle::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	EmitterHandleViewModel = InEmitterHandleViewModel;

	if(EmitterHandleViewModel.IsValid() == false)
	{
		return;
	}
	
	FProperty* MatchingProperty = nullptr;
	for(TFieldIterator<FProperty> It(FVersionedNiagaraEmitterData::StaticStruct()); It; ++It)
	{
		if(*It->GetName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bDeterminism))
		{
			MatchingProperty = *It;
		}
	}

	if(ensure(MatchingProperty != nullptr))
	{
		PropertyDescription = MatchingProperty->GetToolTipText();
	}

	FSlateFontInfo FontInfo = FSlateFontInfo(FCoreStyle::GetDefaultFont(), 9, "Regular");
	FontInfo.OutlineSettings = FFontOutlineSettings(1, FLinearColor::White);
	
	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		.OnClicked(this, &SNiagaraDeterminismToggle::ToggleDeterminismInUI)
		.ToolTipText(this, &SNiagaraDeterminismToggle::GetDeterminismToggleTooltip)
		// Padding is tweaked to push the text a bit lower so it appears more central. Remove when exchanging for images
		.ContentPadding(FMargin(4.f, 3.f, 4.f, 1.f))
		[
			SNew(STextBlock)
			.Text(this, &SNiagaraDeterminismToggle::GetDeterminismButtonText)
			.Font(FontInfo)

			// SNew(SImage)
			// .Image(this, &SNiagaraDeterminismToggle::GetDeterminismImage)
		]
	];
}

void SNiagaraDeterminismToggle::ToggleDeterminism() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return;
	}
	
	FProperty* MatchingProperty = nullptr;
	for(TFieldIterator<FProperty> It(FVersionedNiagaraEmitterData::StaticStruct()); It; ++It)
	{
		if(*It->GetName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bDeterminism))
		{
			MatchingProperty = *It;
		}
	}
	ensure(MatchingProperty != nullptr);

	FScopedTransaction Transaction(LOCTEXT("ToggleDeterminismTransaction", "Determinism Changed"));
	EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().Emitter->PreEditChange(MatchingProperty);
	
	EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bDeterminism = !EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bDeterminism;

	FPropertyChangedEvent PropertyChangedEvent(MatchingProperty);
	UNiagaraSystem::RequestCompileForEmitter(EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter());
}

FReply SNiagaraDeterminismToggle::ToggleDeterminismInUI() const
{
	ToggleDeterminism();
	return FReply::Handled();
}

FText SNiagaraDeterminismToggle::GetDeterminismButtonText() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return FText::GetEmpty();
	}

	return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bDeterminism == true
		? FText::FromString("D")
		: FText::FromString("N D");
}

const FSlateBrush* SNiagaraDeterminismToggle::GetDeterminismImage() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return FAppStyle::GetNoBrush();
	}
	
	return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bDeterminism == true
		? FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Determinism.Enabled")
		: FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Determinism.Disabled");
}

FText SNiagaraDeterminismToggle::GetDeterminismToggleTooltip() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return FText::GetEmpty();
	}
	
	FText DeterminismEnabledText = EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bDeterminism == true
	   ? LOCTEXT("DeterminismEnabled", "Determinism Enabled")
	   : LOCTEXT("DeterminismDisabled", "Determinism Disabled");
	
	FText Tooltip = FText::FormatOrdered(FText::FromString("{0}\n{1}"), DeterminismEnabledText, PropertyDescription);
	return Tooltip;
}

#undef LOCTEXT_NAMESPACE
