// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraLocalSpaceToggle.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SNiagaraLocalSpaceToggle"

void SNiagaraLocalSpaceToggle::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	EmitterHandleViewModel = InEmitterHandleViewModel;

	if(EmitterHandleViewModel.IsValid() == false)
	{
		return;
	}

	FProperty* MatchingProperty = nullptr;
	for(TFieldIterator<FProperty> It(FVersionedNiagaraEmitterData::StaticStruct()); It; ++It)
	{
		if(*It->GetName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bLocalSpace))
		{
			MatchingProperty = *It;
		}
	}
	
	if(ensure(MatchingProperty != nullptr))
	{
		PropertyDescription = MatchingProperty->GetToolTipText();
	}
	
	ChildSlot
	[
		SNew(SButton)
		.OnClicked(this, &SNiagaraLocalSpaceToggle::ToggleLocalSpaceForUI)
		.ToolTipText(this, &SNiagaraLocalSpaceToggle::GetLocalSpaceToggleTooltip)
		.ContentPadding(1.f)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		[
			SNew(SImage)
			.Image(this, &SNiagaraLocalSpaceToggle::GetLocalSpaceToggleImage)
		]
	];
}

void SNiagaraLocalSpaceToggle::ToggleLocalSpace() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return;
	}
	
	FProperty* MatchingProperty = nullptr;
	for(TFieldIterator<FProperty> It(FVersionedNiagaraEmitterData::StaticStruct()); It; ++It)
	{
		if(*It->GetName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bLocalSpace))
		{
			MatchingProperty = *It;
		}
	}
	ensure(MatchingProperty != nullptr);

	FScopedTransaction Transaction(LOCTEXT("ToggleLocalSpaceTransaction", "Local/World Space Toggled"));
	EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().Emitter->PreEditChange(MatchingProperty);
	
	EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bLocalSpace = !EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bLocalSpace;

	FPropertyChangedEvent PropertyChangedEvent(MatchingProperty);
	EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().Emitter->PostEditChangeProperty(PropertyChangedEvent);
}

FReply SNiagaraLocalSpaceToggle::ToggleLocalSpaceForUI() const
{
	ToggleLocalSpace();
	return FReply::Handled();
}

const FSlateBrush* SNiagaraLocalSpaceToggle::GetLocalSpaceToggleImage() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return FAppStyle::GetNoBrush();
	}
	
	return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bLocalSpace ? FAppStyle::GetBrush("Icons.Transform") : FAppStyle::GetBrush("EditorViewport.RelativeCoordinateSystem_World");  
}

FText SNiagaraLocalSpaceToggle::GetLocalSpaceToggleTooltip() const
{
	if(EmitterHandleViewModel.IsValid() == false)
	{
		return FText::GetEmpty();
	}

	FText ToggleTooltip = EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEmitter().GetEmitterData()->bLocalSpace
		? LOCTEXT("ToggleLocalSpaceButtonTooltip_IsLocal", "This emitter operates in local space.")
		: LOCTEXT("ToggleLocalSpaceButtonTooltip_IsWorld", "This emitter operates in world space.");
	FText Tooltip = FText::FormatOrdered(FText::FromString("{0}\n{1}"), ToggleTooltip, PropertyDescription);
	
	return Tooltip;
}

#undef LOCTEXT_NAMESPACE
