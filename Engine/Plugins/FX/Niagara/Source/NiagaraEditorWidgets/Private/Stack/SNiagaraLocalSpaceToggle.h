// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"

class SNiagaraLocalSpaceToggle: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraLocalSpaceToggle) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);

private:
	void ToggleLocalSpace() const;
	FReply ToggleLocalSpaceForUI() const;

	const FSlateBrush* GetLocalSpaceToggleImage() const;
	FText GetLocalSpaceToggleTooltip() const;

private:
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;
	FText PropertyDescription;
};
