// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraSimTargetToggle.h"

#include "NiagaraEditorStyle.h"
#include "NiagaraSystem.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SNiagaraSimTargetToggle"

void SNiagaraSimTargetToggle::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	WeakEmitterHandleViewModel = InEmitterHandleViewModel;

	if (GetEmitterHandle() == nullptr)
	{
		return;
	}
	
	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		.OnClicked(this, &SNiagaraSimTargetToggle::ToggleSimTargetInUI)
		.ToolTipText(this, &SNiagaraSimTargetToggle::GetSimTargetToggleTooltip)
		[
			SNew(SImage)
			.Image(this, &SNiagaraSimTargetToggle::GetSimTargetImage)
		]
	];
}

void SNiagaraSimTargetToggle::ToggleSimTarget() const
{
	FNiagaraEmitterHandle* EmitterHandle = GetEmitterHandle();
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = WeakEmitterHandleViewModel.Pin();
	if ( !EmitterHandle || EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless || !EmitterHandleViewModel.IsValid() )
	{
		return;
	}

	FProperty* MatchingProperty = nullptr;
	for(TFieldIterator<FProperty> It(FVersionedNiagaraEmitterData::StaticStruct()); It; ++It)
	{
		if(*It->GetName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, SimTarget))
		{
			MatchingProperty = *It;
		}
	}
	ensure(MatchingProperty != nullptr);

	FScopedTransaction Transaction(LOCTEXT("ToggleSimTargetTransaction", "Simulation Target Changed"));
	EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().Emitter->PreEditChange(MatchingProperty);
	
	ENiagaraSimTarget CurrentSimTarget = EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->SimTarget;
	EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->SimTarget = CurrentSimTarget == ENiagaraSimTarget::CPUSim ?  ENiagaraSimTarget::GPUComputeSim : ENiagaraSimTarget::CPUSim;

	FPropertyChangedEvent PropertyChangedEvent(MatchingProperty);
	UNiagaraSystem::RequestCompileForEmitter(EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter());
}

FReply SNiagaraSimTargetToggle::ToggleSimTargetInUI() const
{
	ToggleSimTarget();
	return FReply::Handled();
}

const FSlateBrush* SNiagaraSimTargetToggle::GetSimTargetImage() const
{
	if (FNiagaraEmitterHandle* EmitterHandle = GetEmitterHandle())
	{
		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless)
		{
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.LWEIcon");
		}
		else
		{
			return EmitterHandle->GetInstance().GetEmitterData()->SimTarget == ENiagaraSimTarget::CPUSim
				? FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.CPUIcon")
				: FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.GPUIcon");
		}
	}
	return FAppStyle::GetNoBrush();
}

FText SNiagaraSimTargetToggle::GetSimTargetToggleTooltip() const
{
	if (FNiagaraEmitterHandle* EmitterHandle = GetEmitterHandle())
	{
		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless)
		{
			return LOCTEXT("ToggleSimTargetTooltip_LWE", "Lightweight emitter.");
		}
		else if (FVersionedNiagaraEmitterData* VersionedEmitterData = EmitterHandle->GetEmitterData())
		{
			return VersionedEmitterData->SimTarget == ENiagaraSimTarget::CPUSim
				? LOCTEXT("ToggleSimTargetTooltip_CPU", "CPU emitter. Clicking will turn this emitter into a GPU emitter and recompile.")
				: LOCTEXT("ToggleSimTargetTooltip_GPU", "GPU emitter. Clicking will turn this emitter into a CPU emitter and recompile.");		
		}
	}
	return FText::GetEmpty();
}

FNiagaraEmitterHandle* SNiagaraSimTargetToggle::GetEmitterHandle() const
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = WeakEmitterHandleViewModel.Pin();
	return EmitterHandleViewModel ? EmitterHandleViewModel->GetEmitterHandle() : nullptr;
}

#undef LOCTEXT_NAMESPACE
