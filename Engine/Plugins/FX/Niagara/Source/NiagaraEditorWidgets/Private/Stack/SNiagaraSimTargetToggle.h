// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"

class SNiagaraSimTargetToggle: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimTargetToggle) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);

private:
	void ToggleSimTarget() const;
	FReply ToggleSimTargetInUI() const;

	const FSlateBrush* GetSimTargetImage() const;
	FText GetSimTargetToggleTooltip() const;

	FNiagaraEmitterHandle* GetEmitterHandle() const;

private:
	TWeakPtr<FNiagaraEmitterHandleViewModel> WeakEmitterHandleViewModel;
};
