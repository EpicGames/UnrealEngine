// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackErrorItem.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorStyle.h"
#include "Styling/AppStyle.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SNiagaraStackErrorItem"

void SNiagaraStackErrorItem::Construct(const FArguments& InArgs, UNiagaraStackErrorItem* InErrorItem, UNiagaraStackViewModel* InStackViewModel)
{
	ErrorItem = InErrorItem;
	StackViewModel = InStackViewModel;
	const FSlateBrush* IconBrush;
	switch (ErrorItem->GetStackIssue().GetSeverity())
	{
	case EStackIssueSeverity::Error:
		IconBrush = FAppStyle::Get().GetBrush("Icons.ErrorWithColor");
		break;
	case EStackIssueSeverity::Warning:
		IconBrush = FAppStyle::Get().GetBrush("Icons.WarningWithColor");
		break;
	case EStackIssueSeverity::Info:
		IconBrush = FAppStyle::Get().GetBrush("Icons.InfoWithColor");
		break;
	default:
		IconBrush = FAppStyle::Get().GetBrush("NoBrush");
		break;
	}

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SImage)
			.Image(IconBrush)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.Text_UObject(ErrorItem, &UNiagaraStackErrorItem::GetDisplayName)
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
		]
	];
}

void SNiagaraStackErrorItemFix::Construct(const FArguments& InArgs, UNiagaraStackErrorItemFix* InErrorItem, UNiagaraStackViewModel* InStackViewModel)
{
	ErrorItem = InErrorItem;
	StackViewModel = InStackViewModel;

	TSharedPtr<SWidget> FixWidget;

	if (ErrorItem->GetStackIssueFix().GetStyle() == UNiagaraStackEntry::EStackIssueFixStyle::Fix)
	{
		FixWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.Text_UObject(ErrorItem, &UNiagaraStackErrorItemFix::GetDisplayName)
			.ColorAndOpacity(this, &SNiagaraStackErrorItemFix::GetTextColorForSearch, FSlateColor::UseForeground())
			.HighlightText_UObject(StackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.AutoWrapText(true)
		]
		+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.Text_UObject(ErrorItem, &UNiagaraStackErrorItemFix::GetFixButtonText)
			.OnClicked_UObject(ErrorItem, &UNiagaraStackErrorItemFix::OnTryFixError)
		];
	}
	else if(ErrorItem->GetStackIssueFix().GetStyle() == UNiagaraStackEntry::EStackIssueFixStyle::Link)
	{
		FixWidget = SNew(SBox)
		.HAlign(HAlign_Left)
		[
			SNew(SHyperlink)
			.Text_UObject(ErrorItem, &UNiagaraStackErrorItemFix::GetDisplayName)
			.OnNavigate(this, &SNiagaraStackErrorItemFix::LinkNavigate)
		];
	}
	else
	{
		FixWidget = SNullWidget::NullWidget;
	}

	ChildSlot
	[
		FixWidget.ToSharedRef()
	];
}

void SNiagaraStackErrorItemFix::LinkNavigate()
{
	ErrorItem->OnTryFixError();
}

#undef LOCTEXT_NAMESPACE //"SNiagaraStackErrorItem"

