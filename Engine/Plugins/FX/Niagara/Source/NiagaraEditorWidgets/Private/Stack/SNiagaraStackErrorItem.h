// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraStackEntryWidget.h"

class UNiagaraStackErrorItem;
class UNiagaraStackErrorItemLongDescription;
class UNiagaraStackErrorItemFix;
class UNiagaraStackErrorItemDismiss;

class SNiagaraStackErrorItem: public SNiagaraStackEntryWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackErrorItem) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackErrorItem* InErrorItem, UNiagaraStackViewModel* InStackViewModel);

private:
	UNiagaraStackErrorItem* ErrorItem;
};

class SNiagaraStackErrorItemFix : public SNiagaraStackEntryWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackErrorItemFix) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackErrorItemFix* InErrorItem, UNiagaraStackViewModel* InStackViewModel);

private:
	void LinkNavigate();

private:
	UNiagaraStackErrorItemFix* ErrorItem;
};