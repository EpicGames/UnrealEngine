// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackFunctionInputName.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorStyle.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SNiagaraParameterName.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorUtilities.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "NiagaraStackFunctionInputName"

void SNiagaraStackFunctionInputName::Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput, UNiagaraStackViewModel* InStackViewModel)
{
	FunctionInput = InFunctionInput;
	StackViewModel = InStackViewModel;
	StackEntryItem = InFunctionInput;
	IsSelected = InArgs._IsSelected;

	FunctionInput->OnStructureChanged().AddSP(this, &SNiagaraStackFunctionInputName::InputStructureChanged);

	TSharedPtr<SHorizontalBox> NameBox;
	ChildSlot
	[
		SAssignNew(NameBox, SHorizontalBox)
		.IsEnabled_UObject(FunctionInput, &UNiagaraStackEntry::GetOwnerIsEnabled)
	];

	TOptional<FNiagaraVariable> EditVariable = FunctionInput->GetEditConditionVariable();

	TSharedRef<SCheckBox> InlineEnableToggle = SNew(SCheckBox)
		.Visibility(this, &SNiagaraStackFunctionInputName::GetEditConditionCheckBoxVisibility)
		.IsChecked(this, &SNiagaraStackFunctionInputName::GetEditConditionCheckState)
		.OnCheckStateChanged(this, &SNiagaraStackFunctionInputName::OnEditConditionCheckStateChanged);
		
	if(FunctionInput->GetHasEditCondition() && EditVariable.IsSet())
	{
		TSharedRef<SToolTip> Tooltip = FNiagaraParameterUtilities::GetTooltipWidget(EditVariable.GetValue(), false);
		InlineEnableToggle->SetToolTip(Tooltip);
	}
	
	// Edit condition checkbox
	NameBox->AddSlot()
	.AutoWidth()
	.Padding(0, 0, 3, 0)
	[
		InlineEnableToggle
	];
	
	// Name Label
	if(FunctionInput->GetInputFunctionCallNode().IsA<UNiagaraNodeAssignment>())
	{
		NameBox->AddSlot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(ParameterTextBlock, SNiagaraParameterNameTextBlock)
			.EditableTextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterInlineEditableText")
			.ParameterText_UObject(FunctionInput, &UNiagaraStackEntry::GetDisplayName)
			.IsReadOnly(FunctionInput->SupportsRename() == false)
			.IsEnabled(this, &SNiagaraStackFunctionInputName::GetIsEnabled)
			.IsSelected(this, &SNiagaraStackFunctionInputName::GetIsNameWidgetSelected)
			.OnTextCommitted(this, &SNiagaraStackFunctionInputName::OnNameTextCommitted)
			.HighlightText_UObject(InStackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.ToolTipText(this, &SNiagaraStackFunctionInputName::GetToolTipText)
		];
	}
	else
	{
		NameBox->AddSlot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(NameTextBlock, SInlineEditableTextBlock)
			.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterInlineEditableText")
			.Text_UObject(FunctionInput, &UNiagaraStackEntry::GetDisplayName)
			.IsReadOnly(this, &SNiagaraStackFunctionInputName::GetIsNameReadOnly)
			.IsEnabled(this, &SNiagaraStackFunctionInputName::GetIsEnabled)
			.IsSelected(this, &SNiagaraStackFunctionInputName::GetIsNameWidgetSelected)
			.OnTextCommitted(this, &SNiagaraStackFunctionInputName::OnNameTextCommitted)
			.HighlightText_UObject(InStackViewModel, &UNiagaraStackViewModel::GetCurrentSearchText)
			.ToolTipText(this, &SNiagaraStackFunctionInputName::GetToolTipText)
		];
	}

	if (FunctionInput->GetInputType().IsStatic())
	{
		NameBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.ToolTipText(LOCTEXT("TypeModifierTooltip", "This variable is Static and cannot be changed at runtime, or linked to any\n dynamic attributes. Static variables can be used to reduce script instructions & memory."))
			.Padding(3, 2, 1, 2)
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.StaticInputIcon"))
			]
		];
	}
}

void SNiagaraStackFunctionInputName::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (FunctionInput->GetIsRenamePending())
	{
		if (NameTextBlock.IsValid())
		{
			NameTextBlock->EnterEditingMode();
		}
		else if (ParameterTextBlock.IsValid())
		{
			ParameterTextBlock->EnterEditingMode();
		}
		FunctionInput->SetIsRenamePending(false);
	}
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SNiagaraStackFunctionInputName::FillRowContextMenu(class FMenuBuilder& MenuBuilder)
{
	if (FunctionInput->GetInputFunctionCallNode().IsA<UNiagaraNodeAssignment>())
	{
		MenuBuilder.BeginSection("Parameter", LOCTEXT("ParameterHeader", "Parameter"));
		{
			TAttribute<FText> CopyReferenceToolTip;
			CopyReferenceToolTip.Bind(this, &SNiagaraStackFunctionInputName::GetCopyParameterReferenceToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CopyParameterReference", "Copy Reference"),
				CopyReferenceToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnCopyParameterReference),
					FCanExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::CanCopyParameterReference)));

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespace", "Namespace"),
				LOCTEXT("ChangeNamespaceToolTip", "Select a new namespace for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &SNiagaraStackFunctionInputName::GetChangeNamespaceSubMenu));

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespaceModifier", "Namespace Modifier"),
				LOCTEXT("ChangeNamespaceModifierToolTip", "Edit the namespace modifier for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &SNiagaraStackFunctionInputName::GetChangeNamespaceModifierSubMenu));

			if (FunctionInput->CanDeleteInput())
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("DeleteInput", "Remove this parameter"),
					LOCTEXT("DeleteInputToolTip", "Remove this parameter from set parameters node."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
					FUIAction(
						FNiagaraMenuAction_Generic::FOnExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::DeleteInput),
						FNiagaraMenuAction_Generic::FCanExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::CanDeleteInput)));
			}
		}
		MenuBuilder.EndSection();
	}

	{
		TArray<FNiagaraStackFunctionInputActions> CustomActions = FunctionInput->GetCustomActions();
		if (CustomActions.Num() > 0)
		{
			MenuBuilder.BeginSection("InputActions", LOCTEXT("InputActions", "Input Actions"));
			for (const FNiagaraStackFunctionInputActions& Action : CustomActions)
			{
				MenuBuilder.AddMenuEntry(Action.Label, Action.ToolTip, FSlateIcon(), FUIAction(Action.ExecuteAction));
			}
			MenuBuilder.EndSection();
		}
	}

	if (FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic && GbEnableExperimentalInlineDynamicInputs)
	{
		MenuBuilder.BeginSection("View", LOCTEXT("ViewHeader", "View"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UseDefaultDisplay", "Use Default Display"),
				LOCTEXT("UseDefaultDisplayToolTip", "Use the default multi-row tree view display for this input."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetUseDefaultDisplay),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SNiagaraStackFunctionInputName::GetUseDefaultDisplayCheckboxState)),
				NAME_None,
				EUserInterfaceActionType::RadioButton);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("UseInlineExpressionDisplay", "Use Inline Expression Display"),
				LOCTEXT("UseInlineExpressionDisplayToolTip", "Change the format of this input value to display inputs as an editable inline expression."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetUseInlineExpressionDisplay),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SNiagaraStackFunctionInputName::GetUseInlineExpressionDisplayCheckboxState)),
				NAME_None,
				EUserInterfaceActionType::RadioButton);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("UseInlineHorizontalGraphDisplay", "Use Inline Horizontal Graph Display"),
				LOCTEXT("UseInlineHorizontalGraphDisplayToolTip", "Change the format of this input value to display dynamic inputs as a horizontal inline graph."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetUseInlineHorizontalGraphDisplay),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SNiagaraStackFunctionInputName::GetUseInlineHorizontalGraphDisplayCheckboxState)),
				NAME_None,
				EUserInterfaceActionType::RadioButton);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("UseInlineVerticalGraphDisplay", "Use Inline Vertical Graph Display"),
				LOCTEXT("UseInlineVerticalGraphDisplayToolTip", "Change the format of this input value to display dynamic inputs as a vertical inline graph."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetUseInlineVerticalGraphDisplay),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SNiagaraStackFunctionInputName::GetUseInlineVerticalGraphDisplayCheckboxState)),
				NAME_None,
				EUserInterfaceActionType::RadioButton);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("UseInlineHybridGraphDisplay", "Use Inline Hybrid Graph Display"),
				LOCTEXT("UseInlineHybridGraphDisplayToolTip", "Change the format of this input value to display dynamic inputs as a hybrid inline graph."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetUseInlineHybridGraphDisplay),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SNiagaraStackFunctionInputName::GetUseInlineHybridGraphDisplayCheckboxState)),
				NAME_None,
				EUserInterfaceActionType::RadioButton);
		}
		MenuBuilder.EndSection();
	}
}

FSimpleMulticastDelegate& SNiagaraStackFunctionInputName::OnRequestReconstructRow()
{
	return OnRequestReconstructRowDelegate;
}

EVisibility SNiagaraStackFunctionInputName::GetEditConditionCheckBoxVisibility() const
{
	return FunctionInput->GetHasEditCondition() && FunctionInput->GetShowEditConditionInline() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SNiagaraStackFunctionInputName::GetEditConditionCheckState() const
{
	return FunctionInput->GetHasEditCondition() && FunctionInput->GetEditConditionEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackFunctionInputName::OnEditConditionCheckStateChanged(ECheckBoxState InCheckState)
{
	FunctionInput->SetEditConditionEnabled(InCheckState == ECheckBoxState::Checked);
}

bool SNiagaraStackFunctionInputName::GetIsNameReadOnly() const
{
	return FunctionInput->SupportsRename() == false;
}

bool SNiagaraStackFunctionInputName::GetIsNameWidgetSelected() const
{
	return IsSelected.Get();
}

bool SNiagaraStackFunctionInputName::GetIsEnabled() const
{
	return FunctionInput->GetOwnerIsEnabled() && (FunctionInput->GetHasEditCondition() == false || FunctionInput->GetEditConditionEnabled());
}

FText SNiagaraStackFunctionInputName::GetToolTipText() const
{
	// The engine ticks tooltips before widgets so it's possible for the footer to be finalized when
	// the widgets haven't been recreated.
	if (FunctionInput->IsFinalized())
	{
		return FText();
	}
	return FunctionInput->GetTooltipText();
}

void SNiagaraStackFunctionInputName::OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	FunctionInput->OnRenamed(InText);
}

void SNiagaraStackFunctionInputName::GetChangeNamespaceSubMenu(FMenuBuilder& MenuBuilder)
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();

	TArray<FName> NamespacesForWriteParameters;
	FunctionInput->GetNamespacesForNewWriteParameters(NamespacesForWriteParameters);

	TArray<FNiagaraParameterUtilities::FChangeNamespaceMenuData> MenuData;
	FNiagaraParameterUtilities::GetChangeNamespaceMenuData(*FunctionInput->GetDisplayName().ToString(), FNiagaraParameterUtilities::EParameterContext::System, MenuData);
	for (const FNiagaraParameterUtilities::FChangeNamespaceMenuData& MenuDataItem : MenuData)
	{
		if (MenuDataItem.Metadata.Namespaces.Num() == 1 && NamespacesForWriteParameters.Contains(MenuDataItem.Metadata.Namespaces[0]))
		{
			bool bCanChange = MenuDataItem.bCanChange;
			FUIAction Action = FUIAction(
				FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnChangeNamespace, MenuDataItem.Metadata),
				FCanExecuteAction::CreateLambda([bCanChange]() { return bCanChange; }));

			TSharedRef<SWidget> MenuItemWidget = FNiagaraParameterUtilities::CreateNamespaceMenuItemWidget(MenuDataItem.NamespaceParameterName, MenuDataItem.CanChangeToolTip);
			MenuBuilder.AddMenuEntry(Action, MenuItemWidget, NAME_None, MenuDataItem.CanChangeToolTip);
		}
	}
}

void SNiagaraStackFunctionInputName::OnChangeNamespace(FNiagaraNamespaceMetadata Metadata)
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FName NewName = FNiagaraParameterUtilities::ChangeNamespace(InputParameterName, Metadata);
	if (NewName != NAME_None)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeNamespaceTransaction", "Change namespace"));
		FunctionInput->OnRenamed(FText::FromName(NewName));
	}
}

void SNiagaraStackFunctionInputName::GetChangeNamespaceModifierSubMenu(FMenuBuilder& MenuBuilder)
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	TArray<FName> OptionalNamespaceModifiers;
	FNiagaraParameterUtilities::GetOptionalNamespaceModifiers(InputParameterName, FNiagaraParameterUtilities::EParameterContext::System, OptionalNamespaceModifiers);
	for (FName OptionalNamespaceModifier : OptionalNamespaceModifiers)
	{
		TAttribute<FText> SetToolTip;
		SetToolTip.Bind(TAttribute<FText>::FGetter::CreateSP(this, &SNiagaraStackFunctionInputName::GetSetNamespaceModifierToolTip, OptionalNamespaceModifier));
		MenuBuilder.AddMenuEntry(
			FText::FromName(OptionalNamespaceModifier),
			SetToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetNamespaceModifier, OptionalNamespaceModifier),
				FCanExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::CanSetNamespaceModifier, OptionalNamespaceModifier)));
	}

	TAttribute<FText> SetCustomToolTip;
	SetCustomToolTip.Bind(TAttribute<FText>::FGetter::CreateSP(this, &SNiagaraStackFunctionInputName::GetSetCustomNamespaceModifierToolTip));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomNamespaceModifier", "Custom..."),
		SetCustomToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetCustomNamespaceModifier),
			FCanExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::CanSetCustomNamespaceModifier)));

	TAttribute<FText> SetNoneToolTip;
	SetNoneToolTip.Bind(TAttribute<FText>::FGetter::CreateSP(this, &SNiagaraStackFunctionInputName::GetSetNamespaceModifierToolTip, FName(NAME_None)));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NoneNamespaceModifier", "Clear"),
		SetNoneToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::OnSetNamespaceModifier, FName(NAME_None)),
			FCanExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputName::CanSetNamespaceModifier, FName(NAME_None))));
}

FText SNiagaraStackFunctionInputName::GetSetNamespaceModifierToolTip(FName NamespaceModifier) const
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FText AddMessage;
	FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(InputParameterName, NamespaceModifier, AddMessage);
	return AddMessage;
}

bool SNiagaraStackFunctionInputName::CanSetNamespaceModifier(FName NamespaceModifier) const
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FText Unused;
	return FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(InputParameterName, NamespaceModifier, Unused);
}

void SNiagaraStackFunctionInputName::OnSetNamespaceModifier(FName NamespaceModifier)
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FName NewName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(InputParameterName, NamespaceModifier);
	if (NewName != NAME_None)
	{
		FScopedTransaction Transaction(LOCTEXT("AddNamespaceModifierTransaction", "Add namespace modifier."));
		FunctionInput->OnRenamed(FText::FromName(NewName));
	}
}

FText SNiagaraStackFunctionInputName::GetSetCustomNamespaceModifierToolTip() const
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FText AddMessage;
	FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(InputParameterName, AddMessage);
	return AddMessage;
}

bool SNiagaraStackFunctionInputName::CanSetCustomNamespaceModifier() const
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FText Unused;
	return FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(InputParameterName, Unused);
}

void SNiagaraStackFunctionInputName::OnSetCustomNamespaceModifier()
{
	FName InputParameterName = *FunctionInput->GetDisplayName().ToString();
	FName NewName = FNiagaraParameterUtilities::SetCustomNamespaceModifier(InputParameterName);
	if (NewName != NAME_None)
	{
		if (NewName != InputParameterName)
		{
			FScopedTransaction Transaction(LOCTEXT("AddNamespaceModifierTransaction", "Add namespace modifier."));
			FunctionInput->OnRenamed(FText::FromName(NewName));
		}
		if (ParameterTextBlock.IsValid())
		{
			ParameterTextBlock->EnterNamespaceModifierEditingMode();
		}
	}
}

FText SNiagaraStackFunctionInputName::GetCopyParameterReferenceToolTip() const
{
	return LOCTEXT("CopyReferenceToolTip", "Copy a string reference for this parameter to the clipboard.\nThis reference can be used in expressions and custom HLSL nodes.");
}

bool SNiagaraStackFunctionInputName::CanCopyParameterReference() const
{
	return true;
}

void SNiagaraStackFunctionInputName::OnCopyParameterReference()
{
	FPlatformApplicationMisc::ClipboardCopy(*FunctionInput->GetDisplayName().ToString());
}

ECheckBoxState SNiagaraStackFunctionInputName::GetUseDefaultDisplayCheckboxState() const
{
	return FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::None ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackFunctionInputName::OnSetUseDefaultDisplay()
{
	FunctionInput->SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode::None);
	OnRequestReconstructRowDelegate.Broadcast();
}

ECheckBoxState SNiagaraStackFunctionInputName::GetUseInlineExpressionDisplayCheckboxState() const
{
	return FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::Expression ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackFunctionInputName::OnSetUseInlineExpressionDisplay()
{
	FunctionInput->SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode::Expression);
	OnRequestReconstructRowDelegate.Broadcast();
}

ECheckBoxState SNiagaraStackFunctionInputName::GetUseInlineHorizontalGraphDisplayCheckboxState() const
{
	return FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHorizontal ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackFunctionInputName::OnSetUseInlineHorizontalGraphDisplay()
{
	FunctionInput->SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode::GraphHorizontal);
	OnRequestReconstructRowDelegate.Broadcast();
}

ECheckBoxState SNiagaraStackFunctionInputName::GetUseInlineVerticalGraphDisplayCheckboxState() const
{
	return FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphVertical ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackFunctionInputName::OnSetUseInlineVerticalGraphDisplay()
{
	FunctionInput->SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode::GraphVertical);
	OnRequestReconstructRowDelegate.Broadcast();
}

ECheckBoxState SNiagaraStackFunctionInputName::GetUseInlineHybridGraphDisplayCheckboxState() const
{
	return FunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHybrid ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackFunctionInputName::OnSetUseInlineHybridGraphDisplay()
{
	FunctionInput->SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode::GraphHybrid);
	OnRequestReconstructRowDelegate.Broadcast();
}

void SNiagaraStackFunctionInputName::InputStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	if (FunctionInput != nullptr && FunctionInput->IsFinalized() == false && FunctionInput->GetInlineDisplayMode() != ENiagaraStackEntryInlineDisplayMode::None)
	{
		OnRequestReconstructRowDelegate.Broadcast();
	}
}

#undef LOCTEXT_NAMESPACE
