// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraStackEntryWidget.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"
#include "NiagaraEditorSettings.h"

class UNiagaraStackFunctionInput;
class SInlineEditableTextBlock;
class SNiagaraParameterNameTextBlock;

class SNiagaraStackFunctionInputName: public SNiagaraStackEntryWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackFunctionInputName) { }
		SLATE_ATTRIBUTE(bool, IsSelected);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput, UNiagaraStackViewModel* InStackViewModel);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void FillRowContextMenu(class FMenuBuilder& MenuBuilder);

	FSimpleMulticastDelegate& OnRequestReconstructRow();

private:
	EVisibility GetEditConditionCheckBoxVisibility() const;

	ECheckBoxState GetEditConditionCheckState() const;

	void OnEditConditionCheckStateChanged(ECheckBoxState InCheckState);

	bool GetIsNameReadOnly() const;

	bool GetIsNameWidgetSelected() const;

	bool GetIsEnabled() const;

	FText GetToolTipText() const;

	void OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	void GetChangeNamespaceSubMenu(FMenuBuilder& MenuBuilder);
	void OnChangeNamespace(FNiagaraNamespaceMetadata Metadata);

	void GetChangeNamespaceModifierSubMenu(FMenuBuilder& MenuBuilder);

	FText GetSetNamespaceModifierToolTip(FName InNamespaceModifier) const;
	bool CanSetNamespaceModifier(FName InNamespaceModifier) const;
	void OnSetNamespaceModifier(FName InNamespaceModifier);

	FText GetSetCustomNamespaceModifierToolTip() const;
	bool CanSetCustomNamespaceModifier() const;
	void OnSetCustomNamespaceModifier();

	FText GetCopyParameterReferenceToolTip() const;
	bool CanCopyParameterReference() const;
	void OnCopyParameterReference();

	ECheckBoxState GetUseDefaultDisplayCheckboxState() const;
	void OnSetUseDefaultDisplay();

	ECheckBoxState GetUseInlineExpressionDisplayCheckboxState() const;
	void OnSetUseInlineExpressionDisplay();

	ECheckBoxState GetUseInlineHorizontalGraphDisplayCheckboxState() const;
	void OnSetUseInlineHorizontalGraphDisplay();

	ECheckBoxState GetUseInlineVerticalGraphDisplayCheckboxState() const;
	void OnSetUseInlineVerticalGraphDisplay();

	ECheckBoxState GetUseInlineHybridGraphDisplayCheckboxState() const;
	void OnSetUseInlineHybridGraphDisplay();

	void InputStructureChanged(ENiagaraStructureChangedFlags Flags);

private:
	UNiagaraStackFunctionInput* FunctionInput;

	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;

	TSharedPtr<SNiagaraParameterNameTextBlock> ParameterTextBlock;

	TAttribute<bool> IsSelected;

	FSimpleMulticastDelegate OnRequestReconstructRowDelegate;
};