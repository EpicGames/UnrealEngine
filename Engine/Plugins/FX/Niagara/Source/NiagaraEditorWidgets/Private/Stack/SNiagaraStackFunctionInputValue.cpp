// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackFunctionInputValue.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "INiagaraEditorTypeUtilities.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "NiagaraActions.h"
#include "NiagaraCommon.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraStackCommandContext.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraTrace.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "SDropTarget.h"
#include "SNiagaraGraphActionWidget.h"
#include "SNiagaraParameterDropTarget.h"
#include "SNiagaraParameterEditor.h"
#include "ScopedTransaction.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "Stack/SNiagaraStackIndent.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackSummaryViewInputCollection.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/NiagaraHLSLSyntaxHighlighter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "NiagaraStackFunctionInputValue"

const float TextIconSize = 16;

bool SNiagaraStackFunctionInputValue::bLibraryOnly = true;
FName SNiagaraStackFunctionInputValue::FavoriteActionsProfile = TEXT("StackFunctionInputActionsProfile");

class SNiagaraStackFunctionInputTypeIcon : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackFunctionInputTypeIcon) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput)
	{ 
		TSharedRef<SImage> TypePill = SNew(SImage)
			.ColorAndOpacity(UEdGraphSchema_Niagara::GetTypeColor(InFunctionInput->GetInputType()))
			.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"));

		TSharedPtr<SWidget> TypeContent;
		UNiagaraStackFunctionInput::EValueMode InputValueMode = InFunctionInput->GetValueMode();
		if (InputValueMode == UNiagaraStackFunctionInput::EValueMode::Linked)
		{
			// Linked values don't need an icon since they have a custom parameter control.
			TypeContent = TypePill;
		}
		else
		{
			TSharedPtr<SWidget> TypeIcon;
			if (InputValueMode == UNiagaraStackFunctionInput::EValueMode::Dynamic)
			{
				TypeIcon = SNew(SImage)
				.ColorAndOpacity(FLinearColor::White)
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.DynamicInput"));
			}
			else
			{
				TypeIcon = SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FNiagaraStackEditorWidgetsUtilities::GetIconTextForInputMode(InFunctionInput->GetValueMode()));
			}

			TypeContent = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0)
			[
				TypePill
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(3, 2)
			[
				SNew(SBox)
				.HeightOverride(TextIconSize)
				.VAlign(VAlign_Center)
				[
					TypeIcon.ToSharedRef()
				]
			];
		}

		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.InputTypeBorder"))
			.Visibility(EVisibility::Visible)
			.Padding(2)
			[
				TypeContent.ToSharedRef()
			]);

		SetToolTipText(FText::Format(LOCTEXT("InputTypeIconToolTipFormat", "Mode: {0}\nType: {1}"),
			FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(InFunctionInput->GetValueMode()),
			InFunctionInput->GetInputType().GetNameText()));
	}
};

class SNiagaraStackFunctionLinkedSharedButton : public SComboButton
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackFunctionLinkedSharedButton) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput)
	{
		UNiagaraStackFunctionInput* LinkedFunctionInput = InFunctionInput->GetSharedLinkedValueInput();
		if (LinkedFunctionInput == nullptr)
		{
			return;
		}

		SComboButton::Construct(SComboButton::FArguments()
			.ComboButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FComboButtonStyle>("NiagaraEditor.Stack.CompactComboButtonWithBorder"))
			.OnGetMenuContent(this, &SNiagaraStackFunctionLinkedSharedButton::GetLinkedSharedActionMenu)
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(0.7f, .3f, 0.7f, 1.0f))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.LinkedSharedIcon"))
			]);

		SetToolTipText(FText::Format(LOCTEXT("LinkedSharedButtonToolTipFormat",
			"This input is is linked to a shared parameter {0}.\n"
			"Edits made here will modify that linked parameter."),
			FText::FromName(LinkedFunctionInput->GetInputParameterHandle().GetName())));

		FunctionInputWeak = InFunctionInput;
		LinkedFunctionInputWeak = LinkedFunctionInput;
	}

private:
	TSharedRef<SWidget> GetLinkedSharedActionMenu()
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		UNiagaraStackFunctionInput* LinkedFunctionInput = LinkedFunctionInputWeak.Get();
		if (LinkedFunctionInput != nullptr && LinkedFunctionInput->IsFinalized() == false)
		{
			MenuBuilder.BeginSection("LinkedSharedParameterActions", LOCTEXT("LinkedSharedSourceSection", "Linked Shared Parameter Actions"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("GoToLinkedSharedSource", "Go to source parameter"),
					LOCTEXT("GoToLinkedSharedSourceToolTip", "Selects the set parameters node that last set the shared source parameter."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackFunctionLinkedSharedButton::SelectLinkedSharedSourceModule)));
				MenuBuilder.AddMenuEntry(
					LOCTEXT("UnlinkFromSharedSource", "Unlink from source parameter."),
					LOCTEXT("UnlinkFromSharedSourceToolTip", "Unlinks this input from the shared source parameter while preserving the value."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackFunctionLinkedSharedButton::UnlinkFromSharedParameter)));
			}
		}
		return MenuBuilder.MakeWidget();
	}

	void SelectLinkedSharedSourceModule()
	{
		UNiagaraStackFunctionInput* LinkedFunctionInput = LinkedFunctionInputWeak.Get();
		if (LinkedFunctionInput != nullptr && LinkedFunctionInput->IsFinalized() == false)
		{
			LinkedFunctionInput->GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
			LinkedFunctionInput->GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(&LinkedFunctionInput->GetInputFunctionCallNode());
		}
	}

	void UnlinkFromSharedParameter()
	{
		UNiagaraStackFunctionInput* FunctionInput = FunctionInputWeak.Get();
		if (FunctionInput != nullptr && FunctionInput->IsFinalized() == false)
		{
			FunctionInput->UnlinkFromSharedParameter();
		}
	}

private:
	TWeakObjectPtr<UNiagaraStackFunctionInput> FunctionInputWeak;
	TWeakObjectPtr<UNiagaraStackFunctionInput> LinkedFunctionInputWeak;
};

void SNiagaraStackFunctionInputValue::Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput)
{
	FunctionInput = InFunctionInput;
	LayoutMode = InArgs._LayoutMode;
	CompactActionMenuButtonVisibilityAttribute = InArgs._CompactActionMenuButtonVisibility;
	FunctionInput->OnValueChanged().AddSP(this, &SNiagaraStackFunctionInputValue::OnInputValueChanged);
	SyntaxHighlighter = FNiagaraHLSLSyntaxHighlighter::Create();

	TAttribute<bool> EntryIsEnabled;
	EntryIsEnabled.Bind(this, &SNiagaraStackFunctionInputValue::GetEntryEnabled);
	SetEnabled(EntryIsEnabled);

	TSharedPtr<SHorizontalBox> ChildrenBox;
	ChildSlot
	[
			SNew(SNiagaraParameterDropTarget)
			.TypeToTestAgainst(FunctionInput->GetInputType())
			.ExecutionCategory(FunctionInput->GetExecutionCategoryName())
			.TargetParameter(FNiagaraVariable(FunctionInput->GetInputType(), FunctionInput->GetInputParameterHandle().GetParameterHandleString()))
			.DropTargetArgs(SDropTarget::FArguments()
			.OnAllowDrop(this, &SNiagaraStackFunctionInputValue::OnFunctionInputAllowDrop)
			.OnDropped(this, &SNiagaraStackFunctionInputValue::OnFunctionInputDrop)
			.HorizontalImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.DropTarget.BorderHorizontal"))
			.VerticalImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.DropTarget.BorderVertical"))
			.IsEnabled_UObject(FunctionInput, &UNiagaraStackEntry::GetOwnerIsEnabled)
			.bUseAllowDropCache(true)
			.Content()
			[
				SAssignNew(ChildrenBox, SHorizontalBox)
				.IsEnabled(this, &SNiagaraStackFunctionInputValue::GetInputEnabled)
			])
	];

	bool bAddIndent = LayoutMode == ELayoutMode::FullRow;
	bool bAddValue = LayoutMode != ELayoutMode::EditDropDownOnly;
	bool bAddEditDropDown = LayoutMode == ELayoutMode::FullRow || LayoutMode == ELayoutMode::LinkedInline;
	bool bAddCompactEditDropdown = LayoutMode == ELayoutMode::CompactInline || LayoutMode == ELayoutMode::EditDropDownOnly;
	bool bAddResetButtons = LayoutMode == ELayoutMode::FullRow;

	// Indent
	if (bAddIndent)
	{
			ChildrenBox->AddSlot()
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					SNew(SNiagaraStackIndent, FunctionInput, ENiagaraStackIndentMode::Value)
				];
	}

	if (bAddValue)
	{
			ChildrenBox->AddSlot()
				.VAlign(VAlign_Center)
				.Padding(0)
				[
					// Value container and widgets.
					SAssignNew(ValueContainer, SBox)
					.ToolTipText_UObject(FunctionInput, &UNiagaraStackFunctionInput::GetValueToolTip)
					[
						ConstructValueWidgets()
					]
				];

			ValueModeForGeneratedWidgets = FunctionInput->GetValueMode();

			if (InArgs._ValueDecorator.Widget != SNullWidget::NullWidget)
			{
				ChildrenBox->AddSlot()
					.VAlign(VAlign_Center)
					.Padding(0)
					.AutoWidth()
					[
						InArgs._ValueDecorator.Widget
					];
			}
	}

	// Edit drop down
	if (bAddEditDropDown)
	{
				// Handle drop-down button
				ChildrenBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3, 0, 0, 0)
				[
					SAssignNew(SetFunctionInputButton, SComboButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.IsFocusable(false)
					.ForegroundColor(FSlateColor::UseForeground())
					.OnGetMenuContent(this, &SNiagaraStackFunctionInputValue::OnGetAvailableHandleMenu)
					.ContentPadding(FMargin(2))
					.Visibility(this, &SNiagaraStackFunctionInputValue::GetDropdownButtonVisibility)
					.MenuPlacement(MenuPlacement_BelowRightAnchor)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
				];
	}

	// Reset buttons
	if (bAddResetButtons)
	{
				// Reset Button
				ChildrenBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3, 0, 0, 0)
				[
					SNew(SButton)
					.IsFocusable(false)
					.ToolTipText(LOCTEXT("ResetToolTip", "Reset to the default value"))
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ContentPadding(0)
					.Visibility(this, &SNiagaraStackFunctionInputValue::GetResetButtonVisibility)
					.OnClicked(this, &SNiagaraStackFunctionInputValue::ResetButtonPressed)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
					]
				];

				// Reset to base Button
				ChildrenBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3, 0, 0, 0)
				[
					SNew(SButton)
					.IsFocusable(false)
					.ToolTipText(LOCTEXT("ResetToBaseToolTip", "Reset this input to the value defined by the parent emitter"))
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ContentPadding(0)
					.Visibility(this, &SNiagaraStackFunctionInputValue::GetResetToBaseButtonVisibility)
					.OnClicked(this, &SNiagaraStackFunctionInputValue::ResetToBaseButtonPressed)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
						.ColorAndOpacity(FSlateColor(FLinearColor::Green))
					]
				];
	}

	// Compact edit drop down
	if (bAddCompactEditDropdown)
	{
		ChildrenBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.ComboButtonStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.CompactComboButton")
			.IsFocusable(false)
			.ForegroundColor(FSlateColor::UseForeground())
			.OnGetMenuContent(this, &SNiagaraStackFunctionInputValue::OnGetCompactActionMenu)
			.Visibility(CompactActionMenuButtonVisibilityAttribute)
			.MenuPlacement(MenuPlacement_BelowRightAnchor)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
		];
	}
}

void SNiagaraStackFunctionInputValue::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (FunctionInput->GetIsDynamicInputScriptReassignmentPending())
	{
		FunctionInput->SetIsDynamicInputScriptReassignmentPending(false);
		ShowReassignDynamicInputScriptMenu();
	}
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::ConstructValueWidgets()
{
	DisplayedLocalValueStruct.Reset();
	LocalValueStructParameterEditor.Reset();
	LocalValueStructDetailsView.Reset();

	TSharedRef<SHorizontalBox> ValueBox = SNew(SHorizontalBox);

	bool bShowTypeIcon = true;
	switch(FunctionInput->GetValueMode())
	{
	case UNiagaraStackFunctionInput::EValueMode::Local:
		ConstructLocalValueStructWidgets(ValueBox);
		bShowTypeIcon = false;
		break;

	case UNiagaraStackFunctionInput::EValueMode::Linked:
		ConstructLinkedValueWidgets(ValueBox);
		break;

	case UNiagaraStackFunctionInput::EValueMode::LinkedShared:
		ConstructLinkedSharedValueWidgets(ValueBox);
		bShowTypeIcon = false;
		break;

	case UNiagaraStackFunctionInput::EValueMode::Data:
		ConstructTextValueWidgets(ValueBox, TAttribute<FText>::CreateSP(this, &SNiagaraStackFunctionInputValue::GetDataValueText));
		break;

	case UNiagaraStackFunctionInput::EValueMode::ObjectAsset:
		ValueBox->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(FunctionInput->GetInputType().GetClass())
			.ObjectPath_Lambda(
				[this]() -> FString
				{
					UObject* ObjectAsset = FunctionInput->GetObjectAssetValue();
					return ObjectAsset ? ObjectAsset->GetPathName() : FString();
				}
			)
			.DisplayBrowse(true)
			.DisplayUseSelected(true)
			.DisplayThumbnail(true)
			.EnableContentPicker(true)
			.OnObjectChanged_Lambda(
				[this](const FAssetData& AssetData)
				{
					FunctionInput->SetObjectAssetValue(AssetData.GetAsset());
				}
			)
		];
		break;

	case UNiagaraStackFunctionInput::EValueMode::Dynamic:
		ConstructDynamicValueWidgets(ValueBox);
		break;

	case UNiagaraStackFunctionInput::EValueMode::DefaultFunction:
		ConstructTextValueWidgets(ValueBox, TAttribute<FText>::CreateSP(this, &SNiagaraStackFunctionInputValue::GetDefaultFunctionText));
		break;

	case UNiagaraStackFunctionInput::EValueMode::Expression:
	{
		const FEditableTextBoxStyle& TextBoxStyle = FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
		ValueBox->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(SBorder)
			.BorderBackgroundColor(TextBoxStyle.BackgroundColor)
			.Padding(TextBoxStyle.Padding)
			.BorderImage(&TextBoxStyle.BackgroundImageNormal)
			[
				SNew(SMultiLineEditableText)
				.IsReadOnly(false)
				.Marshaller(SyntaxHighlighter)
				.AllowMultiLine(false)
				.Text_UObject(FunctionInput, &UNiagaraStackFunctionInput::GetCustomExpressionText)
				.OnTextCommitted(this, &SNiagaraStackFunctionInputValue::OnExpressionTextCommitted)
			]
		];
		break;
	}

	case UNiagaraStackFunctionInput::EValueMode::InvalidOverride:
		ConstructTextValueWidgets(ValueBox, LOCTEXT("InvalidOverrideText", "Invalid Script Value"));
		break;

	case UNiagaraStackFunctionInput::EValueMode::UnsupportedDefault:
		ConstructTextValueWidgets(ValueBox, LOCTEXT("UnsupportedDefault", "Custom Default"));
		break;
	}

	if (bShowTypeIcon)
	{
		ValueBox->InsertSlot(0)
			.Padding(0, 0, 3, 0)
			.AutoWidth()
			[
				SNew(SNiagaraStackFunctionInputTypeIcon, FunctionInput)
			];
	}

	return ValueBox;
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::GetVersionSelectorDropdownMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	UNiagaraScript* Script = FunctionInput->GetDynamicInputNode()->FunctionScript;
	TArray<FNiagaraAssetVersion> AssetVersions = Script->GetAllAvailableVersions();
	for (FNiagaraAssetVersion& Version : AssetVersions)
	{
		if (!Version.bIsVisibleInVersionSelector)
    	{
    		continue;
    	}
		FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(Version.VersionGuid);
		bool bIsSelected = FunctionInput->GetDynamicInputNode()->SelectedScriptVersion == Version.VersionGuid;
		
		FText Tooltip = LOCTEXT("NiagaraSelectVersion_Tooltip", "Select this version to use for the dynamic input");
		if (!ScriptData->VersionChangeDescription.IsEmpty())
		{
			Tooltip = FText::Format(LOCTEXT("NiagaraSelectVersionChangelist_Tooltip", "Select this version to use for the dynamic input. Change description for this version:\n{0}"), ScriptData->VersionChangeDescription);
		}
		
		FUIAction UIAction(FExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::SwitchToVersion, Version),
        FCanExecuteAction(),
        FIsActionChecked::CreateLambda([bIsSelected]() { return bIsSelected; }));
		FText Format = (Version == Script->GetExposedVersion()) ? FText::FromString("{0}.{1}*") : FText::FromString("{0}.{1}");
		FText Label = FText::Format(Format, Version.MajorVersion, Version.MinorVersion);
		MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);	
	}

	return MenuBuilder.MakeWidget();
}

void SNiagaraStackFunctionInputValue::SwitchToVersion(FNiagaraAssetVersion Version)
{
	FunctionInput->ChangeScriptVersion(Version.VersionGuid);
}

FSlateColor SNiagaraStackFunctionInputValue::GetVersionSelectorColor() const
{
	UNiagaraScript* Script = FunctionInput->GetDynamicInputNode()->FunctionScript;
	
	if (Script && Script->IsVersioningEnabled())
	{
		FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(FunctionInput->GetDynamicInputNode()->SelectedScriptVersion);
		if (ScriptData && ScriptData->Version < Script->GetExposedVersion())
		{
			return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.IconColor.VersionUpgrade");
		}
	}
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.FlatButtonColor");
}

void SNiagaraStackFunctionInputValue::SetToLocalValue()
{
	if (FunctionInput->GetInputType().IsDataInterface())
	{
		FunctionInput->SetDataInterfaceValue(FunctionInput->GetInputType().GetClass());
	}
	else if (FunctionInput->GetInputType().IsUObject() == false)
	{
		const UScriptStruct* LocalValueStruct = FunctionInput->GetInputType().GetScriptStruct();
		if (LocalValueStruct != nullptr)
		{
			TSharedRef<FStructOnScope> LocalValue = MakeShared<FStructOnScope>(LocalValueStruct);
			TArray<uint8> DefaultValueData;
			FNiagaraEditorUtilities::GetTypeDefaultValue(FunctionInput->GetInputType(), DefaultValueData);
			if (DefaultValueData.Num() == LocalValueStruct->GetStructureSize())
			{
				FMemory::Memcpy(LocalValue->GetStructMemory(), DefaultValueData.GetData(), DefaultValueData.Num());
				FunctionInput->SetLocalValue(LocalValue);
			}
		}
	}
}

bool SNiagaraStackFunctionInputValue::GetInputEnabled() const
{
	return FunctionInput->IsFinalized() == false && (FunctionInput->GetHasEditCondition() == false || FunctionInput->GetEditConditionEnabled());
}

bool SNiagaraStackFunctionInputValue::GetEntryEnabled() const
{
	return FunctionInput->IsFinalized() == false && FunctionInput->GetIsEnabledAndOwnerIsEnabled();
}

void SNiagaraStackFunctionInputValue::ConstructTextValueWidgets(TSharedRef<SHorizontalBox>& ValueBox, TAttribute<FText> GetText)
{
	ValueBox->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.Text(GetText)
		];
}

void SNiagaraStackFunctionInputValue::ConstructLocalValueStructWidgets(TSharedRef<SHorizontalBox>& ValueBox)
{
	LocalValueStructParameterEditor.Reset();
	LocalValueStructDetailsView.Reset();

	DisplayedLocalValueStruct = MakeShared<FStructOnScope>(FunctionInput->GetInputType().GetStruct());
	FNiagaraEditorUtilities::CopyDataTo(*DisplayedLocalValueStruct.Get(), *FunctionInput->GetLocalValueStruct().Get());
	if (DisplayedLocalValueStruct.IsValid())
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(FunctionInput->GetInputType());
		if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanCreateParameterEditor())
		{
			TSharedPtr<SNiagaraParameterEditor> ParameterEditor = TypeEditorUtilities->CreateParameterEditor(FunctionInput->GetInputType(), FunctionInput->GetInputDisplayUnit(), FunctionInput->GetInputWidgetCustomization());
			if (LayoutMode == ELayoutMode::CompactInline && ParameterEditor->GetMinimumDesiredWidth().IsSet())
			{
				ParameterEditor->SetMinimumDesiredWidth(ParameterEditor->GetMinimumDesiredWidth().GetValue() / 2.0f);
			}
			ParameterEditor->UpdateInternalValueFromStruct(DisplayedLocalValueStruct.ToSharedRef());
			ParameterEditor->SetOnBeginValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(
				this, &SNiagaraStackFunctionInputValue::ParameterBeginValueChange));
			ParameterEditor->SetOnEndValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(
				this, &SNiagaraStackFunctionInputValue::ParameterEndValueChange));
			ParameterEditor->SetOnValueChanged(SNiagaraParameterEditor::FOnValueChange::CreateSP(
				this, &SNiagaraStackFunctionInputValue::ParameterValueChanged, TWeakPtr<SNiagaraParameterEditor>(ParameterEditor)));

			LocalValueStructParameterEditor = ParameterEditor;

			ValueBox->AddSlot()
				.HAlign(ParameterEditor->GetHorizontalAlignment())
				.VAlign(ParameterEditor->GetVerticalAlignment())
				.Padding(0)
				[
					ParameterEditor.ToSharedRef()
				];
		}
		else
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

			// Originally FDetailsViewArgs(false, false, false, FDetailsViewArgs::HideNameArea, true)
			FDetailsViewArgs Args; 
			Args.bUpdatesFromSelection = false;
			Args.bLockable = false;
			Args.bAllowSearch = false;
			Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			Args.bHideSelectionTip = true;

			TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(
				Args,
				FStructureDetailsViewArgs(),
				nullptr);

			StructureDetailsView->SetStructureData(DisplayedLocalValueStruct);
			StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &SNiagaraStackFunctionInputValue::ParameterPropertyValueChanged);

			LocalValueStructDetailsView = StructureDetailsView;

			ValueBox->AddSlot()
				.Padding(0)
				[
					StructureDetailsView->GetWidget().ToSharedRef()
				];
		}
	}
}

void SNiagaraStackFunctionInputValue::ConstructLinkedValueWidgets(TSharedRef<SHorizontalBox>& ValueBox)
{
	TSharedRef<SWidget> ParameterWidget = SNew(SNiagaraParameterName)
		.ReadOnlyTextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
		.ParameterName(this, &SNiagaraStackFunctionInputValue::GetLinkedValueHandleName)
		.OnDoubleClicked(this, &SNiagaraStackFunctionInputValue::OnLinkedInputDoubleClicked);

	FNiagaraParameterHandle ParameterHandle = FNiagaraParameterHandle(FunctionInput->GetLinkedParameterValue().GetName());
	if (ParameterHandle.IsUserHandle())
	{
		TArray<FNiagaraVariable> UserParameters;
		FunctionInput->GetSystemViewModel()->GetSystem().GetExposedParameters().GetUserParameters(UserParameters);
		FNiagaraVariable* MatchingVariable = UserParameters.FindByPredicate([ParameterHandle](const FNiagaraVariable& Variable)
			{
				return Variable.GetName().ToString() == ParameterHandle.GetName().ToString();
			});

		if (MatchingVariable)
		{
			FText Tooltip = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(*MatchingVariable, FunctionInput->GetSystemViewModel())->Metadata.Description;
			if (!Tooltip.IsEmpty())
			{
				ParameterWidget->SetToolTipText(Tooltip);
			}
		}
	}

	ValueBox->AddSlot()
		.Padding(0)
		[
			ParameterWidget
		];
}

void SNiagaraStackFunctionInputValue::ConstructLinkedSharedValueWidgets(TSharedRef<SHorizontalBox>& ValueBox)
{
	ValueBox->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor(FLinearColor(0.5f, 0, 0.5f, 0.2f))
				.Padding(0)
				[
					SNew(SNiagaraStackFunctionInputValue, FunctionInput->GetSharedLinkedValueInput())
						.LayoutMode(ELayoutMode::LinkedInline)
						.ValueDecorator()
						[
							SNew(SNiagaraStackFunctionLinkedSharedButton, FunctionInput)
								.Visibility(this, &SNiagaraStackFunctionInputValue::GetVisibilityForHoverOnlyWidgets)
						]
				]
		];
}

void SNiagaraStackFunctionInputValue::ConstructDynamicValueWidgets(TSharedRef<SHorizontalBox>& ValueBox)
{
	ValueBox->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(this, &SNiagaraStackFunctionInputValue::GetDynamicValueText)
				.OnDoubleClicked(this, &SNiagaraStackFunctionInputValue::DynamicInputTextDoubleClicked)
		];

	if (FunctionInput->IsScratchDynamicInput())
	{
		ValueBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				.OnClicked(this, &SNiagaraStackFunctionInputValue::ScratchButtonPressed)
				.ToolTipText(LOCTEXT("OpenInScratchToolTip", "Open this dynamic input in the scratch pad."))
				.ContentPadding(FMargin(1.0f, 0.0f))
				.Content()
				[
					SNew(SImage)
					.Image(FNiagaraEditorStyle::Get().GetBrush("Tab.ScratchPad"))
				]
			];
	}
	else if (FunctionInput->GetDynamicInputNode()->FunctionScript && FunctionInput->GetDynamicInputNode()->FunctionScript->IsVersioningEnabled())
	{
		// the function script could be wiped (deleted scratch pad script or missing asset)
		ValueBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SComboButton)
				.HasDownArrow(false)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.OnGetMenuContent(this, &SNiagaraStackFunctionInputValue::GetVersionSelectorDropdownMenu)
				.ContentPadding(FMargin(2))
				.ToolTipText(LOCTEXT("VersionTooltip", "Change the version of this module script"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
					.ColorAndOpacity(this, &SNiagaraStackFunctionInputValue::GetVersionSelectorColor)
					.Text(FEditorFontGlyphs::Random)
				]
			];
	}
}

void SNiagaraStackFunctionInputValue::OnInputValueChanged()
{
	if (LayoutMode == ELayoutMode::EditDropDownOnly)
	{
		return;
	}

	if (ValueModeForGeneratedWidgets == FunctionInput->GetValueMode() &&
		ValueModeForGeneratedWidgets == UNiagaraStackFunctionInput::EValueMode::Local &&
		DisplayedLocalValueStruct.IsValid() &&
		DisplayedLocalValueStruct->GetStruct() == FunctionInput->GetLocalValueStruct()->GetStruct())
	{
		FNiagaraEditorUtilities::CopyDataTo(*DisplayedLocalValueStruct.Get(), *FunctionInput->GetLocalValueStruct().Get());
		if (LocalValueStructParameterEditor.IsValid())
		{
			LocalValueStructParameterEditor->UpdateInternalValueFromStruct(DisplayedLocalValueStruct.ToSharedRef());
		}
		if (LocalValueStructDetailsView.IsValid())
		{
			LocalValueStructDetailsView->SetStructureData(TSharedPtr<FStructOnScope>());
			LocalValueStructDetailsView->SetStructureData(DisplayedLocalValueStruct);
		}
	}
	else
	{
		ValueContainer->SetContent(ConstructValueWidgets());
		ValueModeForGeneratedWidgets = FunctionInput->GetValueMode();
	}
}

void SNiagaraStackFunctionInputValue::ParameterBeginValueChange()
{
	FunctionInput->NotifyBeginLocalValueChange();
}

void SNiagaraStackFunctionInputValue::ParameterEndValueChange()
{
	FunctionInput->NotifyEndLocalValueChange();
}

void SNiagaraStackFunctionInputValue::ParameterValueChanged(TWeakPtr<SNiagaraParameterEditor> ParameterEditor)
{
	TSharedPtr<SNiagaraParameterEditor> ParameterEditorPinned = ParameterEditor.Pin();
	if (ParameterEditorPinned.IsValid())
	{
		ParameterEditorPinned->UpdateStructFromInternalValue(DisplayedLocalValueStruct.ToSharedRef());
		FunctionInput->SetLocalValue(DisplayedLocalValueStruct.ToSharedRef());
	}
}

void SNiagaraStackFunctionInputValue::ParameterPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	FunctionInput->SetLocalValue(DisplayedLocalValueStruct.ToSharedRef());
}

FName SNiagaraStackFunctionInputValue::GetLinkedValueHandleName() const
{
	return FunctionInput->GetLinkedParameterValue().GetName();
}

FText SNiagaraStackFunctionInputValue::GetDataValueText() const
{
	if (FunctionInput->GetDataValueObject() != nullptr)
	{
		return FunctionInput->GetInputType().GetClass()->GetDisplayNameText();
	}
	else
	{
		return FText::Format(LOCTEXT("InvalidDataObjectFormat", "{0} (Invalid)"), FunctionInput->GetInputType().GetClass()->GetDisplayNameText());
	}
}
UEnum* GetDisplayUnitEnum()
{
	static UEnum* UnitEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/CoreUObject.EUnit"));
	return UnitEnum;
}

FText SNiagaraStackFunctionInputValue::GetObjectAssetValueText() const
{
	if (FunctionInput->GetObjectAssetValue() != nullptr)
	{
		return FunctionInput->GetInputType().GetClass()->GetDisplayNameText();
	}
	else
	{
		return FText::Format(LOCTEXT("InvalidObjectAssetFormat", "{0} (null)"), FunctionInput->GetInputType().GetClass()->GetDisplayNameText());
	}
}

FText SNiagaraStackFunctionInputValue::GetDynamicValueText() const
{
	if (UNiagaraNodeFunctionCall* NodeFunctionCall = FunctionInput->GetDynamicInputNode())
	{
		if (!FunctionInput->GetIsExpanded())
		{
			FText CollapsedText = FunctionInput->GetCollapsedStateText();
			if (!CollapsedText.IsEmptyOrWhitespace())
			{
				return CollapsedText;
			}
		}
		FString FunctionName = NodeFunctionCall->FunctionScript ? NodeFunctionCall->FunctionScript->GetName() : NodeFunctionCall->Signature.Name.ToString();
		FText DisplayString = FText::FromString(FName::NameToDisplayString(FunctionName, false));
		EUnit DisplayUnit = FunctionInput->GetInputDisplayUnit();
		if (DisplayUnit != EUnit::Unspecified)
		{
			UEnum* Enum = GetDisplayUnitEnum();
			return FText::Format(FText::FromString("{0} ({1})"), DisplayString, FText::FromString(Enum->GetNameStringByValue(static_cast<int64>(DisplayUnit))));
		}
		return DisplayString;
	}
	return LOCTEXT("InvalidDynamicDisplayName", "(Invalid)");
}

FText SNiagaraStackFunctionInputValue::GetDefaultFunctionText() const
{
	if (FunctionInput->GetDefaultFunctionNode() != nullptr)
	{
		return FText::FromString(FName::NameToDisplayString(FunctionInput->GetDefaultFunctionNode()->GetFunctionName(), false));
	}
	else
	{
		return LOCTEXT("InvalidDefaultFunctionDisplayName", "(Invalid)");
	}
}

void SNiagaraStackFunctionInputValue::OnExpressionTextCommitted(const FText& Name, ETextCommit::Type CommitInfo)
{
	FunctionInput->SetCustomExpression(Name.ToString());
}

FReply SNiagaraStackFunctionInputValue::DynamicInputTextDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	if (FunctionInput->OpenSourceAsset())
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

FReply SNiagaraStackFunctionInputValue::OnLinkedInputDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	FString ParamCollection;
	FString ParamName;

	FNiagaraParameterHandle ParameterHandle = FNiagaraParameterHandle(FunctionInput->GetLinkedParameterValue().GetName());
	ParameterHandle.GetName().ToString().Split(TEXT("."), &ParamCollection, &ParamName);

	TArray<UNiagaraParameterCollection*> AvailableParameterCollections;
	FNiagaraEditorUtilities::GetAvailableParameterCollections(AvailableParameterCollections);
	for (UNiagaraParameterCollection* Collection : AvailableParameterCollections)
	{
		if (Collection->GetNamespace() == *ParamCollection)
		{
			if (UNiagaraParameterCollectionInstance* NPCInst = FunctionInput->GetSystemViewModel()->GetSystem().GetParameterCollectionOverride(Collection))
			{
				//If we override this NPC then open the instance.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NPCInst);
			}
			else
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Collection); 
			}
			
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

TSharedRef<SExpanderArrow> SNiagaraStackFunctionInputValue::CreateCustomNiagaraFunctionInputActionExpander(const FCustomExpanderData& ActionMenuData)
{
	return SNew(SNiagaraFunctionInputActionMenuExpander, ActionMenuData);
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::OnGetAvailableHandleMenu()
{
	SNiagaraFilterBox::FFilterOptions FilterOptions;
	FilterOptions.SetAddLibraryFilter(true);
	FilterOptions.SetAddSourceFilter(true);
	
	SAssignNew(FilterBox, SNiagaraFilterBox, FilterOptions)
	.bLibraryOnly(this, &SNiagaraStackFunctionInputValue::GetLibraryOnly)
	.OnLibraryOnlyChanged(this, &SNiagaraStackFunctionInputValue::SetLibraryOnly)
    .OnSourceFiltersChanged(this, &SNiagaraStackFunctionInputValue::TriggerRefresh);
	
	TSharedRef<SBorder> MenuWidget = SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("Menu.Background"))
	.Padding(5)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
        [
			FilterBox.ToSharedRef()
        ]
		+SVerticalBox::Slot()
		[
			SNew(SBox)
			.WidthOverride(450)
			.HeightOverride(400)
			[
				SAssignNew(ActionSelector, SNiagaraMenuActionSelector)
				.Items(CollectActions())
				.OnGetCategoriesForItem(this, &SNiagaraStackFunctionInputValue::OnGetCategoriesForItem)
                .OnGetSectionsForItem(this, &SNiagaraStackFunctionInputValue::OnGetSectionsForItem)
                .OnCompareSectionsForEquality(this, &SNiagaraStackFunctionInputValue::OnCompareSectionsForEquality)
                .OnCompareSectionsForSorting(this, &SNiagaraStackFunctionInputValue::OnCompareSectionsForSorting)
                .OnCompareCategoriesForEquality(this, &SNiagaraStackFunctionInputValue::OnCompareCategoriesForEquality)
                .OnCompareCategoriesForSorting(this, &SNiagaraStackFunctionInputValue::OnCompareCategoriesForSorting)
                .OnCompareItemsForSorting(this, &SNiagaraStackFunctionInputValue::OnCompareItemsForSorting)
                .OnDoesItemMatchFilterText_Static(&FNiagaraEditorUtilities::DoesItemMatchFilterText)
                .OnGenerateWidgetForSection(this, &SNiagaraStackFunctionInputValue::OnGenerateWidgetForSection)
                .OnGenerateWidgetForCategory(this, &SNiagaraStackFunctionInputValue::OnGenerateWidgetForCategory)
                .OnGenerateWidgetForItem(this, &SNiagaraStackFunctionInputValue::OnGenerateWidgetForItem)
				.OnGetItemWeight_Lambda([](const TSharedPtr<FNiagaraMenuAction_Generic>& Item, const TArray<FString>& FilterTerms)
				{
					return FNiagaraEditorUtilities::GetWeightForItem(Item, SNiagaraStackFunctionInputValue::FavoriteActionsProfile, FilterTerms);
				})
                .OnItemActivated(this, &SNiagaraStackFunctionInputValue::OnItemActivated)
                .AllowMultiselect(false)
                .OnDoesItemPassCustomFilter(this, &SNiagaraStackFunctionInputValue::DoesItemPassCustomFilter)
                .ClickActivateMode(EItemSelectorClickActivateMode::SingleClick)
                .ExpandInitially(false)
				.OnItemRowHoverEvent(this, &SNiagaraStackFunctionInputValue::OnActionRowHoverEvent)
                .OnGetSectionData_Lambda([](const ENiagaraMenuSections& Section)
                {
                    if(Section == ENiagaraMenuSections::Suggested)
                    {
                        return SNiagaraMenuActionSelector::FSectionData(SNiagaraMenuActionSelector::FSectionData::List, true);
                    }

                    return SNiagaraMenuActionSelector::FSectionData(SNiagaraMenuActionSelector::FSectionData::Tree, false);
                })
			]
		]
	];

	if(SetFunctionInputButton.IsValid())
	{ 
		SetFunctionInputButton->SetMenuContentWidgetToFocus(ActionSelector->GetSearchBox());
	}
	return MenuWidget;
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::OnGetCompactActionMenu()
{
	if (StackCommandContext.IsValid() == false)
	{
		StackCommandContext = MakeShared<FNiagaraStackCommandContext>();
		TArray<UNiagaraStackEntry*> SelectedEntries;
		SelectedEntries.Add(FunctionInput);
		StackCommandContext->SetSelectedEntries(SelectedEntries);
	}

	FMenuBuilder MenuBuilder(true, StackCommandContext->GetCommands());
	MenuBuilder.BeginSection("Value", LOCTEXT("ValueHeader", "Value"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AssignSubMenu", "Assign..."),
			LOCTEXT("AssignSubMenuToolTip", "Assign this input a new value..."),
			FNewMenuDelegate::CreateSP(this, &SNiagaraStackFunctionInputValue::OnFillAssignSubMenu));
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ResetToDefaultMenuEntry", "Reset to Default"),
			LOCTEXT("ResetToDefaultMenuEntryToolTip", "Reset this input to the value defined in the script."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::Reset),
				FCanExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::CanReset)));
		if (FunctionInput->HasBaseEmitter())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ResetToBaseMenuEntry", "Reset to Base"),
				LOCTEXT("ResetToBaseMenuEntryToolTip", "Reset this input to the value defined in the base emitter."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::ResetToBase),
					FCanExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::CanResetToBase)));
		}
	}
	MenuBuilder.EndSection();

	StackCommandContext->AddEditMenuItems(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void SNiagaraStackFunctionInputValue::OnFillAssignSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddWidget(OnGetAvailableHandleMenu(), FText());
}

void SNiagaraStackFunctionInputValue::DynamicInputScriptSelected(UNiagaraScript* DynamicInputScript)
{
	FunctionInput->SetDynamicInput(DynamicInputScript);
}

void SNiagaraStackFunctionInputValue::CustomExpressionSelected()
{
	FText CustomHLSLComment = LOCTEXT("NewCustomExpressionComment", "Custom HLSL!");
	FunctionInput->SetCustomExpression(INiagaraHlslTranslator::GetHlslDefaultForType(FunctionInput->GetInputType()) + TEXT(" /* ") + CustomHLSLComment.ToString() + TEXT(" */"));
}

void SNiagaraStackFunctionInputValue::CreateScratchSelected()
{
	FunctionInput->SetScratch();
}

void SNiagaraStackFunctionInputValue::ParameterSelected(FNiagaraVariableBase Parameter)
{
	FunctionInput->SetLinkedParameterValue(Parameter);
}

void SNiagaraStackFunctionInputValue::ParameterWithConversionSelected(FNiagaraVariableBase Parameter, UNiagaraScript* ConversionScript)
{
	FunctionInput->SetLinkedParameterValueViaConversionScript(Parameter, *ConversionScript);
}

EVisibility SNiagaraStackFunctionInputValue::GetResetButtonVisibility() const
{
	return FunctionInput->CanReset() ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility SNiagaraStackFunctionInputValue::GetDropdownButtonVisibility() const
{
	if (FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::LinkedShared)
	{
		return EVisibility::Collapsed;
	}
	else
	{
		return FunctionInput->IsStaticParameter() ? EVisibility::Hidden : EVisibility::Visible;
	}
}

FReply SNiagaraStackFunctionInputValue::ResetButtonPressed() const
{
	FunctionInput->Reset();
	return FReply::Handled();
}

EVisibility SNiagaraStackFunctionInputValue::GetResetToBaseButtonVisibility() const
{
	if (FunctionInput->HasBaseEmitter() && FunctionInput->GetEmitterViewModel().IsValid())
	{
		return FunctionInput->CanResetToBase() ? EVisibility::Visible : EVisibility::Hidden;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

FReply SNiagaraStackFunctionInputValue::ResetToBaseButtonPressed() const
{
	FunctionInput->ResetToBase();
	return FReply::Handled();
}

FReply SNiagaraStackFunctionInputValue::OnFunctionInputDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FNiagaraParameterDragOperation> InputDragDropOperation = InDragDropEvent.GetOperationAs<FNiagaraParameterDragOperation>();
	if (InputDragDropOperation)
	{
		TSharedPtr<FNiagaraParameterAction> Action = StaticCastSharedPtr<FNiagaraParameterAction>(InputDragDropOperation->GetSourceAction());
		if (Action.IsValid())
		{
			FNiagaraTypeDefinition FromType = Action->GetParameter().GetType();
			FNiagaraTypeDefinition ToType = FunctionInput->GetInputType();
			if (FNiagaraEditorUtilities::AreTypesAssignable(FromType, ToType))
			{
				// the types are the same, so we can just link the value directly
				FunctionInput->SetLinkedParameterValue(Action->GetParameter());
				return FReply::Handled();
			}
			else
			{
				// the types don't match, so we use a dynamic input to convert from one to the other
				TArray<UNiagaraScript*> ConversionScripts = FunctionInput->GetPossibleConversionScripts(FromType);
				if (ConversionScripts.Num() > 0)
				{
					FunctionInput->SetLinkedParameterValueViaConversionScript(Action->GetParameter(), *ConversionScripts[0]);
					return FReply::Handled();
				}
			}
		}
	}

	return FReply::Unhandled();
}

bool SNiagaraStackFunctionInputValue::OnFunctionInputAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	if (FunctionInput && DragDropOperation->IsOfType<FNiagaraParameterDragOperation>())
	{
		if (FunctionInput->IsStaticParameter())
		{
			return false;
		}

		TSharedPtr<FNiagaraParameterDragOperation> InputDragDropOperation = StaticCastSharedPtr<FNiagaraParameterDragOperation>(DragDropOperation);
		TSharedPtr<FNiagaraParameterAction> Action = StaticCastSharedPtr<FNiagaraParameterAction>(InputDragDropOperation->GetSourceAction());
		bool bAllowedInExecutionCategory = FNiagaraStackGraphUtilities::ParameterAllowedInExecutionCategory(Action->GetParameter().GetName(), FunctionInput->GetExecutionCategoryName());
		FNiagaraTypeDefinition DropType = Action->GetParameter().GetType();
		
		// check if we can simply link the input directly
		if (bAllowedInExecutionCategory && FNiagaraEditorUtilities::AreTypesAssignable(DropType, FunctionInput->GetInputType()))
		{
			return true;
		}

		// check if we can use a conversion script
		if (bAllowedInExecutionCategory && FunctionInput->GetPossibleConversionScripts(DropType).Num() > 0)
		{
			return true;
		}
	}

	return false;
}

void ReassignDynamicInputScript(UNiagaraStackFunctionInput* FunctionInput, UNiagaraScript* NewDynamicInputScript)
{
	FunctionInput->ReassignDynamicInputScript(NewDynamicInputScript);
}

TArray<TSharedPtr<FNiagaraMenuAction_Generic>> SNiagaraStackFunctionInputValue::CollectDynamicInputActionsForReassign() const
{
	TArray<TSharedPtr<FNiagaraMenuAction_Generic>> DynamicInputActions;
	
	const FText CategoryName = LOCTEXT("DynamicInputValueCategory", "Dynamic Inputs");
	TArray<UNiagaraScript*> DynamicInputScripts;
	FunctionInput->GetAvailableDynamicInputs(DynamicInputScripts, true);
	
	TSet<UNiagaraScript*> ScratchPadDynamicInputs;
	for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel : FunctionInput->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetScriptViewModels())
	{
		ScratchPadDynamicInputs.Add(ScratchPadScriptViewModel->GetOriginalScript());
	}
	
	for (UNiagaraScript* DynamicInputScript : DynamicInputScripts)
	{
		FVersionedNiagaraScriptData* ScriptData = DynamicInputScript->GetLatestScriptData();
		bool bIsInLibrary = ScriptData->LibraryVisibility == ENiagaraScriptLibraryVisibility::Library;
		const FText DisplayName = FNiagaraEditorUtilities::FormatScriptName(DynamicInputScript->GetFName(), bIsInLibrary);
		const FText Tooltip = FNiagaraEditorUtilities::FormatScriptDescription(ScriptData->Description, FSoftObjectPath(DynamicInputScript), bIsInLibrary);
		TTuple<EScriptSource, FText> Source = FNiagaraEditorUtilities::GetScriptSource(FAssetData(DynamicInputScript));

		// scratch pad dynamic inputs are always considered to be in the library and will have Niagara as the source
		if(ScratchPadDynamicInputs.Contains(DynamicInputScript))
		{
			Source = TTuple<EScriptSource, FText>(EScriptSource::Niagara, FText::FromString("Scratch Pad"));
			bIsInLibrary = true;
		}

		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(GetPathNameSafe(DynamicInputScript)));
		FavoritesActionData.bFavoriteByDefault = ScriptData->bSuggested;
			
		TSharedPtr<FNiagaraMenuAction_Generic> DynamicInputAction(new FNiagaraMenuAction_Generic(
			FNiagaraMenuAction_Generic::FOnExecuteAction::CreateStatic(&ReassignDynamicInputScript, FunctionInput, DynamicInputScript),
			DisplayName, {CategoryName.ToString()}, FavoritesActionData, Tooltip, ScriptData->Keywords
            ));
		DynamicInputAction->SourceData = FNiagaraActionSourceData(Source.Key, Source.Value, true);
		DynamicInputAction->bIsInLibrary = bIsInLibrary;

		DynamicInputActions.Add(DynamicInputAction);
	}

	return DynamicInputActions;
}

void SNiagaraStackFunctionInputValue::ShowReassignDynamicInputScriptMenu()
{
	SNiagaraFilterBox::FFilterOptions FilterOptions;
	FilterOptions.SetAddLibraryFilter(true);
	FilterOptions.SetAddSourceFilter(true);
	
	SAssignNew(FilterBox, SNiagaraFilterBox, FilterOptions)
	.bLibraryOnly(this, &SNiagaraStackFunctionInputValue::GetLibraryOnly)
	.OnLibraryOnlyChanged(this, &SNiagaraStackFunctionInputValue::SetLibraryOnly)
	.OnSourceFiltersChanged(this, &SNiagaraStackFunctionInputValue::TriggerRefresh);
	
	TSharedRef<SBorder> MenuWidget = SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("Menu.Background"))
	.Padding(5)
	[
		SNew(SVerticalBox)		
		+SVerticalBox::Slot()
		.AutoHeight()
        [
			FilterBox.ToSharedRef()
        ]
		+SVerticalBox::Slot()
		[
			SNew(SBox)
			.WidthOverride(450)
			.HeightOverride(400)
			[
				SAssignNew(ActionSelector, SNiagaraMenuActionSelector)
				.Items(CollectDynamicInputActionsForReassign())
				.OnGetCategoriesForItem(this, &SNiagaraStackFunctionInputValue::OnGetCategoriesForItem)
                .OnGetSectionsForItem(this, &SNiagaraStackFunctionInputValue::OnGetSectionsForItem)
                .OnCompareSectionsForEquality(this, &SNiagaraStackFunctionInputValue::OnCompareSectionsForEquality)
                .OnCompareSectionsForSorting(this, &SNiagaraStackFunctionInputValue::OnCompareSectionsForSorting)
                .OnCompareCategoriesForEquality(this, &SNiagaraStackFunctionInputValue::OnCompareCategoriesForEquality)
                .OnCompareCategoriesForSorting(this, &SNiagaraStackFunctionInputValue::OnCompareCategoriesForSorting)
                .OnCompareItemsForSorting(this, &SNiagaraStackFunctionInputValue::OnCompareItemsForSorting)
                .OnDoesItemMatchFilterText_Static(&FNiagaraEditorUtilities::DoesItemMatchFilterText)
                .OnGenerateWidgetForSection(this, &SNiagaraStackFunctionInputValue::OnGenerateWidgetForSection)
                .OnGenerateWidgetForCategory(this, &SNiagaraStackFunctionInputValue::OnGenerateWidgetForCategory)
                .OnGenerateWidgetForItem(this, &SNiagaraStackFunctionInputValue::OnGenerateWidgetForItem)
				.OnGetItemWeight_Lambda([](const TSharedPtr<FNiagaraMenuAction_Generic>& Item, const TArray<FString>& FilterTerms)
				 {
					 return FNiagaraEditorUtilities::GetWeightForItem(Item, SNiagaraStackFunctionInputValue::FavoriteActionsProfile, FilterTerms);
				 })
                .OnItemActivated(this, &SNiagaraStackFunctionInputValue::OnItemActivated)
                .AllowMultiselect(false)
                .OnDoesItemPassCustomFilter(this, &SNiagaraStackFunctionInputValue::DoesItemPassCustomFilter)
                .ClickActivateMode(EItemSelectorClickActivateMode::SingleClick)
                .ExpandInitially(false)
				.OnItemRowHoverEvent(this, &SNiagaraStackFunctionInputValue::OnActionRowHoverEvent)
                .OnGetSectionData_Lambda([](const ENiagaraMenuSections& Section)
                {
                    if(Section == ENiagaraMenuSections::Suggested)
                    {
                        return SNiagaraMenuActionSelector::FSectionData(SNiagaraMenuActionSelector::FSectionData::List, true);
                    }

                    return SNiagaraMenuActionSelector::FSectionData(SNiagaraMenuActionSelector::FSectionData::Tree, false);
                })
			]
		]
	];

	FGeometry ThisGeometry = GetCachedGeometry();
	bool bAutoAdjustForDpiScale = false; // Don't adjust for dpi scale because the push menu command is expecting an unscaled position.
	FVector2D MenuPosition = FSlateApplication::Get().CalculatePopupWindowPosition(ThisGeometry.GetLayoutBoundingRect(), MenuWidget->GetDesiredSize(), bAutoAdjustForDpiScale);
	FSlateApplication::Get().PushMenu(AsShared(), FWidgetPath(), MenuWidget, MenuPosition, FPopupTransitionEffect::ContextMenu);
}

bool SNiagaraStackFunctionInputValue::GetLibraryOnly() const
{
	return bLibraryOnly;
}

void SNiagaraStackFunctionInputValue::SetLibraryOnly(bool bInIsLibraryOnly)
{
	bLibraryOnly = bInIsLibraryOnly;
	ActionSelector->RefreshAllCurrentItems(true);
}

FReply SNiagaraStackFunctionInputValue::ScratchButtonPressed() const
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDynamicInputViewModel = 
		FunctionInput->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(FunctionInput->GetDynamicInputNode()->FunctionScript);
	if (ScratchDynamicInputViewModel.IsValid())
	{
		FunctionInput->GetSystemViewModel()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchDynamicInputViewModel.ToSharedRef());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

EVisibility SNiagaraStackFunctionInputValue::GetVisibilityForHoverOnlyWidgets() const
{
	return IsHovered() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SNiagaraStackFunctionInputValue::OnActionRowHoverEvent(const TSharedPtr<FNiagaraMenuAction_Generic>& ActionNode, bool bIsHovered)
{
	ActionNode->bIsHovered = bIsHovered;
}

TArray<TSharedPtr<FNiagaraMenuAction_Generic>> SNiagaraStackFunctionInputValue::CollectActions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(SNiagaraStackFunctionInputValue::CollectActions, NiagaraChannel);
	
	TArray<TSharedPtr<FNiagaraMenuAction_Generic>> OutAllActions;
	bool bIsDataInterfaceOrObject = FunctionInput->GetInputType().IsDataInterface() || FunctionInput->GetInputType().IsUObject();

	FNiagaraActionSourceData NiagaraSourceData(EScriptSource::Niagara, FText::FromString(TEXT("Niagara")), true);
	FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(FavoriteActionsProfile);
	
	// Set a local value
	{
		bool bCanSetLocalValue = 
			(FunctionInput->GetInputType().IsDataInterface() && FunctionInput->GetValueMode() != UNiagaraStackFunctionInput::EValueMode::Data) ||
			(bIsDataInterfaceOrObject == false && FunctionInput->GetValueMode() != UNiagaraStackFunctionInput::EValueMode::Local);

		const FText DisplayName = LOCTEXT("LocalValue", "New Local Value");
		const FText Tooltip = FText::Format(LOCTEXT("LocalValueToolTip", "Set a local editable value for this input."), DisplayName);
		TSharedPtr<FNiagaraMenuAction_Generic> SetLocalValueAction(new FNiagaraMenuAction_Generic(
			FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::SetToLocalValue),
			FNiagaraMenuAction_Generic::FCanExecuteAction::CreateLambda([=]() { return bCanSetLocalValue; }),
            DisplayName, {}, {}, Tooltip, FText()));
		SetLocalValueAction->SourceData = NiagaraSourceData;
		OutAllActions.Add(SetLocalValueAction);
	}

	// Add a dynamic input
	{
		const FText CategoryName = LOCTEXT("DynamicInputValueCategory", "Dynamic Inputs");
		TArray<UNiagaraScript*> DynamicInputScripts;
		FunctionInput->GetAvailableDynamicInputs(DynamicInputScripts, true);	// Note we do not filter here as we filter the displayed data

		// we add scratch pad scripts here so we can check if an available dynamic input is a scratch pad script or asset based
		TSet<UNiagaraScript*> ScratchPadDynamicInputs;
		for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel : FunctionInput->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetScriptViewModels())
		{
			ScratchPadDynamicInputs.Add(ScratchPadScriptViewModel->GetOriginalScript());
		}
		
		for (UNiagaraScript* DynamicInputScript : DynamicInputScripts)
		{
			TTuple<EScriptSource, FText> Source = FNiagaraEditorUtilities::GetScriptSource(DynamicInputScript);
			
			FVersionedNiagaraScriptData* ScriptData = DynamicInputScript->GetLatestScriptData();
			bool bIsInLibrary = ScriptData->LibraryVisibility == ENiagaraScriptLibraryVisibility::Library;
			const FText DisplayName = FNiagaraEditorUtilities::FormatScriptName(DynamicInputScript->GetFName(), bIsInLibrary);
			const FText Tooltip = FNiagaraEditorUtilities::FormatScriptDescription(ScriptData->Description, FSoftObjectPath(DynamicInputScript), bIsInLibrary);

			// scratch pad dynamic inputs are always considered to be in the library and will have Niagara as the source
			if(ScratchPadDynamicInputs.Contains(DynamicInputScript))
			{
				Source = TTuple<EScriptSource, FText>(EScriptSource::Niagara, FText::FromString("Scratch Pad"));
				bIsInLibrary = true;
			}
			
			// We construct an ActionIdentifier to check for it in the favorites list
			FNiagaraFavoritesActionData FavoritesActionData;
			FavoritesActionData.ActionIdentifier = FNiagaraActionIdentifier({FName(GetPathNameSafe(DynamicInputScript))}, {}); 
			FavoritesActionData.bFavoriteByDefault = ScriptData->bSuggested;
			
			TSharedPtr<FNiagaraMenuAction_Generic> DynamicInputAction(new FNiagaraMenuAction_Generic(
                FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::DynamicInputScriptSelected, DynamicInputScript),
                DisplayName, {CategoryName.ToString()}, FavoritesActionData, Tooltip, ScriptData->Keywords));
			DynamicInputAction->FavoritesActionData = FavoritesActionData;
			DynamicInputAction->SourceData = FNiagaraActionSourceData(Source.Key, Source.Value, true);
			DynamicInputAction->PreviewMoviePath = FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(FAssetData(DynamicInputScript));
			
			DynamicInputAction->bIsExperimental = ScriptData->bExperimental;
			DynamicInputAction->bIsInLibrary = bIsInLibrary;
			OutAllActions.Add(DynamicInputAction);
		}
	}

	// Link existing attribute
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	bool bAllowConversions = ensure(Settings != nullptr) && Settings->bShowConvertibleInputsInStack;

	TSet<UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo> AvailableParameterInfos;

	UNiagaraStackFunctionInput::FGetAvailableParameterArgs GetAvailableParameterArgs;
	GetAvailableParameterArgs.bIncludeConversionScripts = bAllowConversions;
	GetAvailableParameterArgs.bIncludeParameterDefinitions = true;
	FunctionInput->GetAvailableParameters(AvailableParameterInfos, GetAvailableParameterArgs);

	// First, we add the inputs that can be directly linked (without a conversion script)
	const FString RootCategoryName = FString("Link Inputs");
	const FText MapInputFormat = LOCTEXT("LinkInputFormat", "Link this input to {0}");
	for (const UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo& AvailableParameterInfo : AvailableParameterInfos)
	{
		if(AvailableParameterInfo.ConversionScript != nullptr)
		{
			continue;
		}
		
		FNiagaraParameterHandle AvailableHandle = FNiagaraParameterHandle(AvailableParameterInfo.Variable.GetName());
		
		TArray<FName> HandleParts = AvailableHandle.GetHandleParts();
		FNiagaraNamespaceMetadata NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(HandleParts);
		if (NamespaceMetadata.IsValid())
		{			
			// Only add handles which are in known namespaces to prevent collecting parameter handles
			// which are being used to configure modules and dynamic inputs in the stack graphs.
			const FText Category = NamespaceMetadata.DisplayName;
			const FText DisplayName = FText::FromName(AvailableHandle.GetParameterHandleString());
			const FText Tooltip = FText::Format(MapInputFormat, FText::FromName(AvailableHandle.GetParameterHandleString()));

			FNiagaraFavoritesActionData FavoritesActionData;
			FavoritesActionData.ActionIdentifier.Names.Add(FName(DisplayName.ToString()));
			FavoritesActionData.bFavoriteByDefault = false;
			
			TSharedPtr<FNiagaraMenuAction_Generic> LinkAction(new FNiagaraMenuAction_Generic(
				FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::ParameterSelected, AvailableParameterInfo.Variable),
				DisplayName, {RootCategoryName, Category.ToString()}, FavoritesActionData,  Tooltip, FText()));
			
			LinkAction->SetParameterVariable(FNiagaraVariable(FunctionInput->GetInputType(), AvailableHandle.GetParameterHandleString()));
			LinkAction->SourceData = NiagaraSourceData;
			LinkAction->AlternateSearchName = FText::FromName(HandleParts.Last());

			OutAllActions.Add(LinkAction);
		}
	}

	// Then we add those that can only get added via conversion script. This will only be valid if the plugin setting allows it to
	const FText ConvertInputFormat = LOCTEXT("ConvertInputFormat", "Link this input to {0} via a conversion script");
	for (const UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo& AvailableParameterInfo : AvailableParameterInfos)
	{
		if(AvailableParameterInfo.ConversionScript == nullptr)
		{
			continue;
		}
		
		FNiagaraVariableBase ParameterVariable = AvailableParameterInfo.Variable;
		UNiagaraScript* ConversionScript = AvailableParameterInfo.ConversionScript;
		const FNiagaraParameterHandle& AvailableHandle(ParameterVariable.GetName());
		TArray<FName> HandleParts = AvailableHandle.GetHandleParts();
		FNiagaraNamespaceMetadata NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(HandleParts);
		if (NamespaceMetadata.IsValid())
		{			
			// Only add handles which are in known namespaces to prevent collecting parameter handles
			// which are being used to configure modules and dynamic inputs in the stack graphs.
			const FText Category = NamespaceMetadata.DisplayName;
			const FText DisplayName = FText::FromName(AvailableHandle.GetParameterHandleString());
			const FText Tooltip = FText::Format(ConvertInputFormat, FText::FromName(AvailableHandle.GetParameterHandleString()));

			TSharedPtr<FNiagaraMenuAction_Generic> LinkAction(new FNiagaraMenuAction_Generic(
				FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::ParameterWithConversionSelected, ParameterVariable, ConversionScript),
				DisplayName, {RootCategoryName, Category.ToString()}, {}, Tooltip, FText()));

			LinkAction->SetParameterVariable(ParameterVariable);

			// set the source data from the script
			TTuple<EScriptSource, FText> Source = FNiagaraEditorUtilities::GetScriptSource(ConversionScript);
			LinkAction->SourceData = FNiagaraActionSourceData(Source.Key, Source.Value, true);;
			LinkAction->AlternateSearchName = FText::FromName(HandleParts.Last());

			OutAllActions.Add(LinkAction);
		}
	}

	// Read from new attribute
	{
		const FText CategoryName = LOCTEXT("MakeCategory", "Make");

		TArray<FName> AvailableNamespaces;
		FunctionInput->GetNamespacesForNewReadParameters(AvailableNamespaces);

		TArray<FString> InputNames;
		for (int32 i = FunctionInput->GetInputParameterHandlePath().Num() - 1; i >= 0; i--)
		{
			InputNames.Add(FunctionInput->GetInputParameterHandlePath()[i].GetName().ToString());
		}
		FName InputName = *FString::Join(InputNames, TEXT("_")).Replace(TEXT("."), TEXT("_"));

		for (const FName& AvailableNamespace : AvailableNamespaces)
		{
			FNiagaraParameterHandle HandleToRead(AvailableNamespace, InputName);
			FNiagaraVariableBase ParameterToRead = FNiagaraVariableBase(FunctionInput->GetInputType(), HandleToRead.GetParameterHandleString());
			bool bIsContained = AvailableParameterInfos.Contains(ParameterToRead);

			if(bIsContained)
			{
				TSet<FName> ExistingNames;
				for(const UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo& AvailableParameter : AvailableParameterInfos)
				{
					FNiagaraParameterHandle AvailableHandle = FNiagaraParameterHandle(AvailableParameter.Variable.GetName());
					ExistingNames.Add(AvailableHandle.GetName());
				}

				// let's get a unique name as the previous parameter already existed
				HandleToRead = FNiagaraParameterHandle(AvailableNamespace, FNiagaraUtilities::GetUniqueName(InputName, ExistingNames));
				ParameterToRead = FNiagaraVariableBase(FunctionInput->GetInputType(), HandleToRead.GetParameterHandleString());
			}
			
			FFormatNamedArguments Args;
			Args.Add(TEXT("AvailableNamespace"), FText::FromName(AvailableNamespace));

			const FText DisplayName = FText::Format(LOCTEXT("ReadLabelFormat", "Read from new {AvailableNamespace} parameter"), Args);
			const FText Tooltip = FText::Format(LOCTEXT("ReadToolTipFormat", "Read this input from a new parameter in the {AvailableNamespace} namespace."), Args);

			TSharedPtr<FNiagaraMenuAction_Generic> MakeAction(new FNiagaraMenuAction_Generic(
				FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::ParameterSelected, ParameterToRead),         
		        DisplayName, {CategoryName.ToString()}, {}, Tooltip, FText()));

			MakeAction->SourceData = NiagaraSourceData;

			OutAllActions.Add(MakeAction);
		}
	}

	if (bIsDataInterfaceOrObject == false && FunctionInput->SupportsCustomExpressions())
	{
		// Leaving the internal usage of bIsDataInterfaceObject that the tooltip and disabling will work properly when they're moved out of a graph action menu.
		const FText DisplayName = LOCTEXT("ExpressionLabel", "New Expression");
		const FText Tooltip = bIsDataInterfaceOrObject
			? LOCTEXT("NoExpresionsForObjects", "Expressions can not be used to set object or data interface parameters.")
			: LOCTEXT("ExpressionToolTipl", "Resolve this variable with a custom expression.");

		TSharedPtr<FNiagaraMenuAction_Generic> ExpressionAction(new FNiagaraMenuAction_Generic(
                FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::CustomExpressionSelected),
                DisplayName, {}, {}, Tooltip, FText()));

		ExpressionAction->SourceData = NiagaraSourceData;

		OutAllActions.Add(ExpressionAction);
	}

	if (bIsDataInterfaceOrObject == false)
	{
		// Leaving the internal usage of bIsDataInterfaceObject that the tooltip and disabling will work properly when they're moved out of a graph action menu.
		const FText DisplayName = LOCTEXT("ScratchLabel", "New Scratch Dynamic Input");
		const FText Tooltip = bIsDataInterfaceOrObject
			? LOCTEXT("NoScratchForObjects", "Dynamic inputs can not be used to set object or data interface parameters.")
			: LOCTEXT("ScratchToolTipl", "Create a new dynamic input in the scratch pad.");

		TSharedPtr<FNiagaraMenuAction_Generic> CreateScratchAction(new FNiagaraMenuAction_Generic(
           FNiagaraMenuAction_Generic::FOnExecuteAction::CreateSP(this, &SNiagaraStackFunctionInputValue::CreateScratchSelected),
           DisplayName, {}, {}, Tooltip, FText()));

		CreateScratchAction->SourceData = NiagaraSourceData;

		OutAllActions.Add(CreateScratchAction);
	}

	TArray<ENiagaraScriptUsage> ScriptUsagesForSharedParameters;
	FunctionInput->GetAvailableScriptUsagesForSharedParameters(ScriptUsagesForSharedParameters);
	if (ScriptUsagesForSharedParameters.Num() > 0)
	{
		UEnum* ScriptUsageEnum = StaticEnum<ENiagaraScriptUsage>();
		for (ENiagaraScriptUsage TargetUsage : ScriptUsagesForSharedParameters)
		{
			FText TargetUsageDisplayName = ScriptUsageEnum->GetDisplayNameTextByValue((int64)TargetUsage);
			const FText DisplayName = FText::Format(LOCTEXT("MoveToSharedParameterFormat", "Shared {0} parameter."), TargetUsageDisplayName);
			const FText ToolTip = FText::Format(LOCTEXT("MoveToSharedParameterToolTipFormat", "Move the value of this input to a shared {0} parameter and show that parameter value inline here."), TargetUsageDisplayName);

			TSharedPtr<FNiagaraMenuAction_Generic> ShareInTargetUsageAction(new FNiagaraMenuAction_Generic(
				FNiagaraMenuAction_Generic::FOnExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::MoveValueToLinkedSharedParameter, TOptional<ENiagaraScriptUsage>(TargetUsage)),
				DisplayName, { TEXT("Share Input Value") }, {}, ToolTip, FText()));

			ShareInTargetUsageAction->SourceData = NiagaraSourceData;
			OutAllActions.Add(ShareInTargetUsageAction);
		}

		const FText TransientDisplayName = LOCTEXT("MoveToSharedTransientParameterFormat", "Shared transient parameter.");
		const FText TransientToolTip = LOCTEXT("MoveToSharedTransientParameterToolTipFormat", "Move the value of this input to a shared transient parameter and show that parameter value inline here.");

		TSharedPtr<FNiagaraMenuAction_Generic> ShareInTransientAction(new FNiagaraMenuAction_Generic(
			FNiagaraMenuAction_Generic::FOnExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::MoveValueToLinkedSharedParameter, TOptional<ENiagaraScriptUsage>()),
			TransientDisplayName, { TEXT("Share Input Value") }, {}, TransientToolTip, FText()));

		ShareInTransientAction->SourceData = NiagaraSourceData;

		OutAllActions.Add(ShareInTransientAction);
	}

	if (FunctionInput->CanDeleteInput())
	{
		const FText DisplayName = LOCTEXT("DeleteInput", "Remove this input");
		const FText Tooltip = FText::Format(LOCTEXT("DeleteInputTooltip", "Remove input from module."), DisplayName);

		TSharedPtr<FNiagaraMenuAction_Generic> DeleteInputAction(new FNiagaraMenuAction_Generic(
                FNiagaraMenuAction_Generic::FOnExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::DeleteInput),
                FNiagaraMenuAction_Generic::FCanExecuteAction::CreateUObject(FunctionInput, &UNiagaraStackFunctionInput::CanDeleteInput),
                DisplayName, {}, {}, Tooltip, FText()));

		DeleteInputAction->SourceData = NiagaraSourceData;
		OutAllActions.Add(DeleteInputAction);
	}

	return OutAllActions;
}

TArray<FString> SNiagaraStackFunctionInputValue::OnGetCategoriesForItem(
	const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	return Item->Categories;
}

TArray<ENiagaraMenuSections> SNiagaraStackFunctionInputValue::OnGetSectionsForItem(
	const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	TArray<ENiagaraMenuSections> Sections{ENiagaraMenuSections::General};
	
	if(Item->FavoritesActionData.IsSet())
	{
		FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(SNiagaraStackFunctionInputValue::FavoriteActionsProfile);
		if(ActionsProfile.IsFavorite(Item->FavoritesActionData.GetValue()))
		{
			Sections.Add(ENiagaraMenuSections::Suggested);
		}
	}
	return Sections;
}

bool SNiagaraStackFunctionInputValue::OnCompareSectionsForEquality(const ENiagaraMenuSections& SectionA,
	const ENiagaraMenuSections& SectionB)
{
	return SectionA == SectionB;
}

bool SNiagaraStackFunctionInputValue::OnCompareSectionsForSorting(const ENiagaraMenuSections& SectionA,
	const ENiagaraMenuSections& SectionB)
{
	return SectionA < SectionB;
}

bool SNiagaraStackFunctionInputValue::OnCompareCategoriesForEquality(const FString& CategoryA, const FString& CategoryB)
{
	return CategoryA.Compare(CategoryB) == 0;
}

bool SNiagaraStackFunctionInputValue::OnCompareCategoriesForSorting(const FString& CategoryA, const FString& CategoryB)
{
	return CategoryA.Compare(CategoryB) < 0;
}

bool SNiagaraStackFunctionInputValue::OnCompareItemsForEquality(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA,
	const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB)
{
	return ItemA->DisplayName.EqualTo(ItemB->DisplayName);
}

bool SNiagaraStackFunctionInputValue::OnCompareItemsForSorting(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA,
	const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB)
{
	return ItemA->DisplayName.CompareTo(ItemB->DisplayName) == -1;
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::OnGenerateWidgetForSection(const ENiagaraMenuSections& Section)
{
	UEnum* SectionEnum = StaticEnum<ENiagaraMenuSections>();
	FText TextContent = SectionEnum->GetDisplayNameTextByValue((int64) Section);
	
	return SNew(STextBlock)
        .Text(TextContent)
        .TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.AssetPickerAssetCategoryText");
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::OnGenerateWidgetForCategory(const FString& Category)
{
	FText TextContent = FText::FromString(Category);

	return SNew(SRichTextBlock)
        .Text(TextContent)
        .DecoratorStyleSet(&FAppStyle::Get())
        .TextStyle(FNiagaraEditorStyle::Get(), "ActionMenu.HeadingTextBlock");
}

TSharedRef<SWidget> SNiagaraStackFunctionInputValue::OnGenerateWidgetForItem(
	const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	FCreateNiagaraWidgetForActionData ActionData(Item);
	ActionData.HighlightText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SNiagaraStackFunctionInputValue::GetFilterText));
	ActionData.FavoriteActionsProfileName = FavoriteActionsProfile;
	ActionData.PreviewMovieViewModel = FunctionInput->GetSystemViewModel()->GetPreviewMovieViewModel();
	return SNew(SNiagaraActionWidget, ActionData)
		.bShowTypeIfParameter(false);
}

bool SNiagaraStackFunctionInputValue::DoesItemPassCustomFilter(const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	bool bLibraryConditionFulfilled = (bLibraryOnly && Item->bIsInLibrary) || !bLibraryOnly;
	return FilterBox->IsSourceFilterActive(Item->SourceData.Source) && bLibraryConditionFulfilled;
}

void SNiagaraStackFunctionInputValue::OnItemActivated(const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	TSharedPtr<FNiagaraMenuAction_Generic> CurrentAction = StaticCastSharedPtr<FNiagaraMenuAction_Generic>(Item);

	if (CurrentAction.IsValid())
	{
		FSlateApplication::Get().DismissAllMenus();
		CurrentAction->Execute();
	}

	ActionSelector.Reset();
	FilterBox.Reset();
}

void SNiagaraStackFunctionInputValue::TriggerRefresh(const TMap<EScriptSource, bool>& SourceState)
{
	ActionSelector->RefreshAllCurrentItems();

	TArray<bool> States;
	SourceState.GenerateValueArray(States);

	int32 NumActive = 0;
	for(bool& State : States)
	{
		if(State == true)
		{
			NumActive++;
		}
	}

	ActionSelector->ExpandTree();
}

#undef LOCTEXT_NAMESPACE
