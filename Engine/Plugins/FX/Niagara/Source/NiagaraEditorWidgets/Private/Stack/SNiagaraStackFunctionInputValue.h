// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "SGraphActionMenu.h"
#include "Widgets/SItemSelector.h"
#include "Widgets/SNiagaraFilterBox.h"

class FNiagaraHLSLSyntaxHighlighter;
class UNiagaraStackFunctionInput;
class UNiagaraScript;
class SNiagaraParameterEditor;
class SBox;
class SHorizontalBox;
class IStructureDetailsView;
class SComboButton;
struct FGraphActionListBuilderBase;
class FNiagaraStackCommandContext;
class FMenuBuilder;

typedef SItemSelector<FString, TSharedPtr<FNiagaraMenuAction_Generic>, ENiagaraMenuSections> SNiagaraMenuActionSelector;

class SNiagaraStackFunctionInputValue: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnColumnWidthChanged, float)

	enum class ELayoutMode
	{
		FullRow,
		CompactInline,
		LinkedInline,
		EditDropDownOnly
	};

public:
	SLATE_BEGIN_ARGS(SNiagaraStackFunctionInputValue)
		: _LayoutMode(ELayoutMode::FullRow)
		, _CompactActionMenuButtonVisibility(EVisibility::Visible)
		{ }
		SLATE_ARGUMENT(ELayoutMode, LayoutMode)
		SLATE_ATTRIBUTE(EVisibility, CompactActionMenuButtonVisibility)
		SLATE_NAMED_SLOT(FArguments, ValueDecorator)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	TSharedRef<SWidget> ConstructValueWidgets();

	bool GetInputEnabled() const;

	bool GetEntryEnabled() const;

	void ConstructTextValueWidgets(TSharedRef<SHorizontalBox>& ValueBox, TAttribute<FText> GetText);

	void ConstructLocalValueStructWidgets(TSharedRef<SHorizontalBox>& ValueBox);

	void ConstructLinkedValueWidgets(TSharedRef<SHorizontalBox>& ValueBox);

	void ConstructLinkedSharedValueWidgets(TSharedRef<SHorizontalBox>& ValueBox);

	void ConstructDynamicValueWidgets(TSharedRef<SHorizontalBox>& ValueBox);

	void OnInputValueChanged();

	void ParameterBeginValueChange();

	void ParameterEndValueChange();

	void ParameterValueChanged(TWeakPtr<SNiagaraParameterEditor> ParameterEditor);

	void ParameterPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	FName GetLinkedValueHandleName() const;

	FText GetDataValueText() const;

	FText GetObjectAssetValueText() const;

	FText GetDynamicValueText() const;

	FText GetDefaultFunctionText() const;

	void OnExpressionTextCommitted(const FText& Name, ETextCommit::Type CommitInfo);

	FReply DynamicInputTextDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);
	FReply OnLinkedInputDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);

	class SNiagaraFunctionInputActionMenuExpander: public SExpanderArrow
	{
		SLATE_BEGIN_ARGS(SNiagaraFunctionInputActionMenuExpander) {}
			SLATE_ATTRIBUTE(float, IndentAmount)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData)
		{
			OwnerRowPtr = ActionMenuData.TableRow;
			SetIndentAmount(InArgs._IndentAmount);
			if (!ActionMenuData.RowAction.IsValid())
			{
				SExpanderArrow::FArguments SuperArgs;
				SuperArgs._IndentAmount = InArgs._IndentAmount;

				SExpanderArrow::Construct(SuperArgs, ActionMenuData.TableRow);
			}
			else
			{
				ChildSlot
				.Padding(TAttribute<FMargin>(this, &SNiagaraFunctionInputActionMenuExpander::GetCustomIndentPadding))
				[
					SNew(SBox)
				];
			}
		}

	private:
		FMargin GetCustomIndentPadding() const
		{
			return SExpanderArrow::GetExpanderPadding();
		}
	};

	static TSharedRef<SExpanderArrow> CreateCustomNiagaraFunctionInputActionExpander(const FCustomExpanderData& ActionMenuData);

	TSharedRef<SWidget> OnGetAvailableHandleMenu();

	TSharedRef<SWidget> OnGetCompactActionMenu();

	void OnFillAssignSubMenu(FMenuBuilder& MenuBuilder);

	TSharedRef<SWidget> GetVersionSelectorDropdownMenu();
	void SwitchToVersion(FNiagaraAssetVersion Version);
	FSlateColor GetVersionSelectorColor() const;

	void SetToLocalValue();

	void DynamicInputScriptSelected(UNiagaraScript* DynamicInputScript);

	void CustomExpressionSelected();

	void CreateScratchSelected();

	void ParameterSelected(FNiagaraVariableBase Parameter);
	void ParameterWithConversionSelected(FNiagaraVariableBase Parameter, UNiagaraScript* ConversionScript);

	EVisibility GetResetButtonVisibility() const;

	EVisibility GetDropdownButtonVisibility() const;

	FReply ResetButtonPressed() const;

	EVisibility GetResetToBaseButtonVisibility() const;

	FReply ResetToBaseButtonPressed() const;

	FReply OnFunctionInputDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

	bool OnFunctionInputAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation);

	TArray<TSharedPtr<FNiagaraMenuAction_Generic>> CollectDynamicInputActionsForReassign() const;

	void ShowReassignDynamicInputScriptMenu();

	bool GetLibraryOnly() const;

	void SetLibraryOnly(bool bInIsLibraryOnly);

	FReply ScratchButtonPressed() const;

	EVisibility GetVisibilityForHoverOnlyWidgets() const;

private:
	UNiagaraStackFunctionInput* FunctionInput;

	ELayoutMode LayoutMode;

	TSharedPtr<SBox> ValueContainer;
	UNiagaraStackFunctionInput::EValueMode ValueModeForGeneratedWidgets = UNiagaraStackFunctionInput::EValueMode::None;

	TSharedPtr<FStructOnScope> DisplayedLocalValueStruct;
	TSharedPtr<SNiagaraParameterEditor> LocalValueStructParameterEditor;
	TSharedPtr<IStructureDetailsView> LocalValueStructDetailsView;

	TSharedPtr<SNiagaraMenuActionSelector> ActionSelector;
	TSharedPtr<SNiagaraFilterBox> FilterBox;
	TSharedPtr<SComboButton> SetFunctionInputButton;
	TSharedPtr<FNiagaraHLSLSyntaxHighlighter> SyntaxHighlighter;

	TAttribute<EVisibility> CompactActionMenuButtonVisibilityAttribute;
	TSharedPtr<FNiagaraStackCommandContext> StackCommandContext;
	
	static bool bLibraryOnly;

	static FName FavoriteActionsProfile;

private:
	TArray<TSharedPtr<FNiagaraMenuAction_Generic>> CollectActions();
	TArray<FString> OnGetCategoriesForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	TArray<ENiagaraMenuSections> OnGetSectionsForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	bool OnCompareSectionsForEquality(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB);
	bool OnCompareSectionsForSorting(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB);
	bool OnCompareCategoriesForEquality(const FString& CategoryA, const FString& CategoryB);
	bool OnCompareCategoriesForSorting(const FString& CategoryA, const FString& CategoryB);
	bool OnCompareItemsForEquality(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA, const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB);
	bool OnCompareItemsForSorting(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA, const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB);
	TSharedRef<SWidget> OnGenerateWidgetForSection(const ENiagaraMenuSections& Section);
	TSharedRef<SWidget> OnGenerateWidgetForCategory(const FString& Category);
	TSharedRef<SWidget> OnGenerateWidgetForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	bool DoesItemPassCustomFilter(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	void OnItemActivated(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	void OnActionRowHoverEvent(const TSharedPtr<FNiagaraMenuAction_Generic>& ActionNode, bool bIsHovered);

	void TriggerRefresh(const TMap<EScriptSource, bool>& SourceState);

	FText GetFilterText() const { return ActionSelector->GetFilterText(); }
};
