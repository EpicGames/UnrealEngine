// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackIndent.h"

#include "ViewModels/Stack/NiagaraStackEntry.h"

void SNiagaraStackIndent::Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry, ENiagaraStackIndentMode InMode)
{
	StackEntry = InStackEntry;
	Mode = InMode;

	if (Mode == ENiagaraStackIndentMode::Name)
	{
		SingleIndentWidth = 16.0f;
	}
	else if (Mode == ENiagaraStackIndentMode::Value)
	{
		SingleIndentWidth = 14.0f;
	}

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(this, &SNiagaraStackIndent::GetIndentWidth)
	];
}

int32 SNiagaraStackIndent::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FSlateBrush* BackgroundBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
	const FSlateBrush* DropShadowBrush = FAppStyle::Get().GetBrush("DetailsView.ArrayDropShadow");

	int32 IndentLevel = StackEntry->GetIndentLevel();
	if (Mode == ENiagaraStackIndentMode::Name)
	{
		float PaintOffsetX = 1.0f;
		for (int32 i = 0; i < IndentLevel; ++i)
		{
			FSlateColor BackgroundColor = GetRowBackgroundColor(i);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(SingleIndentWidth, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2D((SingleIndentWidth * i) + PaintOffsetX, 0))),
				BackgroundBrush,
				ESlateDrawEffect::None,
				BackgroundColor.GetColor(InWidgetStyle));

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2D(SingleIndentWidth, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2D((SingleIndentWidth * i) + PaintOffsetX, 0))),
				DropShadowBrush);
		}
	}
	else
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(SingleIndentWidth, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2D((SingleIndentWidth * (IndentLevel - 1)), 0))),
			DropShadowBrush);
	}
	return LayerId + 1;
}

FOptionalSize SNiagaraStackIndent::GetIndentWidth() const
{
	int32 IndentLevel = StackEntry->GetIndentLevel();
	return IndentLevel * SingleIndentWidth;
}

FSlateColor SNiagaraStackIndent::GetRowBackgroundColor(int32 IndentLevel) const
{
	return FLinearColor(0, 0, 0, FMath::Min(IndentLevel + 1, 4) * 0.08f);
}