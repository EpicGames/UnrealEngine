// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Types/SlateStructs.h"

class UNiagaraStackEntry;

enum class ENiagaraStackIndentMode
{
	Name,
	Value
};

class SNiagaraStackIndent : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackIndent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry, ENiagaraStackIndentMode InMode);

private:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	FOptionalSize GetIndentWidth() const;
	FSlateColor GetRowBackgroundColor(int32 IndentLevel) const;

private:
	UNiagaraStackEntry* StackEntry;
	ENiagaraStackIndentMode Mode;
	float SingleIndentWidth;
};