// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackInheritanceIcon.h"

#include "Styling/StyleColors.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"

void SNiagaraStackInheritanceIcon::Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry)
{
	StackEntry = InStackEntry;
	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SNiagaraStackInheritanceIcon::IsVisible));
	FLinearColor IconColor = FStyleColors::Foreground.GetSpecifiedColor() * FLinearColor(1.0f, 1.0f, 1.0f, 0.4f);
	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(14)
		.HeightOverride(14)
		.ToolTipText_UObject(StackEntry, &UNiagaraStackEntry::GetInheritanceMessage)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Lock"))
			.ColorAndOpacity(IconColor)
			.DesiredSizeOverride(FVector2D(12.0f, 12.0f))
		]
	];
}

EVisibility SNiagaraStackInheritanceIcon::IsVisible() const
{
	return StackEntry->GetIsInherited() ? EVisibility::Visible : EVisibility::Collapsed;
}

