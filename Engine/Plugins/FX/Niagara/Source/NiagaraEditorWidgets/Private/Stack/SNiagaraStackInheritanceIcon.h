// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UNiagaraStackEntry;

class SNiagaraStackInheritanceIcon : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackInheritanceIcon) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry);

private:
	EVisibility IsVisible() const;

private:
	UNiagaraStackEntry* StackEntry;
};