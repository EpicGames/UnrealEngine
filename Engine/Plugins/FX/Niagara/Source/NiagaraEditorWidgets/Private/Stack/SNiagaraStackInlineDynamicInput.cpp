// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackInlineDynamicInput.h"

#include "EditorFontGlyphs.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraScript.h"
#include "Stack/SNiagaraStackFunctionInputValue.h"
#include "Styling/StyleColors.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"


#define LOCTEXT_NAMESPACE "NiagaraStackInlineDynamicInput"

const float SNiagaraStackInlineDynamicInput::TextIconSize = 16;

class SNiagaraStackInlineDynamicInputExpressionTokenBorder : public SBorder
{
public:
	DECLARE_DELEGATE_TwoParams(FHoveredChanged, FGuid /*InputId*/, bool /*bIsHovered*/);

public:
	SLATE_BEGIN_ARGS(SNiagaraStackInlineDynamicInputExpressionTokenBorder)
		: _HoveredInputId(FGuid())
		{}
		SLATE_ATTRIBUTE(FGuid, HoveredInputId)
		SLATE_ATTRIBUTE(FMargin, Padding)
		SLATE_EVENT(FHoveredChanged, OnHoveredChanged)
		SLATE_DEFAULT_SLOT(FArguments, Content);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* FunctionInput, FGuid InInputId)
	{
		FunctionInputWeak = FunctionInput;
		InputId = InInputId;
		HoveredInputIdAttribute = InArgs._HoveredInputId;
		OnHoveredChangedDelegate = InArgs._OnHoveredChanged;
		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(this, &SNiagaraStackInlineDynamicInputExpressionTokenBorder::GetBorderColor)
			.VAlign(VAlign_Center)
			.Padding(InArgs._Padding)
			[
				InArgs._Content.Widget
			]);
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		const bool bWasHovered = IsHovered();
		SBorder::OnMouseEnter(MyGeometry, MouseEvent);
		if (bWasHovered == false && IsHovered())
		{
			OnHoveredChangedDelegate.ExecuteIfBound(InputId, true);
		}
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		const bool bWasHovered = IsHovered();
		SBorder::OnMouseLeave(MouseEvent);
		if (bWasHovered && IsHovered() == false)
		{
			OnHoveredChangedDelegate.ExecuteIfBound(InputId, false);
		}
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		bool bHandled = false;
		if (FunctionInputWeak.IsValid())
		{
			bHandled = FunctionInputWeak->OpenSourceAsset();
		}
		return bHandled 
			? FReply::Handled()
			: SBorder::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
	}

private:
	FSlateColor GetBorderColor() const
	{
		return InputId == HoveredInputIdAttribute.Get() ? FLinearColor(1.0f, 1.0f, 1.0f, 0.25f) : FLinearColor::Transparent;
	}

private:
	TWeakObjectPtr<UNiagaraStackFunctionInput> FunctionInputWeak;
	FGuid InputId;
	TAttribute<FGuid> HoveredInputIdAttribute;
	FHoveredChanged OnHoveredChangedDelegate;
};

void SNiagaraStackInlineDynamicInput::Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput)
{
	RootFunctionInput = InFunctionInput;
	RootFunctionInput->OnStructureChanged().AddSP(this, &SNiagaraStackInlineDynamicInput::RootFunctionInputStructureChanged);
	RootFunctionInput->OnValueChanged().AddSP(this, &SNiagaraStackInlineDynamicInput::RootFunctionInputValueChanged);
	bSingleValueMode = false;
	ConstructChildren();
}

SNiagaraStackInlineDynamicInput::FInputDisplayEntries SNiagaraStackInlineDynamicInput::CollectInputDisplayEntriesRecursive(UNiagaraStackFunctionInput* Input)
{
	FInputDisplayEntries InputDisplayEntries;
	InputDisplayEntries.InputEntry = Input;
	if (Input->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic)
	{
		TArray<UNiagaraStackFunctionInput*> ChildInputs;
		Input->GetFilteredChildInputs(ChildInputs);
		for (UNiagaraStackFunctionInput* ChildInput : ChildInputs)
		{
			InputDisplayEntries.ChildInputDisplayEntries.Add(CollectInputDisplayEntriesRecursive(ChildInput));
		}
	}
	else if (Input->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Data && Input->GetChildDataObject() != nullptr)
	{
		InputDisplayEntries.ChildObjectEntry = Input->GetChildDataObject();
	}
	return InputDisplayEntries;
}

void SNiagaraStackInlineDynamicInput::RootFunctionInputStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	FInputDisplayEntries NewInputDisplayEntries = CollectInputDisplayEntriesRecursive(RootFunctionInput);
	if(RootInputDisplayEntries != NewInputDisplayEntries)
	{
		ConstructChildren();
	}
}

void SNiagaraStackInlineDynamicInput::RootFunctionInputValueChanged()
{
	FInputDisplayEntries NewInputDisplayEntries = CollectInputDisplayEntriesRecursive(RootFunctionInput);
	if (RootInputDisplayEntries != NewInputDisplayEntries)
	{
		ConstructChildren();
	}
}

bool SNiagaraStackInlineDynamicInput::TryMatchFormatInputs(
	const TArray<FInputDisplayEntries>& ChildInputDisplayEntries,
	const TArray<FNiagaraInlineDynamicInputFormatToken>& InlineFormat,
	TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>& OutMatchedFormatInputs)
{
	if (InlineFormat.Num() == 0 || GbEnableCustomInlineDynamicInputFormats == false)
	{
		return false;
	}

	TArray<FString> MatchedInputNames;
	for (const FNiagaraInlineDynamicInputFormatToken& InlineFormatToken : InlineFormat)
	{
		if (InlineFormatToken.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
		{
			const FInputDisplayEntries* MatchingChildInputDisplayEntry = ChildInputDisplayEntries.FindByPredicate([InlineFormatToken](const FInputDisplayEntries& ChildInputDisplayEntry)
				{ return ChildInputDisplayEntry.InputEntry->GetInputParameterHandle().GetName().ToString() == InlineFormatToken.Value; });
			if (MatchingChildInputDisplayEntry == nullptr)
			{
				OutMatchedFormatInputs.Empty();
				return false;
			}
			else
			{
				OutMatchedFormatInputs.Add(TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>(InlineFormatToken, *MatchingChildInputDisplayEntry));
				MatchedInputNames.AddUnique(InlineFormatToken.Value);
			}
		}
		else if (InlineFormatToken.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Decorator || InlineFormatToken.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::LineBreak)
		{
			OutMatchedFormatInputs.Add(TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>(InlineFormatToken, FInputDisplayEntries()));
		}
	}

	if (MatchedInputNames.Num() != ChildInputDisplayEntries.Num())
	{
		OutMatchedFormatInputs.Empty();
		return false;
	}
	else
	{
		return true;
	}
}

void SNiagaraStackInlineDynamicInput::ConstructChildren()
{
	WidthSynchronizers.Empty();
	if (RootFunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::Expression)
	{
		TSharedPtr<SWrapBox> ExpressionWrapBox;
		TSharedPtr<SVerticalBox> DataInterfaceVerticalBox;

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ExpressionWrapBox, SWrapBox)
				.UseAllottedSize(true)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(DataInterfaceVerticalBox, SVerticalBox)
			]
		];

		RootInputDisplayEntries = CollectInputDisplayEntriesRecursive(RootFunctionInput);
		ConstructWrappedExpressionInputWidgets(ExpressionWrapBox.ToSharedRef(), DataInterfaceVerticalBox.ToSharedRef(), RootInputDisplayEntries, FGuid::NewGuid(), true);
	}
	else if (RootFunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHorizontal || 
		RootFunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphVertical ||
		RootFunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHybrid)
	{
		TSharedPtr<SGridPanel> GraphPanel;
		TSharedRef<SVerticalBox> DataInterfaceVerticalBox = SNew(SVerticalBox);
		RootInputDisplayEntries = CollectInputDisplayEntriesRecursive(RootFunctionInput);
		TArray<FInputDisplayEntries> InputDisplayEntries;
		InputDisplayEntries.Add(RootInputDisplayEntries);
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0, 5, 0, 5)
			[
				RootFunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphHorizontal
					? ConstructHorizontalGraphInputWidgets(InputDisplayEntries, nullptr, DataInterfaceVerticalBox, TSharedPtr<FWidthSynchronize>())
					: RootFunctionInput->GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::GraphVertical
						? ConstructVerticalGraphInputWidgets(InputDisplayEntries, nullptr, DataInterfaceVerticalBox)
						: ConstructHybridGraphInputWidgets(InputDisplayEntries, nullptr, DataInterfaceVerticalBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				DataInterfaceVerticalBox
			]
		];
	}
	else
	{
		ChildSlot[SNullWidget::NullWidget];
	}
}

TSharedRef<SWidget> ConstructFunctionNameWithDynamicInputIcon(FText FunctionDisplayName, FNiagaraTypeDefinition InputType)
{
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(0, 0, 5, 0)
	[
		SNew(SBorder)
		.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.InputTypeBorder"))
		.BorderBackgroundColor(FLinearColor::White)
		.Padding(FMargin(2, 2, 2, 2))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ToolTipText(FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic))
				.ColorAndOpacity(UEdGraphSchema_Niagara::GetTypeColor(InputType))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ToolTipText(FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic))
				.ColorAndOpacity(FLinearColor::White)
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.DynamicInput"))
			]
		]
	]
	+ SHorizontalBox::Slot()
	[
		SNew(STextBlock)
		.Text(FunctionDisplayName)
	];
}

FText GetDynamicInputFunctionDisplayName(const UNiagaraNodeFunctionCall* FunctionCallNode)
{
	return FunctionCallNode != nullptr && FunctionCallNode->FunctionScript != nullptr
		? FText::FromString(FName::NameToDisplayString(FunctionCallNode->FunctionScript->GetName(), false))
		: LOCTEXT("UnknownFunctionDisplayName", "Unknown Function");
}

FText GetDynamicInputToolTip(const UNiagaraStackFunctionInput* FunctionInput)
{
	return FText::Format(LOCTEXT("DynamicInputNameAndToolTipFormat", "{0}\n\n{1}"), GetDynamicInputFunctionDisplayName(FunctionInput->GetDynamicInputNode()), FunctionInput->GetValueToolTip());
}

void SNiagaraStackInlineDynamicInput::ConstructWrappedExpressionInputWidgets(TSharedRef<SWrapBox> ExpressionWrapBox, TSharedRef<SVerticalBox> DataInterfaceVerticalBox, const FInputDisplayEntries& InputDisplayEntries, FGuid InputId, bool bIsRootInput)
{
	UNiagaraStackFunctionInput* FunctionInput = InputDisplayEntries.InputEntry;
	if (FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic)
	{
		ConstructWrappedExpressionDynamicInputWidgets(ExpressionWrapBox, DataInterfaceVerticalBox, InputDisplayEntries, InputId, bIsRootInput);
	}
	else
	{
		TSharedPtr<SBorder> InputValueBorder;
		ExpressionWrapBox->AddSlot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InputValueBorder, SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
				.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
				.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
				.ToolTipText_UObject(FunctionInput, &UNiagaraStackFunctionInput::GetDisplayName)
				.Padding(FMargin(4, 1, 0, 1))
			];

		TSharedRef<SWidget> InputValueWidget = SNew(SNiagaraStackFunctionInputValue, FunctionInput)
			.LayoutMode(SNiagaraStackFunctionInputValue::ELayoutMode::CompactInline)
			.CompactActionMenuButtonVisibility(this, &SNiagaraStackInlineDynamicInput::GetCompactActionMenuButtonVisibility, InputId);

		if (FunctionInput->GetHasEditCondition())
		{
			InputValueBorder->SetContent(SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.Visibility(this, &SNiagaraStackInlineDynamicInput::GetEditConditionCheckBoxVisibility, FunctionInput)
					.IsChecked(this, &SNiagaraStackInlineDynamicInput::GetEditConditionCheckState, FunctionInput)
					.OnCheckStateChanged(this, &SNiagaraStackInlineDynamicInput::OnEditConditionCheckStateChanged, FunctionInput)
				]
				+ SHorizontalBox::Slot()
				[
					InputValueWidget
				]);
		}
		else
		{
			InputValueBorder->SetContent(InputValueWidget);
		}

		if (FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Data)
		{
			UNiagaraStackObject* ChildDataObject = FunctionInput->GetChildDataObject();
			if (ChildDataObject != nullptr)
			{
				ConstructDataInterfaceWidgets(DataInterfaceVerticalBox, InputDisplayEntries, InputId);
			}
		}
	}
}

TSharedRef<SWidget> ConstructDecoratorText(const FString& DisplayString)
{
	return SNew(STextBlock)
		.Text(FText::FromString(DisplayString))
		.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.InlineDynamicInputFormatText")
		.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic)));
}

TSharedRef<SWidget> ConstructDynamicInputLabel(const UNiagaraStackFunctionInput* FunctionInput)
{
	return SNew(SHorizontalBox)
	.ToolTipText(FunctionInput->GetValueToolTip())
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	.Padding(0, 0, 3, 0)
	[
		SNew(SBorder)
		.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.InputTypeBorder"))
		.BorderBackgroundColor(FLinearColor::White)
		.Padding(FMargin(2, 2, 2, 2))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ToolTipText(FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic))
				.ColorAndOpacity(UEdGraphSchema_Niagara::GetTypeColor(FunctionInput->GetInputType()))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ToolTipText(FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode::Dynamic))
				.ColorAndOpacity(FLinearColor::White)
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.DynamicInput"))
			]
		]
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text(GetDynamicInputFunctionDisplayName(FunctionInput->GetDynamicInputNode()))
	];
}

void SNiagaraStackInlineDynamicInput::ConstructWrappedExpressionDynamicInputWidgets(TSharedRef<SWrapBox> ExpressionWrapBox, TSharedRef<SVerticalBox> DataInterfaceVerticalBox, const FInputDisplayEntries& InputDisplayEntries, FGuid InputId, bool bIsRootInput)
{
	UNiagaraStackFunctionInput* FunctionInput = InputDisplayEntries.InputEntry;

	TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>> OutMatchedFormatInputs;
	if (FunctionInput->GetDynamicInputNode() != nullptr && TryMatchFormatInputs(InputDisplayEntries.ChildInputDisplayEntries, FunctionInput->GetDynamicInputNode()->GetScriptData()->InlineExpressionFormat, OutMatchedFormatInputs))
	{
		// If this isn't the root input and the matched format doesn't start and end with decorators, we add parentheses before and after for readability.
		bool bAddParentheses = 
			bIsRootInput == false &&
			OutMatchedFormatInputs.Num() > 2 &&
			(OutMatchedFormatInputs[0].Key.Usage != ENiagaraInlineDynamicInputFormatTokenUsage::Decorator || OutMatchedFormatInputs.Last(0).Key.Usage != ENiagaraInlineDynamicInputFormatTokenUsage::Decorator);

		if (bAddParentheses)
		{
			ExpressionWrapBox->AddSlot()
			[
				SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
				.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
				.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
				.ToolTipText(GetDynamicInputToolTip(FunctionInput))
				.Padding(FMargin(3, 0))
				[
					ConstructDecoratorText("(")
				]
			];
		}

		for (const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair : OutMatchedFormatInputs)
		{
			if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
			{
				ConstructWrappedExpressionInputWidgets(ExpressionWrapBox, DataInterfaceVerticalBox, FormatTokenInputPair.Value, FGuid::NewGuid(), false);
			}
			else if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Decorator)
			{
				ExpressionWrapBox->AddSlot()
				[
					SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
					.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
					.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
					.ToolTipText(GetDynamicInputToolTip(FunctionInput))
					.Padding(FMargin(3, 0, 6, 0))
					[
						ConstructDecoratorText(FormatTokenInputPair.Key.Value)
					]
				];
			}
		}

		if (bAddParentheses)
		{
			ExpressionWrapBox->AddSlot()
			[
				SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
				.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
				.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
				.ToolTipText(GetDynamicInputToolTip(FunctionInput))
				.Padding(FMargin(3, 0))
				[
					ConstructDecoratorText(")")
				]
			];
		}
	}
	else
	{
		ExpressionWrapBox->AddSlot()
		[
			SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
			.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
			.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
			.Padding(FMargin(3, 0))
			[
				ConstructDynamicInputLabel(FunctionInput)
			]
		];

		ExpressionWrapBox->AddSlot()
		[
			SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
			.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
			.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
			.ToolTipText(GetDynamicInputToolTip(FunctionInput))
			.Padding(FMargin(3, 0))
			[
				ConstructDecoratorText("(")
			]
		];

		bool bFirst = true;
		for (const FInputDisplayEntries& ChildInputDisplayEntries : InputDisplayEntries.ChildInputDisplayEntries)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				ExpressionWrapBox->AddSlot()
				[
					SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
					.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
					.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
					.ToolTipText(GetDynamicInputToolTip(FunctionInput))
					.Padding(FMargin(3, 0, 6, 0))
					[
						ConstructDecoratorText(",")
					]
				];
			}
			ConstructWrappedExpressionInputWidgets(ExpressionWrapBox, DataInterfaceVerticalBox, ChildInputDisplayEntries, FGuid::NewGuid(), false);
		}

		ExpressionWrapBox->AddSlot()
		[
			SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
			.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
			.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
			.ToolTipText(GetDynamicInputToolTip(FunctionInput))
			.Padding(FMargin(3, 0))
			[
				ConstructDecoratorText(")")
			]
		];
	}

	ExpressionWrapBox->AddSlot()
	[
		SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
		.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
		.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
		.ToolTipText(GetDynamicInputToolTip(FunctionInput))
		[
			SNew(SNiagaraStackFunctionInputValue, FunctionInput)
			.LayoutMode(SNiagaraStackFunctionInputValue::ELayoutMode::EditDropDownOnly)
			.CompactActionMenuButtonVisibility(this, &SNiagaraStackInlineDynamicInput::GetCompactActionMenuButtonVisibility, InputId)
		]
	];
}

const FSlateBrush* GetHorizontalGraphNodeBorderBrush(int32 RowIndex, int32 TotalRows)
{
	if (TotalRows == 1)
	{
		// Single
		return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Single");
	}
	else
	{
		if (RowIndex == 0)
		{
			// Top
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Top");
		}
		else if (RowIndex < TotalRows - 1)
		{
			// Middle
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Middle");
		}
		else
		{
			// Bottom
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Bottom");
		}
	}
}

const FSlateBrush* GetVerticalGraphNodeBorderBrush(int32 ColumnIndex, int32 TotalColumns, bool bHasHeaderRow)
{
	if (TotalColumns == 1)
	{
		// Single
		return bHasHeaderRow 
			? FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Bottom")
			: FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Single");
	}
	else
	{
		if (ColumnIndex == 0)
		{
			// Left
			return bHasHeaderRow
				? FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Corner.BottomLeft")
				: FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Left");
		}
		else if (ColumnIndex < TotalColumns - 1)
		{
			// Middle
			return bHasHeaderRow
				? FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Edge.Bottom")
				: FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Middle");
		}
		else
		{
			// Right
			return bHasHeaderRow
				? FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Corner.BottomRight")
				: FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Right");
		}
	}
}

const FSlateBrush* GetHybridGraphNodeBorderBrush(int32 NodeColumnIndex, int32 TotalNodeColumns, int32 ColumnColumnIndex, int32 TotalColumnColumns)
{
	if (TotalNodeColumns == 1 && TotalColumnColumns == 1)
	{
		// Single
		return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Single");
	}
	else
	{
		if (NodeColumnIndex == 0 && ColumnColumnIndex == 0)
		{
			// Left
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Left");
		}
		else if (NodeColumnIndex < TotalNodeColumns - 1 || ColumnColumnIndex < TotalColumnColumns - 1)
		{
			// Middle
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Middle");
		}
		else
		{
			// Right
			return FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Right");
		}
	}
}

FMargin GetHorizontalGraphNodeBorderPadding(int32 RowIndex, int32 TotalRows, bool bHasChildren)
{
	if (TotalRows == 1 || bHasChildren)
	{
		// Single
		return FMargin(8, 4);
	}
	else
	{
		if (RowIndex == 0)
		{
			// Top
			return FMargin(8, 4, 8, 2);
		}
		else if (RowIndex < TotalRows - 1)
		{
			// Middle
			return FMargin(8, 2);
		}
		else
		{
			// Bottom
			return FMargin(8, 2, 8, 4);
		}
	}
}

FMargin GetVerticalGraphNodeBorderPadding(int32 ColumnIndex, int32 TotalColumns)
{
	if (TotalColumns == 1)
	{
		// Single
		return FMargin(6, 4);
	}
	else
	{
		if (ColumnIndex == 0)
		{
			// Left
			return FMargin(6, 4, 2, 4);
		}
		else if (ColumnIndex < TotalColumns - 1)
		{
			// Middle
			return FMargin(2, 4);
		}
		else
		{
			// Right
			return FMargin(2, 4, 6, 4);
		}
	}
}

FMargin GetHybridGraphNodeBorderPadding(int32 NodeColumnIndex, int32 TotalNodeColumns, int32 ColumnColumnIndex, int32 TotalColumnColumns)
{
	if (TotalNodeColumns == 1 && TotalColumnColumns == 1)
	{
		// Single
		return FMargin(6, 4);
	}
	else
	{
		if (NodeColumnIndex == 0 && ColumnColumnIndex == 0)
		{
			// Left
			return FMargin(6, 4, 2, 4);
		}
		else if (NodeColumnIndex < TotalNodeColumns - 1 || ColumnColumnIndex < TotalColumnColumns - 1)
		{
			// Middle
			return FMargin(2, 4);
		}
		else
		{
			// Right
			return FMargin(2, 4, 6, 4);
		}
	}
}

EVerticalAlignment GetHorizontalGraphNodeGridVerticalAlignment(int32 RowIndex, int32 TotalRows)
{
	if (TotalRows == 1)
	{
		return VAlign_Top;
	}

	if (RowIndex == TotalRows - 1)
	{
		return VAlign_Top;
	}

	return VAlign_Fill;
}

EHorizontalAlignment GetVerticalGraphNodeGridHorizontalAlignment(int32 ColumnIndex, int32 TotalColumns)
{
	if (TotalColumns == 1)
	{
		return HAlign_Left;
	}

	if (ColumnIndex == TotalColumns - 1)
	{
		return HAlign_Left;
	}

	return HAlign_Fill;
}

EHorizontalAlignment GetHybridGraphNodeGridHorizontalAlignment(int32 NodeColumnIndex, int32 TotalNodeColumns, int32 ColumnColumnIndex, int32 TotalColumnColumns)
{
	if (NodeColumnIndex < TotalNodeColumns - 1 && ColumnColumnIndex == TotalColumnColumns - 1)
	{
		return HAlign_Fill;
	}
	return HAlign_Left;
}

FMargin GetHorizontalGraphNodeGridPadding(int32 RowIndex, int32 TotalRows)
{
	if (TotalRows == 1)
	{
		// Single
		return FMargin(0, 2, 0, 2);
	}
	else
	{
		if (RowIndex == 0)
		{
			// Left
			return FMargin(0, 2, 0, 0);
		}
		else if (RowIndex < TotalRows - 1)
		{
			// Middle
			return FMargin(0);
		}
		else
		{
			// Right
			return FMargin(0, 0, 0, 2);
		}
	}
}

FMargin GetVerticalGraphNodeGridPadding(int32 ColumnIndex, int32 TotalColumns)
{
	if (TotalColumns == 1)
	{
		// Single
		return FMargin(2, 0, 2, 0);
	}
	else
	{
		if (ColumnIndex == 0)
		{
			// Left
			return FMargin(2, 0, 0, 0);
		}
		else if (ColumnIndex < TotalColumns - 1)
		{
			// Middle
			return FMargin(0);
		}
		else
		{
			// Right
			return FMargin(0, 0, 2, 0);
		}
	}
}

void SNiagaraStackInlineDynamicInput::FWidthSynchronize::AddWidget(TSharedRef<SWidget> InWidget)
{
	WeakWidgets.Add(TWeakPtr<SWidget>(InWidget));
}

FOptionalSize SNiagaraStackInlineDynamicInput::FWidthSynchronize::GetMaxDesiredWidth() const
{
	float MaxDesiredWidth = TNumericLimits<float>::Lowest();
	for (const TWeakPtr<SWidget>& WeakWidget : WeakWidgets)
	{
		TSharedPtr<SWidget> Widget = WeakWidget.Pin();
		if (Widget.IsValid())
		{
			MaxDesiredWidth = FMath::Max(MaxDesiredWidth, Widget->GetDesiredSize().X);
		}
	}
	return MaxDesiredWidth > 0 
		? FOptionalSize(MaxDesiredWidth)
		: FOptionalSize();
}

TSharedRef<SWidget> SNiagaraStackInlineDynamicInput::ConstructGraphInputWidgetForFormatTokenInputPair(
	const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair,
	const UNiagaraStackFunctionInput* OwningInput,
	FGuid InputId)
{
	if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
	{
		UNiagaraStackFunctionInput* FunctionInput = FormatTokenInputPair.Value.InputEntry;
		TSharedPtr<SWidget> InputWidget;
		if (FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic)
		{
			// Inputs which reference another dynamic input just show a label for the input name and then a drop-down arrow.
			return 
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_UObject(FunctionInput, &UNiagaraStackEntry::GetDisplayName)
				]
				//+ SHorizontalBox::Slot()
				//[
				//	SNew
				//]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SNiagaraStackFunctionInputValue, FunctionInput)
					.LayoutMode(SNiagaraStackFunctionInputValue::ELayoutMode::EditDropDownOnly)
				];
		}
		else
		{
			// Other inputs display a label and display the standard value control.
			return
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 0, 5, 0)
				[
					SNew(STextBlock)
					.Text_UObject(FunctionInput, &UNiagaraStackEntry::GetDisplayName)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 2, 0, 2)
				[
					SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
					.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
					.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
					.ToolTipText_UObject(FunctionInput, &UNiagaraStackFunctionInput::GetDisplayName)
					[
						SNew(SNiagaraStackFunctionInputValue, FunctionInput)
						.LayoutMode(SNiagaraStackFunctionInputValue::ELayoutMode::CompactInline)
					]
				];
		}
	}
	else if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Decorator)
	{
		return 
			SNew(SBox)
			.ToolTipText(GetDynamicInputToolTip(OwningInput))
			.Padding(2)
			[
				ConstructDecoratorText(FormatTokenInputPair.Key.Value)
			];
	}
	return SNullWidget::NullWidget;
}

void SNiagaraStackInlineDynamicInput::GenerateFormattedGroups(
	const TArray<FInputDisplayEntries>& InputDisplayEntries,
	const TArray<FNiagaraInlineDynamicInputFormatToken>& InlineFormat,
	bool bIgnoreLinebreaks,
	bool bTreatInputsWithoutChildrenLikeDecorators,
	bool& bOutFormatMatched,
	TArray<TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>>& OutFormattedGroups)
{
	// Try to match the inputs to the format.  If they can not be matched then use a generic format.
	TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>> FormattedInputs;
	bOutFormatMatched = TryMatchFormatInputs(InputDisplayEntries, InlineFormat, FormattedInputs);
	if(bOutFormatMatched == false)
	{
		for (const FInputDisplayEntries& InputDisplayEntry : InputDisplayEntries)
		{
			FormattedInputs.Add(TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>(
				FNiagaraInlineDynamicInputFormatToken(ENiagaraInlineDynamicInputFormatTokenUsage::Input, FString()),
				InputDisplayEntry));
		}
	}

	// Sort the formatters and inputs into groups for rows and columns
	TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>> CurrentFormattedGroup;
	bool bCurrentGroupHasInput = false;
	for (const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair : FormattedInputs)
	{
		ENiagaraInlineDynamicInputFormatTokenUsage PairMode;
		if (bTreatInputsWithoutChildrenLikeDecorators &&
			FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input &&
			FormatTokenInputPair.Value.ChildInputDisplayEntries.Num() == 0)
		{
			PairMode = ENiagaraInlineDynamicInputFormatTokenUsage::Decorator;
		}
		else
		{
			PairMode = FormatTokenInputPair.Key.Usage;
		}

		switch (PairMode)
		{
		case ENiagaraInlineDynamicInputFormatTokenUsage::Input:
			if (bCurrentGroupHasInput)
			{
				OutFormattedGroups.Add(CurrentFormattedGroup);
				CurrentFormattedGroup.Empty();
			}
			CurrentFormattedGroup.Add(FormatTokenInputPair);
			bCurrentGroupHasInput = true;
			break;
		case ENiagaraInlineDynamicInputFormatTokenUsage::Decorator:
			CurrentFormattedGroup.Add(FormatTokenInputPair);
			break;
		case ENiagaraInlineDynamicInputFormatTokenUsage::LineBreak:
			if (bIgnoreLinebreaks == false && CurrentFormattedGroup.Num() > 0)
			{
				OutFormattedGroups.Add(CurrentFormattedGroup);
				CurrentFormattedGroup.Empty();
				bCurrentGroupHasInput = false;
			}
			break;
		}
	}
	if (CurrentFormattedGroup.Num() > 0)
	{
		OutFormattedGroups.Add(CurrentFormattedGroup);
		CurrentFormattedGroup.Empty();
	}
}

TSharedRef<SWidget> SNiagaraStackInlineDynamicInput::ConstructHorizontalGraphInputWidgets(
	const TArray<FInputDisplayEntries>& InputDisplayEntries,
	const UNiagaraStackFunctionInput* OwningInput,
	TSharedRef<SVerticalBox> DataInterfaceVerticalBox,
	TSharedPtr<FWidthSynchronize> InputWidthSynchronizer)
{
	const UNiagaraNodeFunctionCall* DynamicInputFunctionCall = 
		OwningInput != nullptr && OwningInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic
			? OwningInput->GetDynamicInputNode() : nullptr;
	bool bFormatMatched;
	TArray<TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>> FormattedRows;
	TArray<FNiagaraInlineDynamicInputFormatToken> InlineFormat = DynamicInputFunctionCall != nullptr 
		? DynamicInputFunctionCall->GetScriptData()->InlineGraphFormat 
		: TArray<FNiagaraInlineDynamicInputFormatToken>();

	bool bIgnoreLinebreaks = false;
	bool bTreatInputsWithoutChildrenLikeDecorators = false;
	GenerateFormattedGroups(InputDisplayEntries, InlineFormat, bIgnoreLinebreaks, bTreatInputsWithoutChildrenLikeDecorators, bFormatMatched, FormattedRows);

	// Generate the rows.
	TSharedRef<SGridPanel> InputGrid = SNew(SGridPanel);
	int32 GraphNodeRowIndex = 0;
	int32 GraphNodeHeaderRowCount = bFormatMatched == false && DynamicInputFunctionCall != nullptr ? 1 : 0;
	int32 TotalGraphNodeRowCount = FormattedRows.Num() + GraphNodeHeaderRowCount;
	int32 ChildInputsRowIndex = 0;

	auto AddGraphNodeRow = [InputGrid, TotalGraphNodeRowCount, InputWidthSynchronizer](int32 GraphNodeRowIndex, bool bHasChildren, TSharedRef<SWidget> Widget)
	{
		TSharedPtr<SWidget> RowWidget;
		if (InputWidthSynchronizer.IsValid())
		{
			InputWidthSynchronizer->AddWidget(Widget);
			RowWidget = SNew(SBox)
				.WidthOverride(InputWidthSynchronizer.ToSharedRef(), &FWidthSynchronize::GetMaxDesiredWidth)
				.Content()
				[
					Widget
				];
		}
		else
		{
			RowWidget = Widget;
		}
		InputGrid->AddSlot(2, GraphNodeRowIndex)
		.VAlign(GetHorizontalGraphNodeGridVerticalAlignment(GraphNodeRowIndex, TotalGraphNodeRowCount))
		.Padding(GetHorizontalGraphNodeGridPadding(GraphNodeRowIndex, TotalGraphNodeRowCount))
		[
			SNew(SBorder)
			.BorderImage(GetHorizontalGraphNodeBorderBrush(GraphNodeRowIndex, TotalGraphNodeRowCount))
			.BorderBackgroundColor(FStyleColors::DropdownOutline)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(GetHorizontalGraphNodeBorderPadding(GraphNodeRowIndex, TotalGraphNodeRowCount, bHasChildren))
			[
				RowWidget.ToSharedRef()
			]
		];
	};

	if (bFormatMatched == false && DynamicInputFunctionCall != nullptr)
	{
		// For unformatted inputs add a header with the dynamic input icon and the function name.
		AddGraphNodeRow(GraphNodeRowIndex, false, ConstructDynamicInputLabel(OwningInput));
		GraphNodeRowIndex++;
	}

	// Generate row widgets.
	TSharedRef<FWidthSynchronize> ChildInputWidthSynchronizer = MakeShared<FWidthSynchronize>();
	WidthSynchronizers.Add(ChildInputWidthSynchronizer);
	for (const TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>& FormattedRow : FormattedRows)
	{
		FGuid InputId = FGuid::NewGuid();
		TOptional<FInputDisplayEntries> RowDisplayEntries;
		TSharedPtr<SWidget> RowWidget;
		if(FormattedRow.Num() == 1)
		{
			RowWidget = ConstructGraphInputWidgetForFormatTokenInputPair(FormattedRow[0], OwningInput, InputId);
			if(FormattedRow[0].Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
			{
				RowDisplayEntries = FormattedRow[0].Value;
			}
		}
		else
		{
			TSharedRef<SHorizontalBox> RowBox = SNew(SHorizontalBox);
			RowWidget = RowBox;
			for (const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair : FormattedRow)
			{
				RowBox->AddSlot()
				.AutoWidth()
				[
					ConstructGraphInputWidgetForFormatTokenInputPair(FormatTokenInputPair, OwningInput, InputId)
				];
				if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
				{
					RowDisplayEntries = FormatTokenInputPair.Value;
				}
			}
		}

		if (RowDisplayEntries.IsSet() && RowDisplayEntries->InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Data)
		{
			UNiagaraStackObject* ChildDataObject = RowDisplayEntries->InputEntry->GetChildDataObject();
			if (ChildDataObject != nullptr)
			{
				ConstructDataInterfaceWidgets(DataInterfaceVerticalBox, RowDisplayEntries.GetValue(), InputId);
			}
		}

		bool bIsDynamicInput = RowDisplayEntries.IsSet() && RowDisplayEntries->InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic;
		AddGraphNodeRow(GraphNodeRowIndex, bIsDynamicInput, RowWidget.ToSharedRef());

		// Add additional widgets for dynamic inputs.
		if (RowDisplayEntries.IsSet() && bIsDynamicInput)
		{
			// Connection arrow.
			FMargin BoxPadding = GetHorizontalGraphNodeBorderPadding(GraphNodeRowIndex, TotalGraphNodeRowCount, true);
			BoxPadding.Left = 2;
			BoxPadding.Right = 2;
			InputGrid->AddSlot(1, GraphNodeRowIndex)
			.VAlign(GetHorizontalGraphNodeGridVerticalAlignment(GraphNodeRowIndex, TotalGraphNodeRowCount))
			.Padding(GetHorizontalGraphNodeGridPadding(GraphNodeRowIndex, TotalGraphNodeRowCount))
			[
				SNew(SBox)
				.VAlign(VAlign_Top)
				.Padding(BoxPadding)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Long_Arrow_Right)
				]
			];

			// Child inputs.
			auto FindNextNodeRowWithChildren = [FormattedRows](int32 StartRow) -> int32
			{
				for (int32 NodeRow = StartRow; NodeRow < FormattedRows.Num(); NodeRow++)
				{
					for (const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& RowItem : FormattedRows[NodeRow])
					{
						if (RowItem.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input && RowItem.Value.ChildInputDisplayEntries.Num() > 0)
						{
							return NodeRow;
						}
					}
				}
				return INDEX_NONE;
			};

			int32 StartRow = ChildInputsRowIndex;
			int32 NextRowWithChildren = FindNextNodeRowWithChildren(GraphNodeRowIndex + 1);
			int32 EndRow = 
				GraphNodeHeaderRowCount + 
				(NextRowWithChildren == INDEX_NONE ? FormattedRows.Num() - 1 : NextRowWithChildren - 1);
			int32 RowSpan = EndRow - StartRow + 1;
			InputGrid->AddSlot(0, StartRow)
			.RowSpan(RowSpan)
			.HAlign(HAlign_Right)
			[
				ConstructHorizontalGraphInputWidgets(RowDisplayEntries->ChildInputDisplayEntries, RowDisplayEntries->InputEntry, DataInterfaceVerticalBox, ChildInputWidthSynchronizer)
			];
			ChildInputsRowIndex += RowSpan;
		}
		GraphNodeRowIndex++;
	}

	return InputGrid;
}

TSharedRef<SWidget> SNiagaraStackInlineDynamicInput::ConstructVerticalGraphInputWidgets(
	const TArray<FInputDisplayEntries>& InputDisplayEntries,
	const UNiagaraStackFunctionInput* OwningInput,
	TSharedRef<SVerticalBox> DataInterfaceVerticalBox)
{
	const UNiagaraNodeFunctionCall* DynamicInputFunctionCall =
		OwningInput != nullptr && OwningInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic
			? OwningInput->GetDynamicInputNode() : nullptr;
	bool bFormatMatched;
	TArray<TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>> FormattedColumns;
	TArray<FNiagaraInlineDynamicInputFormatToken> InlineFormat = DynamicInputFunctionCall != nullptr
		? DynamicInputFunctionCall->GetScriptData()->InlineGraphFormat
		: TArray<FNiagaraInlineDynamicInputFormatToken>();

	bool bIgnoreLinebreaks = true;
	bool bTreatInputsWithoutChildrenLikeDecorators = false;
	GenerateFormattedGroups(InputDisplayEntries, InlineFormat, bIgnoreLinebreaks, bTreatInputsWithoutChildrenLikeDecorators, bFormatMatched, FormattedColumns);

	// Generate the columns.
	TSharedRef<SGridPanel> InputGrid = SNew(SGridPanel);
	int32 GraphNodeColumnIndex = 0;
	int32 ChildInputsColumnIndex = 0;
	int32 TotalGraphNodeColumnCount = FormattedColumns.Num();
	bool bHasHeaderRow;

	if (bFormatMatched == false && DynamicInputFunctionCall != nullptr)
	{
		// For unformatted inputs add a header with the dynamic input icon and the function name.
		bHasHeaderRow = true;
		bool bHasInputs = FormattedColumns.Num() > 0;
		InputGrid->AddSlot(0, 0)
		.ColumnSpan(TotalGraphNodeColumnCount)
		.Padding(GetVerticalGraphNodeGridPadding(0, 1))
		[
			SNew(SBorder)
			.BorderImage(bHasInputs 
				? FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Top")
				: FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Single"))
			.BorderBackgroundColor(FStyleColors::DropdownOutline)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(bHasInputs ? FMargin(6, 4, 6, 0) : FMargin(6, 4))
			[
				ConstructDynamicInputLabel(OwningInput)
			]
		];
	}
	else
	{
		bHasHeaderRow = false;
	}

	auto AddGraphNodeColumn = [InputGrid, TotalGraphNodeColumnCount, bHasHeaderRow](int32 GraphNodeColumnIndex, bool bHasChildren, TSharedRef<SWidget> Widget)
	{
		InputGrid->AddSlot(GraphNodeColumnIndex, 1)
		.Padding(GetVerticalGraphNodeGridPadding(GraphNodeColumnIndex, TotalGraphNodeColumnCount))
		[
			SNew(SBorder)
			.BorderImage(GetVerticalGraphNodeBorderBrush(GraphNodeColumnIndex, TotalGraphNodeColumnCount, bHasHeaderRow))
			.BorderBackgroundColor(FStyleColors::DropdownOutline)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(GetVerticalGraphNodeBorderPadding(GraphNodeColumnIndex, TotalGraphNodeColumnCount))
			[
				Widget
			]
		];
	};

	// Figure out which dynamic input row is last so that it can spawn any additional non dynamic input rows.
	int32 LastDynamicInputColumnIndex = INDEX_NONE;
	for (int32 FormattedColumnIndex = 0; FormattedColumnIndex < FormattedColumns.Num(); FormattedColumnIndex++)
	{
		TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>* DynamicInputPair = FormattedColumns[FormattedColumnIndex].FindByPredicate([](const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& Pair)
			{ return Pair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input && Pair.Value.InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic; });
		if (DynamicInputPair != nullptr)
		{
			LastDynamicInputColumnIndex = FormattedColumnIndex;
		}
	}

	// Generate column widgets.
	TSharedRef<FWidthSynchronize> ChildInputWidthSynchronizer = MakeShared<FWidthSynchronize>();
	WidthSynchronizers.Add(ChildInputWidthSynchronizer);
	for (const TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>& FormattedColumn : FormattedColumns)
	{
		FGuid InputId = FGuid::NewGuid();
		TOptional<FInputDisplayEntries> ColumnDisplayEntries;
		TSharedPtr<SWidget> ColumnWidget;
		if(FormattedColumn.Num() == 1)
		{
			ColumnWidget = ConstructGraphInputWidgetForFormatTokenInputPair(FormattedColumn[0], OwningInput, InputId);
			if(FormattedColumn[0].Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
			{
				ColumnDisplayEntries = FormattedColumn[0].Value;
			}
		}
		else
		{
			TSharedRef<SHorizontalBox> ColumnBox = SNew(SHorizontalBox);
			ColumnWidget = ColumnBox;
			for (const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair : FormattedColumn)
			{
				ColumnBox->AddSlot()
				.AutoWidth()
				[
					ConstructGraphInputWidgetForFormatTokenInputPair(FormatTokenInputPair, OwningInput, InputId)
				];
				if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input)
				{
					ColumnDisplayEntries = FormatTokenInputPair.Value;
				}
			}
		}

		if (ColumnDisplayEntries.IsSet() && ColumnDisplayEntries->InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Data)
		{
			UNiagaraStackObject* ChildDataObject = ColumnDisplayEntries->InputEntry->GetChildDataObject();
			if (ChildDataObject != nullptr)
			{
				ConstructDataInterfaceWidgets(DataInterfaceVerticalBox, ColumnDisplayEntries.GetValue(), InputId);
			}
		}

		bool bIsDynamicInput = ColumnDisplayEntries.IsSet() && ColumnDisplayEntries->InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic;
		AddGraphNodeColumn(GraphNodeColumnIndex, bIsDynamicInput, ColumnWidget.ToSharedRef());

		// Add additional widgets for dynamic inputs.
		if (ColumnDisplayEntries.IsSet() && bIsDynamicInput)
		{
			// Child inputs.
			InputGrid->AddSlot(GraphNodeColumnIndex, 3)
			.VAlign(VAlign_Top)
			[
				ConstructVerticalGraphInputWidgets(ColumnDisplayEntries->ChildInputDisplayEntries, ColumnDisplayEntries->InputEntry, DataInterfaceVerticalBox)
			];

			// Connection arrow.
			FMargin BoxPadding = GetVerticalGraphNodeBorderPadding(GraphNodeColumnIndex, TotalGraphNodeColumnCount);
			BoxPadding.Top = 2;
			BoxPadding.Bottom = 2;
			InputGrid->AddSlot(GraphNodeColumnIndex, 2)
			.HAlign(HAlign_Center)
			.Padding(FMargin(0, -2))
			[
				SNew(SBox)
				.WidthOverride(16.0f)
				.HeightOverride(12.0f)
				[
					SNew(SImage)
					.Image(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Pointer"))
					.ColorAndOpacity(FStyleColors::DropdownOutline)
				]
			];

			ChildInputsColumnIndex++;
		}
		GraphNodeColumnIndex++;
	}

	return InputGrid;
}

TSharedRef<SWidget> SNiagaraStackInlineDynamicInput::ConstructHybridGraphInputWidgets(
	const TArray<FInputDisplayEntries>& InputDisplayEntries,
	const UNiagaraStackFunctionInput* OwningInput,
	TSharedRef<SVerticalBox> DataInterfaceVerticalBox)
{
	const UNiagaraNodeFunctionCall* DynamicInputFunctionCall =
		OwningInput != nullptr && OwningInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic
			? OwningInput->GetDynamicInputNode() : nullptr;
	bool bFormatMatched;
	TArray<TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>> FormattedColumns;
	TArray<FNiagaraInlineDynamicInputFormatToken> InlineFormat = DynamicInputFunctionCall != nullptr 
		? DynamicInputFunctionCall->GetScriptData()->InlineExpressionFormat 
		: TArray<FNiagaraInlineDynamicInputFormatToken>();
	
	bool bIgnoreLinebreaks = true;
	bool bTreatInputsWithoutChildrenLikeDecorators = true;
	GenerateFormattedGroups(InputDisplayEntries, InlineFormat, bIgnoreLinebreaks, bTreatInputsWithoutChildrenLikeDecorators, bFormatMatched, FormattedColumns);

	// Generate the columns
	TSharedRef<SGridPanel> InputGrid = SNew(SGridPanel);
	int32 GraphNodeColumnIndex = 0;
	bool bNeedsDynamicInputLabel = bFormatMatched == false && DynamicInputFunctionCall != nullptr ? 1 : 0;
	int32 TotalGraphNodeColumnCount = FormattedColumns.Num();

	if (bNeedsDynamicInputLabel && FormattedColumns.Num() == 0)
	{
		InputGrid->AddSlot(0, 0)
		[
			SNew(SBorder)
			.BorderImage(GetHybridGraphNodeBorderBrush(0, 1, 0, 1))
			.BorderBackgroundColor(FStyleColors::DropdownOutline)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(GetHybridGraphNodeBorderPadding(0, 1, 0, 1))
			[
				ConstructDynamicInputLabel(OwningInput)
			]
		];
	}

	// Generate column widgets.
	for (const TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>& FormattedColumn : FormattedColumns)
	{
		FGuid InputId = FGuid::NewGuid();
		TOptional<FInputDisplayEntries> ColumnDisplayEntries;
		TSharedRef<SGridPanel> ColumnGrid = SNew(SGridPanel);
		ColumnGrid->SetRowFill(0, 1);
		int32 ColumnGridColumnIndex = 0;
		int32 TotalColumnGridColumns = FormattedColumn.Num();
		if (bNeedsDynamicInputLabel && GraphNodeColumnIndex == 0)
		{
			TotalColumnGridColumns++;
			ColumnGrid->AddSlot(0, 0)
			[
				SNew(SBorder)
				.BorderImage(GetHybridGraphNodeBorderBrush(GraphNodeColumnIndex, TotalGraphNodeColumnCount, ColumnGridColumnIndex, TotalColumnGridColumns))
				.BorderBackgroundColor(FStyleColors::DropdownOutline)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(GetHybridGraphNodeBorderPadding(GraphNodeColumnIndex, TotalGraphNodeColumnCount, ColumnGridColumnIndex, TotalColumnGridColumns) + FMargin(0, 0, 5, 0))
				[
					ConstructDynamicInputLabel(OwningInput)
				]
			];
			ColumnGridColumnIndex++;
		}

		for (const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair : FormattedColumn)
		{
			ColumnGrid->AddSlot(ColumnGridColumnIndex, 0)
			.HAlign(GetHybridGraphNodeGridHorizontalAlignment(GraphNodeColumnIndex, TotalGraphNodeColumnCount, ColumnGridColumnIndex, TotalColumnGridColumns))
			[
				SNew(SBorder)
				.BorderImage(GetHybridGraphNodeBorderBrush(GraphNodeColumnIndex, TotalGraphNodeColumnCount, ColumnGridColumnIndex, TotalColumnGridColumns))
				.BorderBackgroundColor(FStyleColors::DropdownOutline)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.Padding(GetHybridGraphNodeBorderPadding(GraphNodeColumnIndex, TotalGraphNodeColumnCount, ColumnGridColumnIndex, TotalColumnGridColumns))
				[
					ConstructGraphInputWidgetForFormatTokenInputPair(FormatTokenInputPair, OwningInput, InputId)
				]
			];
			if (FormatTokenInputPair.Key.Usage == ENiagaraInlineDynamicInputFormatTokenUsage::Input && FormatTokenInputPair.Value.InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic)
			{
				// Connection arrow.
				ColumnGrid->AddSlot(ColumnGridColumnIndex, 1, SGridPanel::Layer(10))
				.Padding(FMargin(0.0f, -2.0f))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Bottom)
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					.HeightOverride(12.0f)
					[
						SNew(SImage)
						.Image(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Pointer"))
						.ColorAndOpacity(FStyleColors::DropdownOutline)
					]
				];
				ColumnDisplayEntries = FormatTokenInputPair.Value;
			}
			if (GraphNodeColumnIndex < TotalGraphNodeColumnCount - 1 && ColumnGridColumnIndex == TotalColumnGridColumns - 1)
			{
				ColumnGrid->SetColumnFill(ColumnGridColumnIndex, 1);
			}
			ColumnGridColumnIndex++;
		}

		if (ColumnDisplayEntries.IsSet() && ColumnDisplayEntries->InputEntry->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Data)
		{
			UNiagaraStackObject* ChildDataObject = ColumnDisplayEntries->InputEntry->GetChildDataObject();
			if (ChildDataObject != nullptr)
			{
				ConstructDataInterfaceWidgets(DataInterfaceVerticalBox, ColumnDisplayEntries.GetValue(), InputId);
			}
		}

		InputGrid->AddSlot(GraphNodeColumnIndex, 0)
		.VAlign(VAlign_Fill)
		[
			ColumnGrid
		];

		// Add additional widgets for dynamic inputs.
		if (ColumnDisplayEntries.IsSet())
		{
			InputGrid->AddSlot(GraphNodeColumnIndex, 1)
			.HAlign(HAlign_Left)
			[
				ConstructHybridGraphInputWidgets(ColumnDisplayEntries->ChildInputDisplayEntries, ColumnDisplayEntries->InputEntry, DataInterfaceVerticalBox)
			];
		}
		GraphNodeColumnIndex++;
	}

	return InputGrid;
}

void SNiagaraStackInlineDynamicInput::ConstructDataInterfaceWidgets(TSharedRef<SVerticalBox> DataInterfaceVerticalBox, const FInputDisplayEntries& InputDisplayEntries, FGuid InputId)
{
	UNiagaraStackFunctionInput* FunctionInput = InputDisplayEntries.InputEntry;
	UNiagaraStackObject* DataInterfaceObject = InputDisplayEntries.ChildObjectEntry;
	if (DataInterfaceObject->GetObject() == nullptr)
	{
		return;
	}

	TSharedRef<FNiagaraObjectSelection> SelectedObject = MakeShared<FNiagaraObjectSelection>();
	SelectedObject->SetSelectedObject(DataInterfaceObject->GetObject());
	DataInterfaceVerticalBox->AddSlot()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.Padding(0, 5, 0, 0)
		[
			SNew(SNiagaraStackInlineDynamicInputExpressionTokenBorder, FunctionInput, InputId)
			.HoveredInputId(this, &SNiagaraStackInlineDynamicInput::GetHoveredInputId)
			.OnHoveredChanged(this, &SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 5, 0)
				[
					SNew(SBox)
					.WidthOverride(TextIconSize)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FNiagaraStackEditorWidgetsUtilities::GetIconTextForInputMode(UNiagaraStackFunctionInput::EValueMode::Data))
						.ToolTipText(FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode::Data))
						.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForInputMode(UNiagaraStackFunctionInput::EValueMode::Data)))
					]
				]
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
					.Text(FunctionInput->GetInputType().GetClass()->GetDisplayNameText())
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0, 5, 0, 0)
		[
			SNew(SNiagaraSelectedObjectsDetails, SelectedObject)
		]
	];
}

FGuid SNiagaraStackInlineDynamicInput::GetHoveredInputId() const
{
	return HoveredInputId;
}

void SNiagaraStackInlineDynamicInput::OnFormatTokenOutlineHoveredChanged(FGuid InputId, bool bOutlineIsHovered)
{
	if (bOutlineIsHovered)
	{
		HoveredInputId = InputId;
	}
	else
	{
		if (HoveredInputId == InputId)
		{
			HoveredInputId = FGuid();
		}
	}
}

EVisibility SNiagaraStackInlineDynamicInput::GetEditConditionCheckBoxVisibility(UNiagaraStackFunctionInput* FunctionInput) const
{
	return FunctionInput->GetHasEditCondition() && FunctionInput->GetShowEditConditionInline() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SNiagaraStackInlineDynamicInput::GetEditConditionCheckState(UNiagaraStackFunctionInput* FunctionInput) const
{
	return FunctionInput->GetHasEditCondition() && FunctionInput->GetEditConditionEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackInlineDynamicInput::OnEditConditionCheckStateChanged(ECheckBoxState InCheckState, UNiagaraStackFunctionInput* FunctionInput)
{
	FunctionInput->SetEditConditionEnabled(InCheckState == ECheckBoxState::Checked);
}

EVisibility SNiagaraStackInlineDynamicInput::GetCompactActionMenuButtonVisibility(FGuid InputId) const
{
	return InputId == HoveredInputId ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE