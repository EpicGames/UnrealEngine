// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWrapBox;
class SGridPanel;
class UNiagaraNodeFunctionCall;

class SNiagaraStackInlineDynamicInput : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackInlineDynamicInput) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackFunctionInput* InFunctionInput);

private:
	struct FInputDisplayEntries
	{
		FInputDisplayEntries()
			: InputEntry(nullptr)
			, ChildObjectEntry(nullptr)
		{
		}

		FORCEINLINE bool operator ==(const FInputDisplayEntries& Other) const
		{
			return 
				InputEntry == Other.InputEntry &&
				ChildInputDisplayEntries == Other.ChildInputDisplayEntries &&
				ChildObjectEntry == Other.ChildObjectEntry;	
		}

		FORCEINLINE bool operator !=(const FInputDisplayEntries& Other) const { return !(*this == Other); }

		UNiagaraStackFunctionInput* InputEntry;
		TArray<FInputDisplayEntries> ChildInputDisplayEntries;
		UNiagaraStackObject* ChildObjectEntry;
	};

	class FWidthSynchronize
	{
	public:
		void AddWidget(TSharedRef<SWidget> InWidget);
		FOptionalSize GetMaxDesiredWidth() const;
	private:
		TArray<TWeakPtr<SWidget>> WeakWidgets;
	};

private:
	static FInputDisplayEntries CollectInputDisplayEntriesRecursive(UNiagaraStackFunctionInput* Input);

	void RootFunctionInputStructureChanged(ENiagaraStructureChangedFlags Flags);

	void RootFunctionInputValueChanged();

	static bool TryMatchFormatInputs(
		const TArray<FInputDisplayEntries>& ChildInputs,
		const TArray<FNiagaraInlineDynamicInputFormatToken>& InlineFormat,
		TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>& OutMatchedFormatInputs);

	void ConstructChildren();

	void ConstructWrappedExpressionInputWidgets(TSharedRef<SWrapBox> ExpressionWrapBox, TSharedRef<SVerticalBox> DataInterfaceVerticalBox, const FInputDisplayEntries& InputDisplayEntries, FGuid InputId, bool bIsRootInput);

	void ConstructWrappedExpressionDynamicInputWidgets(TSharedRef<SWrapBox> ExpressionWrapBox, TSharedRef<SVerticalBox> DataInterfaceVerticalBox, const FInputDisplayEntries& InputDisplayEntries, FGuid InputId, bool bIsRootInput);

	TSharedRef<SWidget> ConstructGraphInputWidgetForFormatTokenInputPair(
		const TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>& FormatTokenInputPair,
		const UNiagaraStackFunctionInput* OwningInput,
		FGuid InputId);

	void GenerateFormattedGroups(
		const TArray<FInputDisplayEntries>& InputDisplayEntries,
		const TArray<FNiagaraInlineDynamicInputFormatToken>& InlineFormat,
		bool bIgnoreLinebreaks,
		bool bTreatInputsWithoutChildrenLikeDecorators,
		bool& bOutFormatMatched,
		TArray<TArray<TPair<FNiagaraInlineDynamicInputFormatToken, FInputDisplayEntries>>>& OutFormattedGroups);

	TSharedRef<SWidget> ConstructHorizontalGraphInputWidgets(
		const TArray<FInputDisplayEntries>& InputDisplayEntries,
		const UNiagaraStackFunctionInput* OwningInput,
		TSharedRef<SVerticalBox> DataInterfaceVerticalBox,
		TSharedPtr<FWidthSynchronize> InputWidthSynchronizer);

	TSharedRef<SWidget> ConstructVerticalGraphInputWidgets(
		const TArray<FInputDisplayEntries>& InputDisplayEntries,
		const UNiagaraStackFunctionInput* OwningInput,
		TSharedRef<SVerticalBox> DataInterfaceVerticalBox);

	TSharedRef<SWidget> ConstructHybridGraphInputWidgets(
		const TArray<FInputDisplayEntries>& InputDisplayEntries,
		const UNiagaraStackFunctionInput* OwningInput,
		TSharedRef<SVerticalBox> DataInterfaceVerticalBox);

	void ConstructDataInterfaceWidgets(TSharedRef<SVerticalBox> DataInterfaceVerticalBox, const FInputDisplayEntries& InputDisplayEntries, FGuid InputId);

	FGuid GetHoveredInputId() const;

	void OnFormatTokenOutlineHoveredChanged(FGuid InputId, bool bIsHovered);

	EVisibility GetEditConditionCheckBoxVisibility(UNiagaraStackFunctionInput* FunctionInput) const;

	ECheckBoxState GetEditConditionCheckState(UNiagaraStackFunctionInput* FunctionInput) const;

	void OnEditConditionCheckStateChanged(ECheckBoxState InCheckState, UNiagaraStackFunctionInput* FunctionInput);

	EVisibility GetCompactActionMenuButtonVisibility(FGuid InputId) const;

private:
	UNiagaraStackFunctionInput* RootFunctionInput;
	FInputDisplayEntries RootInputDisplayEntries;
	TArray<TSharedPtr<FWidthSynchronize>> WidthSynchronizers;

	bool bSingleValueMode;
	FGuid HoveredInputId;

	static const float TextIconSize;
};