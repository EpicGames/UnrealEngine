// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStackIssueIcon.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "NiagaraEditorStyle.h"
#include "Internationalization/Text.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SNiagaraStackIssueIcon"

const float SNiagaraStackIssueIcon::NormalSize = 16.0f;
const float SNiagaraStackIssueIcon::CompactSize = 10.0f;

void SNiagaraStackIssueIcon::Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel, UNiagaraStackEntry* InStackEntry)
{
	StackViewModel = InStackViewModel;
	StackEntry = InStackEntry;
	IconMode = InArgs._IconMode;
	if (StackEntry != nullptr)
	{
		StackEntry->OnStructureChanged().AddSP(this, &SNiagaraStackIssueIcon::UpdateFromEntry);
	}

	TSharedPtr<SWidget> IconWidget;
	if (IconMode == EIconMode::Normal)
	{
		IconWidget =
			SNew(SBox)
			.IsEnabled(this, &SNiagaraStackIssueIcon::GetIconIsEnabled)
			.ToolTipText(this, &SNiagaraStackIssueIcon::GetIconToolTip)
			.HeightOverride(this, &SNiagaraStackIssueIcon::GetIconHeight)
			.WidthOverride(NormalSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SNiagaraStackIssueIcon::GetIconBrush)
			];

		if (InArgs._OnClicked.IsBound())
		{
			TSharedRef<SButton> IconButton =
				SNew(SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(InArgs._OnClicked)
				.Content()
				[
					IconWidget.ToSharedRef()
				];

			IconWidget = IconButton;
		}
	}
	else if (IconMode == EIconMode::Compact)
	{
		IconWidget = 
			SNew(SBorder)
			.BorderImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.ModuleHighlightLarge"))
			.BorderBackgroundColor(FStyleColors::Panel)
			.IsEnabled(this, &SNiagaraStackIssueIcon::GetIconIsEnabled)
			.ToolTipText(this, &SNiagaraStackIssueIcon::GetIconToolTip)
			[
				SNew(SBox)
				.HeightOverride(this, &SNiagaraStackIssueIcon::GetIconHeight)
				.WidthOverride(CompactSize)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SNiagaraStackIssueIcon::GetIconBrush)
				]
			];
	}

	ChildSlot
	[
		IconWidget.ToSharedRef()
	];
	UpdateFromEntry(ENiagaraStructureChangedFlags::StructureChanged);
}

SNiagaraStackIssueIcon::~SNiagaraStackIssueIcon()
{
	if (StackEntry.IsValid())
	{
		StackEntry->OnStructureChanged().RemoveAll(this);
	}
}

bool SNiagaraStackIssueIcon::GetIconIsEnabled() const
{
	return StackEntry.IsValid() && StackEntry->IsFinalized() == false && StackEntry->GetOwnerIsEnabled() && StackEntry->GetIsEnabled();
}

const FSlateBrush* SNiagaraStackIssueIcon::GetIconBrush() const
{
	return IconBrush;
}

FText SNiagaraStackIssueIcon::GetIconToolTip() const
{
	if (IconToolTipCache.IsSet() == false)
	{
		if (StackEntry.IsValid() == false || StackEntry->IsFinalized())
		{
			IconToolTipCache = FText();
		}
		else
		{
			FTextBuilder ToolTipBuilder;
			TArray<FText> ToolTipParts;
			if (StackEntry->GetTotalNumberOfErrorIssues() > 0)
			{
				if (StackEntry->GetTotalNumberOfErrorIssues() == 1)
				{
					ToolTipParts.Add(LOCTEXT("ErrorFormatSingle", "1 error"));
				}
				else
				{
					ToolTipParts.Add(FText::Format(LOCTEXT("ErrorFormatMultiple", "{0} errors"), FText::AsNumber(StackEntry->GetTotalNumberOfErrorIssues())));
				}
			}
			if (StackEntry->GetTotalNumberOfWarningIssues() > 0)
			{
				if (StackEntry->GetTotalNumberOfWarningIssues() == 1)
				{
					ToolTipParts.Add(LOCTEXT("WarningFormatSingle", "1 warning"));
				}
				else
				{
					ToolTipParts.Add(FText::Format(LOCTEXT("WarningFormatMultiple", "{0} warnings"), FText::AsNumber(StackEntry->GetTotalNumberOfWarningIssues())));
				}
			}
			if (StackEntry->GetTotalNumberOfInfoIssues() > 0)
			{
				if (StackEntry->GetTotalNumberOfInfoIssues() == 1)
				{
					ToolTipParts.Add(LOCTEXT("InfoFormatSingle", "1 info"));
				}
				else
				{
					ToolTipParts.Add(FText::Format(LOCTEXT("InfoFormatMultiple", "{0} infos"), FText::AsNumber(StackEntry->GetTotalNumberOfInfoIssues())));
				}
			}

			if (ToolTipParts.Num() == 4)
			{
				ToolTipBuilder.AppendLineFormat(LOCTEXT("FourPartFormat", "{0}, {1}, {2}, and {3}"), ToolTipParts[0], ToolTipParts[1], ToolTipParts[2], ToolTipParts[3]);				
			}
			else if (ToolTipParts.Num() == 3)
			{
				ToolTipBuilder.AppendLineFormat(LOCTEXT("ThreePartFormat", "{0}, {1}, and {2}"), ToolTipParts[0], ToolTipParts[1], ToolTipParts[2]);
			}
			else if (ToolTipParts.Num() == 2)
			{
				ToolTipBuilder.AppendLineFormat(LOCTEXT("TwoPartFormat", "{0} and {1}"), ToolTipParts[0], ToolTipParts[1]);
			}
			else if (ToolTipParts.Num() == 1)
			{
				ToolTipBuilder.AppendLine(ToolTipParts[0]);
			}

			FText IssueLineFormat = LOCTEXT("IssueLineFormat", "{0} - {1} - {2}");
			auto SeverityToText = [](EStackIssueSeverity Severity)
			{
				switch (Severity)
				{
				case EStackIssueSeverity::Error:
					return LOCTEXT("Error", "Error");
				case EStackIssueSeverity::Warning:
					return LOCTEXT("Warning", "Warning");
				case EStackIssueSeverity::Info:
					return LOCTEXT("Issue", "Issue");
				default:
					return FText();
				}
			};

			auto GetFullDisplayName = [](const UNiagaraStackViewModel* InStackViewModel, UNiagaraStackEntry* EntryWithIssue)
			{
				TArray<UNiagaraStackEntry*> EntryPath;
				InStackViewModel->GetPathForEntry(EntryWithIssue, EntryPath);
				EntryPath.Add(EntryWithIssue);
				TArray<FText> DisplayNameParts;
				for (UNiagaraStackEntry* Entry : EntryPath)
				{
					if (Entry->GetDisplayName().IsEmpty() == false)
					{
						DisplayNameParts.Add(Entry->GetDisplayName());
					}
				}
				return FText::Join(LOCTEXT("DisplayNameJoinDelimiter", " - "), DisplayNameParts);
			};

			int32 IssueLinesAppended = 0;
			int32 MaxIssueLines = 5;
			int32 TotalIssues = 0;
			TArray<UNiagaraStackEntry*> EntriesToCheck;
			EntriesToCheck.Add(StackEntry.Get());
			EntriesToCheck.Append(StackEntry->GetAllChildrenWithIssues());
			for (UNiagaraStackEntry* EntryToCheck : EntriesToCheck)
			{
				for (int32 IssueIndex = 0; IssueIndex < EntryToCheck->GetIssues().Num() && IssueLinesAppended < MaxIssueLines; IssueIndex++, IssueLinesAppended++)
				{
					const UNiagaraStackEntry::FStackIssue& Issue = EntryToCheck->GetIssues()[IssueIndex];
					ToolTipBuilder.AppendLineFormat(IssueLineFormat, GetFullDisplayName(StackViewModel, EntryToCheck), SeverityToText(Issue.GetSeverity()), Issue.GetShortDescription());
				}
				TotalIssues += EntryToCheck->GetIssues().Num();
			}

			if (TotalIssues > MaxIssueLines)
			{
				ToolTipBuilder.AppendLineFormat(LOCTEXT("MoreIssuesFormat", "(And {0} more {0}|plural(one=issue,other=issues)...)"), FText::AsNumber(TotalIssues - MaxIssueLines));
				//"There {NumCats}|plural(one=is,other=are) {NumCats} {NumCats}|plural(one=cat,other=cats)"
			}

			IconToolTipCache = ToolTipBuilder.ToText();
		}
	}
	return IconToolTipCache.GetValue();
}

FOptionalSize SNiagaraStackIssueIcon::GetIconHeight() const
{
	return IconHeight;
}

void SNiagaraStackIssueIcon::UpdateFromEntry(ENiagaraStructureChangedFlags Flags)
{
	if (StackEntry.IsValid() == false || StackEntry->IsFinalized() || StackEntry->HasIssuesOrAnyChildHasIssues() == false)
	{
		IconBrush = FAppStyle::GetBrush("NoBrush");
		IconHeight = 0;
		IconToolTipCache = FText();
		return;
	}

	IconHeight = IconMode == EIconMode::Compact ? CompactSize : NormalSize;
	if (StackEntry->GetTotalNumberOfErrorIssues() > 0)
	{
		IconBrush = IconMode == EIconMode::Compact
			? FNiagaraEditorWidgetsStyle::Get().GetBrush("Niagara.CompactStackIssue.Error")
			: FAppStyle::Get().GetBrush("Icons.ErrorWithColor");
	}
	else if (StackEntry->GetTotalNumberOfWarningIssues() > 0)
	{
		IconBrush = IconMode == EIconMode::Compact
			? FNiagaraEditorWidgetsStyle::Get().GetBrush("Niagara.CompactStackIssue.Warning")
			: FAppStyle::Get().GetBrush("Icons.WarningWithColor");
	}
	else if (StackEntry->GetTotalNumberOfInfoIssues() > 0)
	{
		IconBrush = IconMode == EIconMode::Compact
			? FNiagaraEditorWidgetsStyle::Get().GetBrush("Niagara.CompactStackIssue.Info")
			: FAppStyle::Get().GetBrush("Icons.InfoWithColor");
	}

	IconToolTipCache.Reset();
}

#undef LOCTEXT_NAMESPACE
