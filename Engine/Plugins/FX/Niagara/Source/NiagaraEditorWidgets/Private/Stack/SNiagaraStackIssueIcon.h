// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorCommon.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/SlateDelegates.h"
#include "Types/SlateStructs.h"

class UNiagaraStackViewModel;
class UNiagaraStackEntry;

class SNiagaraStackIssueIcon : public SCompoundWidget
{
public:
	enum class EIconMode
	{
		Normal,
		Compact
	};

public:
	SLATE_BEGIN_ARGS(SNiagaraStackIssueIcon) 
		: _IconMode(EIconMode::Normal)
		{}
		SLATE_ARGUMENT(EIconMode, IconMode)
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel, UNiagaraStackEntry* InStackEntry);

	~SNiagaraStackIssueIcon();
	
private:
	bool GetIconIsEnabled() const;

	const FSlateBrush* GetIconBrush() const;
	FText GetIconToolTip() const;
	FOptionalSize GetIconHeight() const;
	EVisibility GetCompactIconBorderVisibility() const;

	void UpdateFromEntry(ENiagaraStructureChangedFlags Flags);

private:
	const FSlateBrush* IconBrush;
	float IconHeight;

	mutable TOptional<FText> IconToolTipCache;

	UNiagaraStackViewModel* StackViewModel;
	TWeakObjectPtr<UNiagaraStackEntry> StackEntry;
	EIconMode IconMode;

	static const float NormalSize;
	static const float CompactSize;
};