// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackItem.h"

#include "EditorFontGlyphs.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "SEnumCombo.h"
#include "Stack/SNiagaraStackInheritanceIcon.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "ViewModels/Stack/NiagaraStackClipboardUtilities.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "NiagaraStackItem"

class SNiagaraStackItemHeaderValue : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackItemHeaderValue) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraStackItemHeaderValueHandler> InHeaderValueHandler)
	{
		HeaderValueHandler = InHeaderValueHandler;

		TSharedPtr<SWidget> ChildWidget;
		if (HeaderValueHandler->GetMode() == INiagaraStackItemHeaderValueHandler::EValueMode::BoolToggle)
		{
			ChildWidget = ConstructBoolToggle();
		}
		else if (HeaderValueHandler->GetMode() == INiagaraStackItemHeaderValueHandler::EValueMode::EnumDropDown)
		{
			ChildWidget = ConstructEnumDropDown();
		}
		else
		{
			ChildWidget = SNullWidget::NullWidget;
		}

		ChildSlot
		[
			ChildWidget.ToSharedRef()
		];
	}

private:
	TSharedRef<SWidget> ConstructEnumDropDown()
	{
		TSharedRef<SWidget> EnumWidget = SNew(SEnumComboBox, HeaderValueHandler->GetEnum())
			.ComboButtonStyle(&FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FComboButtonStyle>("NiagaraEditor.Stack.CompactComboButton"))
			.CurrentValue(HeaderValueHandler.ToSharedRef(), &INiagaraStackItemHeaderValueHandler::GetEnumValue)
			.ContentPadding(FMargin(2, 0))
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemHeaderValueText")
			.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &SNiagaraStackItemHeaderValue::EnumValueChanged));

		TSharedPtr<SWidget> IconWidget;
		if (HeaderValueHandler->GetIconBrush() != nullptr)
		{
			IconWidget = SNew(SImage)
				.Image(HeaderValueHandler->GetIconBrush())
				.DesiredSizeOverride(FVector2D(14.0f, 14.0f));
		}

		TSharedPtr<SWidget> LabelWidget;
		if (HeaderValueHandler->GetLabelText().IsEmpty() == false)
		{
			LabelWidget = SNew(STextBlock)
				.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemHeaderValueLabelText")
				.Text(HeaderValueHandler->GetLabelText());
		}

		if (IconWidget.IsValid() || LabelWidget.IsValid())
		{
			TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);
			if (IconWidget.IsValid())
			{
				ContentBox->AddSlot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0, 0, 5, 0)
					[
						IconWidget.ToSharedRef()
					];
			}

			if (LabelWidget.IsValid())
			{
				ContentBox->AddSlot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0, 0, 5, 0)
					[
						LabelWidget.ToSharedRef()
					];
			}

			ContentBox->AddSlot()
				.AutoWidth()
				[
					EnumWidget
				];

			return SNew(SBorder)
				.BorderImage(FNiagaraEditorWidgetsStyle::Get().GetBrush(TEXT("NiagaraEditor.Stack.ItemHeaderValue.BackgroundBrush")))
				.Padding(7, 2, 2, 2)
				[
					ContentBox
				];
		}
		else
		{
			return EnumWidget;
		}
	}

	TSharedRef<SWidget> ConstructBoolToggle()
	{
		if (HeaderValueHandler->GetIconBrush() != nullptr)
		{
			return SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.ToolTipText(HeaderValueHandler->GetLabelText())
				.OnCheckStateChanged(this, &SNiagaraStackItemHeaderValue::BoolCheckStateChanged)
				.IsChecked(this, &SNiagaraStackItemHeaderValue::GetBoolCheckState)
				.Padding(FMargin(4, 2))
				[
					SNew(SImage)
					.Image(HeaderValueHandler->GetIconBrush())
					.DesiredSizeOverride(FVector2D(14.0f, 14.0f))
				];
		}

		if (HeaderValueHandler->GetLabelText().IsEmpty() == false)
		{
			return SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.OnCheckStateChanged(this, &SNiagaraStackItemHeaderValue::BoolCheckStateChanged)
				.IsChecked(this, &SNiagaraStackItemHeaderValue::GetBoolCheckState)
				[
					SNew(STextBlock)
					.Text(HeaderValueHandler->GetLabelText())
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				];
		}

		return SNullWidget::NullWidget;
	}

	void EnumValueChanged(int32 Value, ESelectInfo::Type SelectionType)
	{
		HeaderValueHandler->NotifyEnumValueChanged(Value);
	}

	ECheckBoxState GetBoolCheckState() const
	{
		return HeaderValueHandler->GetBoolValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void BoolCheckStateChanged(ECheckBoxState InCheckState)
	{
		HeaderValueHandler->NotifyBoolValueChanged(InCheckState == ECheckBoxState::Checked);
	}

private:
	TSharedPtr<INiagaraStackItemHeaderValueHandler> HeaderValueHandler;
};

void SNiagaraStackItem::Construct(const FArguments& InArgs, UNiagaraStackItem& InItem, UNiagaraStackViewModel* InStackViewModel)
{
	Item = &InItem;
	StackEntryItem = Item;
	StackViewModel = InStackViewModel;

	TSharedRef<SHorizontalBox> RowBox = SNew(SHorizontalBox);

	// Icon Brush
	if (Item->GetSupportedIconMode() == UNiagaraStackEntry::EIconMode::Brush)
	{
		RowBox->AddSlot()
		.Padding(2, 0, 3, 0)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image_UObject(Item, &UNiagaraStackItem::GetIconBrush)
		];
	}

	// Icon Text
	if (Item->GetSupportedIconMode() == UNiagaraStackEntry::EIconMode::Text)
	{
		RowBox->AddSlot()
		.Padding(2, 0, 3, 0)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
			.Text(Item->GetIconText())
		];
	}

	// Display name
	RowBox->AddSlot()
		.Padding(2, 0, 10, 0)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew(DisplayNameWidget, SNiagaraStackDisplayName, InItem, *InStackViewModel)
			.NameStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemText")
			.EditableNameStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.EditableItemText")
		];

	// Entry based header values.
	ConstructHeaderValues(RowBox);

	// Allow derived classes to add additional widgets.
	AddCustomRowWidgets(RowBox);

	// Edit Mode Button
	if (Item->SupportsEditMode())
	{
		TSharedPtr<SButton> EditButton;
		
		RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 2, 0)
		[
			SAssignNew(EditButton, SButton)
			.IsFocusable(false)
			.ToolTipText(this, &SNiagaraStackItem::GetEditModeButtonToolTipText)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			.OnClicked(this, &SNiagaraStackItem::EditModeButtonClicked)
			.ContentPadding(1)
			.Visibility(this, &SNiagaraStackItem::IsEditButtonVisible)
		];

		if(Item->GetEditModeButtonText().IsSet())
		{
			EditButton->SetContent
			(
				SNew(STextBlock)
				.Text(Item->GetEditModeButtonText().GetValue())			
			);
		}
		else
		{
			EditButton->SetContent
			(
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Edit"))
			);
		}
	}

	// Reset to base button
	if (Item->SupportsResetToBase())
	{
		RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 2, 0)
		[
			SNew(SButton)
			.Visibility(this, &SNiagaraStackItem::GetResetToBaseButtonVisibility)
			.IsFocusable(false)
			.ToolTipText(this, &SNiagaraStackItem::GetResetToBaseButtonToolTipText)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ContentPadding(0)
			.OnClicked(this, &SNiagaraStackItem::ResetToBaseButtonClicked)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
				.ColorAndOpacity(FSlateColor(FLinearColor::Green))
			]
		];
	}

	// Inheritance Icon
	if (Item->SupportsInheritance())
	{
		RowBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0, 0, 2, 0)
		[
			SNew(SNiagaraStackInheritanceIcon, Item)
		];
	}

	// Delete button
	if (Item->SupportsDelete())
	{
		RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 0, 2, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.ToolTipText(this, &SNiagaraStackItem::GetDeleteButtonToolTipText)
			.Visibility(this, &SNiagaraStackItem::GetDeleteButtonVisibility)
			.OnClicked(this, &SNiagaraStackItem::DeleteClicked)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	// Enabled checkbox
	if (Item->SupportsChangeEnabled())
	{
		RowBox->AddSlot()
		.Padding(0)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked(this, &SNiagaraStackItem::CheckEnabledStatus)
			.OnCheckStateChanged(this, &SNiagaraStackItem::OnCheckStateChanged)
			.IsEnabled(this, &SNiagaraStackItem::GetEnabledCheckBoxEnabled)
		];
	}

	ChildSlot
	[
		// Allow derived classes add a container for the row widgets, e.g. a drop target.
		AddContainerForRowWidgets(RowBox)
	];
}

void SNiagaraStackItem::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (Item->GetIsRenamePending())
	{
		if (DisplayNameWidget.IsValid())
		{
			DisplayNameWidget->StartRename();
		}
		Item->SetIsRenamePending(false);
	}
	SNiagaraStackEntryWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedRef<SWidget> SNiagaraStackItem::AddContainerForRowWidgets(TSharedRef<SWidget> RowWidgets)
{
	return RowWidgets;
}

void SNiagaraStackItem::ConstructHeaderValues(TSharedRef<SHorizontalBox> HorizontalBox)
{
	TArray<TSharedRef<INiagaraStackItemHeaderValueHandler>> HeaderValueHandlers;

	if (Item->SupportsHeaderValues())
	{
		Item->GetHeaderValueHandlers(HeaderValueHandlers);
	}

	TArray<TSharedRef<SWidget>> AlignLeft;
	TArray<TSharedRef<SWidget>> AlignCenterAlignFill;
	TArray<TSharedRef<SWidget>> AlignRight;

	for (TSharedRef<INiagaraStackItemHeaderValueHandler> HeaderValueHandler : HeaderValueHandlers)
	{
		TSharedRef<SWidget> HeaderValueWidget = SNew(SNiagaraStackItemHeaderValue, HeaderValueHandler)
			.IsEnabled(this, &SNiagaraStackItem::GetEntryIsEnabled);
		if (HeaderValueHandler->GetHAlign() == HAlign_Left)
		{
			AlignLeft.Add(HeaderValueWidget);
		}
		else if (HeaderValueHandler->GetHAlign() == HAlign_Right)
		{
			AlignRight.Add(HeaderValueWidget);
		}
		else
		{
			AlignCenterAlignFill.Add(HeaderValueWidget);
		}
	}

	if (AlignCenterAlignFill.Num() == 0)
	{
		AlignCenterAlignFill.Add(SNew(SSpacer));
	}

	auto AddWidgets = [](TSharedRef<SHorizontalBox> HorizontalBox, TArray<TSharedRef<SWidget>> Widgets, EHorizontalAlignment HorizontalAlignment, bool bAutoWidth)
	{
		for (TSharedRef<SWidget> Widget : Widgets)
		{
			SHorizontalBox::FScopedWidgetSlotArguments Slot = HorizontalBox->AddSlot();

			Slot
				.HAlign(HorizontalAlignment)
				.VAlign(VAlign_Center)
				.Padding(5, 0)
				[
					Widget
				];
			
			if (bAutoWidth)
			{
				Slot.AutoWidth();
			}
		}
	};

	AddWidgets(HorizontalBox, AlignLeft, HAlign_Left, true);
	AddWidgets(HorizontalBox, AlignCenterAlignFill, HAlign_Fill, false);
	AddWidgets(HorizontalBox, AlignRight, HAlign_Right, true);
}

FText SNiagaraStackItem::GetEditModeButtonToolTipText() const
{
	return Item->GetEditModeButtonTooltip().IsSet() ? Item->GetEditModeButtonTooltip().GetValue() : LOCTEXT("EditButtonDefaultTooltip", "Activate Edit Mode");
}

FReply SNiagaraStackItem::EditModeButtonClicked()
{
	if (Item->SupportsEditMode())
	{
		Item->OnEditButtonClicked();
	}
	return FReply::Handled();
}

EVisibility SNiagaraStackItem::IsEditButtonVisible() const
{
	if(Item->SupportsEditMode())
	{
		return Item->IsEditButtonVisible();
	}

	return EVisibility::Collapsed;
}

EVisibility SNiagaraStackItem::GetResetToBaseButtonVisibility() const
{
	FText Unused;
	return Item->TestCanResetToBaseWithMessage(Unused) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SNiagaraStackItem::GetResetToBaseButtonToolTipText() const
{
	FText CanResetToBaseMessage;
	Item->TestCanResetToBaseWithMessage(CanResetToBaseMessage);
	return CanResetToBaseMessage;
}

FReply SNiagaraStackItem::ResetToBaseButtonClicked()
{
	Item->ResetToBase();
	return FReply::Handled();
}

FText SNiagaraStackItem::GetDeleteButtonToolTipText() const
{
	FText CanDeleteMessage;
	Item->TestCanDeleteWithMessage(CanDeleteMessage);
	return CanDeleteMessage;
}

EVisibility SNiagaraStackItem::GetDeleteButtonVisibility() const
{
	FText Unused;
	return Item->TestCanDeleteWithMessage(Unused) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SNiagaraStackItem::DeleteClicked()
{
	TArray<UNiagaraStackEntry*> EntriesToDelete;
	EntriesToDelete.Add(Item);
	FNiagaraStackClipboardUtilities::DeleteSelection(EntriesToDelete);
	return FReply::Handled();
}

void SNiagaraStackItem::OnCheckStateChanged(ECheckBoxState InCheckState)
{
	Item->SetIsEnabled(InCheckState == ECheckBoxState::Checked);
}

ECheckBoxState SNiagaraStackItem::CheckEnabledStatus() const
{
	return Item->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool SNiagaraStackItem::GetEnabledCheckBoxEnabled() const
{
	return Item->IsFinalized() == false && Item->GetOwnerIsEnabled();
}

bool SNiagaraStackItem::GetEntryIsEnabled() const
{
	return Item->IsFinalized() == false && Item->GetIsEnabledAndOwnerIsEnabled();
}

#undef LOCTEXT_NAMESPACE