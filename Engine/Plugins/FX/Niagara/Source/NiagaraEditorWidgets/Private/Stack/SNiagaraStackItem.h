// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraStackEntryWidget.h"
#include "Styling/SlateTypes.h"
#include "Layout/Visibility.h"

class UNiagaraStackItem;
class UNiagaraStackViewModel; 

class SNiagaraStackItem: public SNiagaraStackEntryWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackItem) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackItem& InItem, UNiagaraStackViewModel* InStackViewModel);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	virtual void AddCustomRowWidgets(TSharedRef<SHorizontalBox> HorizontalBox) { }

	virtual TSharedRef<SWidget> AddContainerForRowWidgets(TSharedRef<SWidget> RowWidgets);

private:
	void ConstructHeaderValues(TSharedRef<SHorizontalBox> HorizontalBox);

	FReply EditModeButtonClicked();
	EVisibility IsEditButtonVisible() const;
	FText GetEditModeButtonToolTipText() const;

	EVisibility GetResetToBaseButtonVisibility() const;

	FText GetResetToBaseButtonToolTipText() const;

	FReply ResetToBaseButtonClicked();

	FText GetDeleteButtonToolTipText() const;

	EVisibility GetDeleteButtonVisibility() const;

	FReply DeleteClicked();

	void OnCheckStateChanged(ECheckBoxState InCheckState);

	ECheckBoxState CheckEnabledStatus() const;

	bool GetEnabledCheckBoxEnabled() const;

	bool GetEntryIsEnabled() const;

private:
	UNiagaraStackItem* Item;
	TSharedPtr<SNiagaraStackDisplayName> DisplayNameWidget;
};