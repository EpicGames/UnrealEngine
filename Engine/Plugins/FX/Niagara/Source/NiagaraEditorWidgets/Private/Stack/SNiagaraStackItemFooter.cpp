// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackItemFooter.h"
#include "ViewModels/Stack/NiagaraStackItemFooter.h"
#include "Styling/AppStyle.h"
#include "NiagaraEditorStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "NiagaraStackItemExpander"

void SNiagaraStackItemFooter::Construct(const FArguments& InArgs, UNiagaraStackItemFooter& InItemFooter)
{
	ItemFooter = &InItemFooter;
	ExpandedToolTipText = LOCTEXT("HideAdvancedToolTip", "Hide Advanced");
	CollapsedToolTipText = LOCTEXT("ShowAdvancedToolTip", "Show Advanced");

	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.Visibility(this, &SNiagaraStackItemFooter::GetExpandButtonVisibility)
		.HAlign(HAlign_Center)
		.ContentPadding(2)
		.ToolTipText(this, &SNiagaraStackItemFooter::GetToolTipText)
		.OnClicked(this, &SNiagaraStackItemFooter::ExpandButtonClicked)
		.IsFocusable(false)
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				// add the dropdown button for advanced properties 
				SNew(SImage)
				.Image(this, &SNiagaraStackItemFooter::GetButtonBrush)
			]
			+SHorizontalBox::Slot()
			[
				// add a little star next to the button if any advanced properties were changed
				SNew(STextBlock)
				.LineHeightPercentage(0.5f)
				.RenderTransform(FSlateRenderTransform(FVector2D(3, -8)))
				.Text(FText::FromString("*"))
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.HeadingTextBlock")
				.ColorAndOpacity(FSlateColor(FLinearColor(0.855f, 0.855f, 0.855f)))
	            .Visibility(this, &SNiagaraStackItemFooter::GetOverrideIconVisibility)
			]
		]
	];
}

EVisibility SNiagaraStackItemFooter::GetExpandButtonVisibility() const
{
	return ItemFooter->GetHasAdvancedContent() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SNiagaraStackItemFooter::GetOverrideIconVisibility() const
{
	return ItemFooter->GetHasAdvancedContent() && ItemFooter->HasChangedAdvancedContent() ? EVisibility::HitTestInvisible : EVisibility::Hidden;
}

const FSlateBrush* SNiagaraStackItemFooter::GetButtonBrush() const
{
	if (IsHovered())
	{
		return ItemFooter->GetShowAdvanced()
			? FAppStyle::GetBrush("DetailsView.PulldownArrow.Up.Hovered")
			: FAppStyle::GetBrush("DetailsView.PulldownArrow.Down.Hovered");
	}
	else
	{
		return ItemFooter->GetShowAdvanced()
			? FAppStyle::GetBrush("DetailsView.PulldownArrow.Up")
			: FAppStyle::GetBrush("DetailsView.PulldownArrow.Down");
	}
}

FText SNiagaraStackItemFooter::GetToolTipText() const
{
	// The engine ticks tooltips before widgets so it's possible for the footer to be finalized when
	// the widgets haven't been recreated.
	if (ItemFooter->IsFinalized())
	{
		return FText();
	}
	return ItemFooter->GetShowAdvanced() ? ExpandedToolTipText : CollapsedToolTipText;
}

FReply SNiagaraStackItemFooter::ExpandButtonClicked()
{
	ItemFooter->ToggleShowAdvanced();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE