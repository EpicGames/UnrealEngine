// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UNiagaraStackItemFooter;

class SNiagaraStackItemFooter: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackItemFooter) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackItemFooter& InItemFooter);

private:
	EVisibility GetExpandButtonVisibility() const;

	EVisibility GetOverrideIconVisibility() const;

	const struct FSlateBrush* GetButtonBrush() const;

	FText GetToolTipText() const;

	FReply ExpandButtonClicked();

private:
	UNiagaraStackItemFooter* ItemFooter;
	FText ExpandedToolTipText;
	FText CollapsedToolTipText;
};