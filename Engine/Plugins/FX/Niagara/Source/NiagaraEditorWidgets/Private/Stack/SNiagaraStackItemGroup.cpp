// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackItemGroup.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorStyle.h"
#include "Styling/AppStyle.h"
#include "Stack/SNiagaraStackInheritanceIcon.h"
#include "Stack/SNiagaraStackItemGroupAddButton.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "NiagaraStackItemGroup"


void SNiagaraStackItemGroup::Construct(const FArguments& InArgs, UNiagaraStackItemGroup& InGroup, UNiagaraStackViewModel* InStackViewModel)
{
	Group = &InGroup;
	StackEntryItem = Group;
	StackViewModel = InStackViewModel;

	TSharedRef<SHorizontalBox> RowBox = SNew(SHorizontalBox);

	// Name
	RowBox->AddSlot()
	.VAlign(VAlign_Center)
	.Padding(2, 0, 0, 0)
	[
		SNew(SNiagaraStackDisplayName, InGroup, *InStackViewModel)
		.NameStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.GroupText")
		.EditableNameStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.EditableGroupText")
	];

	// Inheritance Icon
	if (Group->SupportsInheritance())
	{
		RowBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0, 0, 2, 0)
		[
			SNew(SNiagaraStackInheritanceIcon, Group)
		];
	}

	// Delete group button
	if (Group->SupportsDelete())
	{
		RowBox->AddSlot()
		.AutoWidth()
		.Padding(0, 0, 2, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.ToolTipText(this, &SNiagaraStackItemGroup::GetDeleteButtonToolTip)
			.OnClicked(this, &SNiagaraStackItemGroup::DeleteClicked)
			.Visibility(this, &SNiagaraStackItemGroup::GetDeleteButtonVisibility)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	// Enabled button
	if (Group->SupportsChangeEnabled())
	{
		RowBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0, 0, 0, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &SNiagaraStackItemGroup::CheckEnabledStatus)
			.OnCheckStateChanged(this, &SNiagaraStackItemGroup::OnCheckStateChanged)
			.IsEnabled(this, &SNiagaraStackItemGroup::GetEnabledCheckBoxEnabled)
		];
	}

	// Add button
	RowBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.Padding(2, 0, 0, 0)
	[
		ConstructAddButton()
	];

	ChildSlot
	[
		RowBox
	];
}

TSharedRef<SWidget> SNiagaraStackItemGroup::ConstructAddButton()
{
	INiagaraStackItemGroupAddUtilities* AddUtilities = Group->GetAddUtilities();
	if (AddUtilities != nullptr)
	{
		return SNew(SNiagaraStackItemGroupAddButton, Group, AddUtilities);
	}
	return SNullWidget::NullWidget;
}

FText SNiagaraStackItemGroup::GetDeleteButtonToolTip() const
{
	FText Message;
	Group->TestCanDeleteWithMessage(Message);
	return Message;
}

EVisibility SNiagaraStackItemGroup::GetDeleteButtonVisibility() const
{
	FText UnusedMessage;
	return Group->SupportsDelete() && Group->TestCanDeleteWithMessage(UnusedMessage)
		? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SNiagaraStackItemGroup::DeleteClicked()
{
	Group->Delete();
	return FReply::Handled();
}

void SNiagaraStackItemGroup::OnCheckStateChanged(ECheckBoxState InCheckState)
{
	Group->SetIsEnabled(InCheckState == ECheckBoxState::Checked);
}

ECheckBoxState SNiagaraStackItemGroup::CheckEnabledStatus() const
{
	return Group->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool SNiagaraStackItemGroup::GetEnabledCheckBoxEnabled() const
{
	return Group->GetOwnerIsEnabled();
}

#undef LOCTEXT_NAMESPACE
