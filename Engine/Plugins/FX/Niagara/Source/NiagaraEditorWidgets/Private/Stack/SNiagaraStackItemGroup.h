// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraStackEntryWidget.h"
#include "Layout/Visibility.h"

class UNiagaraStackItemGroup;
class UNiagaraStackViewModel;

class SNiagaraStackItemGroup : public SNiagaraStackEntryWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackItemGroup) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackItemGroup& InGroup, UNiagaraStackViewModel* InStackViewModel);

private:
	TSharedRef<SWidget> ConstructAddButton();

	FText GetDeleteButtonToolTip() const;

	EVisibility GetDeleteButtonVisibility() const;

	FReply DeleteClicked();

	void OnCheckStateChanged(ECheckBoxState InCheckState);
	ECheckBoxState CheckEnabledStatus() const;
	bool GetEnabledCheckBoxEnabled() const;

private:
	UNiagaraStackItemGroup* Group;
};
