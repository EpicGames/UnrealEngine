// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackItemGroupAddButton.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "Stack/SNiagaraStackItemGroupAddMenu.h"
#include "ViewModels/Stack/INiagaraStackItemGroupAddUtilities.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SNiagaraStackItemGroupAddButton"

const float SNiagaraStackItemGroupAddButton::TextIconSize = 16;

void SNiagaraStackItemGroupAddButton::Construct(const FArguments& InArgs, UNiagaraStackEntry* InSourceEntry, INiagaraStackItemGroupAddUtilities* InAddUtilities)
{
	SourceEntryWeak = InSourceEntry;
	AddUtilities = InAddUtilities;
	TSharedPtr<SWidget> Content;
	if(InSourceEntry != nullptr && InAddUtilities != nullptr)
	{
		TSharedPtr<SWidget> ButtonContent;
		const FButtonStyle* ButtonStyle = nullptr;
		if (AddUtilities->GetShowLabel())
		{
			ButtonContent = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(1.0f))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Plus)
					.ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor(
						FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(InSourceEntry->GetExecutionCategoryName())))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0f))
				[
					SNew(STextBlock)
					.Text(AddUtilities->GetAddItemName())
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				];
			ButtonStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>("NiagaraEditor.Stack.LabeledAddItemButton");
		}
		else
		{
			ButtonContent = SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground());
			ButtonStyle = &FNiagaraEditorWidgetsStyle::Get().GetWidgetStyle<FButtonStyle>(
				FNiagaraStackEditorWidgetsUtilities::GetAddItemButtonStyleNameForExecutionCategory(InSourceEntry->GetExecutionCategoryName()));
		}

		FText AddToGroupFormat = LOCTEXT("AddToGroupFormat", "Add a new {0} to this group.");
		if (AddUtilities->GetAddMode() == INiagaraStackItemGroupAddUtilities::AddFromAction)
		{
			Content = SAssignNew(AddActionButton, SComboButton)
				.ButtonStyle(ButtonStyle)
				.ToolTipText(FText::Format(AddToGroupFormat, AddUtilities->GetAddItemName()))
				.HasDownArrow(false)
				.OnGetMenuContent(this, &SNiagaraStackItemGroupAddButton::GetAddMenu)
				.IsEnabled_UObject(InSourceEntry, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				.ContentPadding(1.0f)
				.MenuPlacement(MenuPlacement_BelowRightAnchor)
				.ButtonContent()
				[
					ButtonContent.ToSharedRef()
				];
		}
		else if (AddUtilities->GetAddMode() == INiagaraStackItemGroupAddUtilities::AddDirectly)
		{
			Content = SNew(SButton)
				.ButtonStyle(ButtonStyle)
				.ToolTipText(FText::Format(AddToGroupFormat, AddUtilities->GetAddItemName()))
				.IsEnabled_UObject(InSourceEntry, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
				.ContentPadding(1.0f)
				.OnClicked(this, &SNiagaraStackItemGroupAddButton::AddDirectlyButtonClicked)
				.Content()
				[
					ButtonContent.ToSharedRef()
				];
		}
	}
	else
	{
		// TODO Log error here.
		Content = SNullWidget::NullWidget;
	}

	ChildSlot
	[
		Content.ToSharedRef()
	];
}

TSharedRef<SWidget> SNiagaraStackItemGroupAddButton::GetAddMenu()
{
	if(SourceEntryWeak.IsValid())
	{
		TSharedRef<SNiagaraStackItemGroupAddMenu> AddMenu = SNew(SNiagaraStackItemGroupAddMenu, SourceEntryWeak.Get(), AddUtilities, INDEX_NONE);
		AddActionButton->SetMenuContentWidgetToFocus(AddMenu->GetFilterTextBox()->AsShared());
		return AddMenu;
	}
	return SNullWidget::NullWidget;
}

FReply SNiagaraStackItemGroupAddButton::AddDirectlyButtonClicked()
{
	if(SourceEntryWeak.IsValid())
	{
		AddUtilities->AddItemDirectly();
		SourceEntryWeak->SetIsExpandedInOverview(true);
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE