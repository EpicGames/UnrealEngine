// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UNiagaraStackEntry;
class INiagaraStackItemGroupAddUtilities;
class SComboButton;

class SNiagaraStackItemGroupAddButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackItemGroupAddButton)
		:_Width(TextIconSize * 2)
		{}
		SLATE_ARGUMENT(float, Width)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackEntry* InSourceEntry, INiagaraStackItemGroupAddUtilities* InAddUtilities);

private:
	TSharedRef<SWidget> GetAddMenu();

	FReply AddDirectlyButtonClicked();

private:
	TWeakObjectPtr<UNiagaraStackEntry> SourceEntryWeak;
	INiagaraStackItemGroupAddUtilities* AddUtilities;
	TSharedPtr<SComboButton> AddActionButton;
	static const float TextIconSize;
};