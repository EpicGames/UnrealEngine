// Copyright Epic Games, Inc. All Rights Reserved.
#include "Stack/SNiagaraStackItemGroupAddMenu.h"
#include "Styling/AppStyle.h"
#include "NiagaraActions.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "ViewModels/Stack/INiagaraStackItemGroupAddUtilities.h"
#include "SGraphActionMenu.h"
#include "SNiagaraGraphActionWidget.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SItemSelector.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "NiagaraStackItemGroupAddMenu"

bool SNiagaraStackItemGroupAddMenu::bLibraryOnly = true;

void SNiagaraStackItemGroupAddMenu::Construct(const FArguments& InArgs, UNiagaraStackEntry* InSourceEntry, INiagaraStackItemGroupAddUtilities* InAddUtilities, int32 InInsertIndex)
{
	SourceEntry = InSourceEntry;
	AddUtilities = InAddUtilities;
	InsertIndex = InInsertIndex;
	bSetFocusOnNextTick = true;

	SNiagaraFilterBox::FFilterOptions FilterOptions;
	FilterOptions.SetAddSourceFilter(AddUtilities->SupportsSourceFilter());
	FilterOptions.SetAddLibraryFilter(AddUtilities->SupportsLibraryFilter());

	// if a filter is not supported, the delegates won't impact anything, so it's fine to bind them in any case.
	SAssignNew(FilterBox, SNiagaraFilterBox, FilterOptions)
	.bLibraryOnly(this, &SNiagaraStackItemGroupAddMenu::GetLibraryOnly)
	.OnLibraryOnlyChanged(this, &SNiagaraStackItemGroupAddMenu::SetLibraryOnly)
    .OnSourceFiltersChanged(this, &SNiagaraStackItemGroupAddMenu::TriggerRefresh);	
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("AddToGroupFormatTitle", "Add new {0}"), AddUtilities->GetAddItemName()))
			]
			+SVerticalBox::Slot()
			.AutoHeight()
            [
				FilterBox.ToSharedRef()
            ]
			+SVerticalBox::Slot()
			[
				SNew(SBox)
				.WidthOverride(450)
				.HeightOverride(400)
				[
					SAssignNew(ActionSelector, SNiagaraStackAddSelector)
					.Items(CollectActions())
					.OnGetCategoriesForItem(this, &SNiagaraStackItemGroupAddMenu::OnGetCategoriesForItem)
	                .OnGetSectionsForItem(this, &SNiagaraStackItemGroupAddMenu::OnGetSectionsForItem)
	                .OnCompareSectionsForEquality(this, &SNiagaraStackItemGroupAddMenu::OnCompareSectionsForEquality)
	                .OnCompareSectionsForSorting(this, &SNiagaraStackItemGroupAddMenu::OnCompareSectionsForSorting)
	                .OnCompareCategoriesForEquality(this, &SNiagaraStackItemGroupAddMenu::OnCompareCategoriesForEquality)
	                .OnCompareCategoriesForSorting(this, &SNiagaraStackItemGroupAddMenu::OnCompareCategoriesForSorting)
	                .OnCompareItemsForSorting(this, &SNiagaraStackItemGroupAddMenu::OnCompareItemsForSorting)
	                .OnDoesItemMatchFilterText_Static(&FNiagaraEditorUtilities::DoesItemMatchFilterText)
	                .OnGenerateWidgetForSection(this, &SNiagaraStackItemGroupAddMenu::OnGenerateWidgetForSection)
	                .OnGenerateWidgetForCategory(this, &SNiagaraStackItemGroupAddMenu::OnGenerateWidgetForCategory)
	                .OnGenerateWidgetForItem(this, &SNiagaraStackItemGroupAddMenu::OnGenerateWidgetForItem)
	                .OnGetItemWeight_Lambda([](const TSharedPtr<FNiagaraMenuAction_Generic>& Item, const TArray<FString>& FilterTerms)
	                {
	                	return FNiagaraEditorUtilities::GetWeightForItem(Item, INiagaraStackItemGroupAddAction::FavoriteActionsProfile, FilterTerms);
	                })
	                .OnItemActivated(this, &SNiagaraStackItemGroupAddMenu::OnItemActivated)
	                .AllowMultiselect(false)
	                .OnDoesItemPassCustomFilter(this, &SNiagaraStackItemGroupAddMenu::DoesItemPassCustomFilter)
	                .ClickActivateMode(EItemSelectorClickActivateMode::SingleClick)
	                .ExpandInitially(AddUtilities->GetAutoExpandAddActions())
					.OnItemRowHoverEvent(this, &SNiagaraStackItemGroupAddMenu::OnItemRowHoverEvent)
	                .OnGetSectionData_Lambda([](const ENiagaraMenuSections& Section)
	                {
	                    if(Section == ENiagaraMenuSections::Suggested)
	                    {
	                        return SNiagaraStackAddSelector::FSectionData(SNiagaraStackAddSelector::FSectionData::List, true);
	                    }

	                    return SNiagaraStackAddSelector::FSectionData(SNiagaraStackAddSelector::FSectionData::Tree, false);
	                })
				]
			]
		]
	];
}

TSharedPtr<SWidget> SNiagaraStackItemGroupAddMenu::GetFilterTextBox()
{
	return ActionSelector->GetSearchBox();
}

TArray<TSharedPtr<FNiagaraMenuAction_Generic>> SNiagaraStackItemGroupAddMenu::CollectActions()
{
	TArray<TSharedPtr<FNiagaraMenuAction_Generic>> OutActions;
	
	FNiagaraStackItemGroupAddOptions AddOptions;
	AddOptions.bIncludeDeprecated = false;
	AddOptions.bIncludeNonLibrary = true;

	// We translate group add actions into the generic menu actions here
	
	TArray<TSharedRef<INiagaraStackItemGroupAddAction>> AddActions;
	AddUtilities->GenerateAddActions(AddActions, AddOptions);

	// For each group action, we parse its data and match data formats to the generic action
	for (TSharedRef<INiagaraStackItemGroupAddAction> AddAction : AddActions)
	{		
		TSharedPtr<FNiagaraMenuAction_Generic> NewNodeAction(
            new FNiagaraMenuAction_Generic(
            FNiagaraMenuAction_Generic::FOnExecuteAction::CreateRaw(AddUtilities, &INiagaraStackItemGroupAddUtilities::ExecuteAddAction, AddAction, InsertIndex),
            AddAction->GetDisplayName(), AddAction->GetCategories(), AddAction->GetFavoritesData(), AddAction->GetDescription(), AddAction->GetKeywords()));
		NewNodeAction->SourceData = AddAction->GetSourceData();
		NewNodeAction->bIsInLibrary = AddAction->IsInLibrary();
		NewNodeAction->PreviewMoviePath = AddAction->GetPreviewMoviePath();
		
		OutActions.Add(NewNodeAction);
	}

	return OutActions;
}

void SNiagaraStackItemGroupAddMenu::OnItemActivated(const TSharedPtr<FNiagaraMenuAction_Generic>& SelectedAction)
{
	TSharedPtr<FNiagaraMenuAction_Generic> CurrentAction = StaticCastSharedPtr<FNiagaraMenuAction_Generic>(SelectedAction);

	if (CurrentAction.IsValid())
	{
		FSlateApplication::Get().DismissAllMenus();
		CurrentAction->Execute();

		if(SourceEntry.IsValid())
		{
			SourceEntry->SetIsExpandedInOverview(true);
		}
	}	
}

void SNiagaraStackItemGroupAddMenu::OnItemRowHoverEvent(const TSharedPtr<FNiagaraMenuAction_Generic>& Item, bool bHover) const
{
	Item->bIsHovered = bHover;
}

void SNiagaraStackItemGroupAddMenu::TriggerRefresh(const TMap<EScriptSource, bool>& SourceState)
{
	ActionSelector->RefreshAllCurrentItems();

	TArray<bool> States;
	SourceState.GenerateValueArray(States);

	int32 NumActive = 0;
	for(bool& State : States)
	{
		if(State == true)
		{
			NumActive++;
		}
	}

	ActionSelector->ExpandTree();
}

bool SNiagaraStackItemGroupAddMenu::GetLibraryOnly() const
{
	return bLibraryOnly;
}

void SNiagaraStackItemGroupAddMenu::SetLibraryOnly(bool bInLibraryOnly)
{
	bLibraryOnly = bInLibraryOnly;
	ActionSelector->RefreshAllCurrentItems(true);
}

TArray<FString> SNiagaraStackItemGroupAddMenu::OnGetCategoriesForItem(
	const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	return Item->Categories;
}

TArray<ENiagaraMenuSections> SNiagaraStackItemGroupAddMenu::OnGetSectionsForItem(
	const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	TArray<ENiagaraMenuSections> Sections{ENiagaraMenuSections::General};
	
	if(Item->FavoritesActionData.IsSet())
	{
		FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(INiagaraStackItemGroupAddAction::FavoriteActionsProfile);
		if(ActionsProfile.IsFavorite(Item->FavoritesActionData.GetValue()))
		{
			Sections.Add(ENiagaraMenuSections::Suggested);
		}
	}
		
	return Sections;
}

bool SNiagaraStackItemGroupAddMenu::OnCompareSectionsForEquality(const ENiagaraMenuSections& SectionA,
	const ENiagaraMenuSections& SectionB)
{
	return SectionA == SectionB;
}

bool SNiagaraStackItemGroupAddMenu::OnCompareSectionsForSorting(const ENiagaraMenuSections& SectionA,
	const ENiagaraMenuSections& SectionB)
{
	return SectionA < SectionB;
}

bool SNiagaraStackItemGroupAddMenu::OnCompareCategoriesForEquality(const FString& CategoryA, const FString& CategoryB)
{
	return CategoryA.Compare(CategoryB) == 0;
}

bool SNiagaraStackItemGroupAddMenu::OnCompareCategoriesForSorting(const FString& CategoryA, const FString& CategoryB)
{
	return CategoryA.Compare(CategoryB) < 0;
}

bool SNiagaraStackItemGroupAddMenu::OnCompareItemsForEquality(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA,
	const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB)
{
	return ItemA->DisplayName.EqualTo(ItemB->DisplayName);
}

bool SNiagaraStackItemGroupAddMenu::OnCompareItemsForSorting(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA,
	const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB)
{
	return ItemA->DisplayName.CompareTo(ItemB->DisplayName) == -1;
}

TSharedRef<SWidget> SNiagaraStackItemGroupAddMenu::OnGenerateWidgetForSection(const ENiagaraMenuSections& Section)
{
	UEnum* SectionEnum = StaticEnum<ENiagaraMenuSections>();
	FText TextContent = SectionEnum->GetDisplayNameTextByValue((int64) Section);
	
	return SNew(STextBlock)
        .Text(TextContent)
        .TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.AssetPickerAssetCategoryText");
}

TSharedRef<SWidget> SNiagaraStackItemGroupAddMenu::OnGenerateWidgetForCategory(const FString& Category)
{
	FText TextContent = FText::FromString(Category);

	return SNew(SRichTextBlock)
        .Text(TextContent)
        .DecoratorStyleSet(&FAppStyle::Get())
        .TextStyle(FNiagaraEditorStyle::Get(), "ActionMenu.HeadingTextBlock");
}

TSharedRef<SWidget> SNiagaraStackItemGroupAddMenu::OnGenerateWidgetForItem(
	const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	FCreateNiagaraWidgetForActionData ActionData(Item);
	ActionData.FavoriteActionsProfileName = INiagaraStackItemGroupAddAction::FavoriteActionsProfile;
	ActionData.HighlightText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SNiagaraStackItemGroupAddMenu::GetFilterText));
	ActionData.PreviewMovieViewModel = SourceEntry.IsValid() ? SourceEntry->GetSystemViewModel()->GetPreviewMovieViewModel() : nullptr;
	return SNew(SNiagaraActionWidget, ActionData);
}

bool SNiagaraStackItemGroupAddMenu::DoesItemPassCustomFilter(const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	bool bLibraryConditionFulfilled = !AddUtilities->SupportsLibraryFilter() || (bLibraryOnly && Item->bIsInLibrary) || !bLibraryOnly;
	bool bSourceConditionFulfilled = !AddUtilities->SupportsSourceFilter() || FilterBox->IsSourceFilterActive(Item->SourceData.Source); 
	return bSourceConditionFulfilled && bLibraryConditionFulfilled;
}

#undef LOCTEXT_NAMESPACE
