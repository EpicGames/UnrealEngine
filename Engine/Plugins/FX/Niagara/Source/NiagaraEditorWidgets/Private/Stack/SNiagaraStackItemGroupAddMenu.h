// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraActions.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "EdGraph/EdGraphSchema.h"
#include "Styling/SlateTypes.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "Widgets/SItemSelector.h"
#include "Widgets/SNiagaraFilterBox.h"

class INiagaraStackItemGroupAddUtilities;

typedef SItemSelector<FString, TSharedPtr<FNiagaraMenuAction_Generic>, ENiagaraMenuSections> SNiagaraStackAddSelector;

class SNiagaraStackItemGroupAddMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackItemGroupAddMenu) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackEntry* InSourceEntry, INiagaraStackItemGroupAddUtilities* InAddUtilities, int32 InInsertIndex);

	TSharedPtr<SWidget> GetFilterTextBox();

private:
	bool GetLibraryOnly() const;

	void SetLibraryOnly(bool bInLibraryOnly);

private:
	INiagaraStackItemGroupAddUtilities* AddUtilities;
	TWeakObjectPtr<UNiagaraStackEntry> SourceEntry;
	int32 InsertIndex;

	TSharedPtr<SNiagaraStackAddSelector> ActionSelector;
	TSharedPtr<SNiagaraFilterBox> FilterBox;

	bool bSetFocusOnNextTick;
	
	static bool bLibraryOnly;

private:
	TArray<TSharedPtr<FNiagaraMenuAction_Generic>> CollectActions();
	TArray<FString> OnGetCategoriesForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	TArray<ENiagaraMenuSections> OnGetSectionsForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	bool OnCompareSectionsForEquality(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB);
	bool OnCompareSectionsForSorting(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB);
	bool OnCompareCategoriesForEquality(const FString& CategoryA, const FString& CategoryB);
	bool OnCompareCategoriesForSorting(const FString& CategoryA, const FString& CategoryB);
	bool OnCompareItemsForEquality(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA, const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB);
	bool OnCompareItemsForSorting(const TSharedPtr<FNiagaraMenuAction_Generic>& ItemA, const TSharedPtr<FNiagaraMenuAction_Generic>& ItemB);
	TSharedRef<SWidget> OnGenerateWidgetForSection(const ENiagaraMenuSections& Section);
	TSharedRef<SWidget> OnGenerateWidgetForCategory(const FString& Category);
	TSharedRef<SWidget> OnGenerateWidgetForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	bool DoesItemPassCustomFilter(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	void OnItemActivated(const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	void OnItemRowHoverEvent(const TSharedPtr<FNiagaraMenuAction_Generic>& Item, bool bHover) const;

	void TriggerRefresh(const TMap<EScriptSource, bool>& SourceState);

	FText GetFilterText() const { return ActionSelector->GetFilterText(); }
};