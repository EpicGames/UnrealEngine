// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackModuleItem.h"

#include "EditorFontGlyphs.h"
#include "NiagaraActions.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "NiagaraMessages.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "ScopedTransaction.h"
#include "SDropTarget.h"
#include "SGraphActionMenu.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraActionMenuExpander.h"
#include "Widgets/SNiagaraLibraryOnlyToggleHeader.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "NiagaraStackModuleItem"

bool SNiagaraStackModuleItem::bLibraryOnly = true;

void SNiagaraStackModuleItem::Construct(const FArguments& InArgs, UNiagaraStackModuleItem& InModuleItem, UNiagaraStackViewModel* InStackViewModel)
{
	ModuleItem = &InModuleItem;
	SNiagaraStackItem::Construct(SNiagaraStackItem::FArguments(), InModuleItem, InStackViewModel);
}

void SNiagaraStackModuleItem::FillRowContextMenu(FMenuBuilder& MenuBuilder)
{
	FNiagaraStackEditorWidgetsUtilities::AddStackModuleItemContextMenuActions(MenuBuilder, *ModuleItem, this->AsShared());
	FNiagaraStackEditorWidgetsUtilities::AddStackItemContextMenuActions(MenuBuilder, *ModuleItem);
}

FReply SNiagaraStackModuleItem::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (ModuleItem->OpenSourceAsset())
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SNiagaraStackModuleItem::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (ModuleItem->GetIsModuleScriptReassignmentPending())
	{
		ModuleItem->SetIsModuleScriptReassignmentPending(false);
		ShowReassignModuleScriptMenu();
	}
	SNiagaraStackItem::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SNiagaraStackModuleItem::AddCustomRowWidgets(TSharedRef<SHorizontalBox> HorizontalBox)
{
	// Scratch navigation
	if(ModuleItem->IsScratchModule())
	{
		HorizontalBox->AddSlot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			.OnClicked(this, &SNiagaraStackModuleItem::ScratchButtonPressed)
			.ToolTipText(LOCTEXT("OpenInScratchToolTip", "Open this module in the scratch pad."))
			.ContentPadding(FMargin(1.0f, 0.0f))
			.Content()
			[
				SNew(SImage)
				.Image(FNiagaraEditorStyle::Get().GetBrush("Tab.ScratchPad"))
			]
		];
	}

	// Version selector
	HorizontalBox->AddSlot()
    .VAlign(VAlign_Center)
    .AutoWidth()
    [
        SNew(SComboButton)
        .HasDownArrow(false)
        .ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
        .ForegroundColor(FSlateColor::UseForeground())
        .OnGetMenuContent(this, &SNiagaraStackModuleItem::GetVersionSelectorDropdownMenu)
        .ContentPadding(FMargin(2))
        .ToolTipText(this, &SNiagaraStackModuleItem::GetVersionSelectionMenuTooltip)
        .HAlign(HAlign_Center)
        .VAlign(VAlign_Center)
        .Visibility(this, &SNiagaraStackModuleItem::GetVersionSelectionMenuVisibility)
        .IsEnabled(this, &SNiagaraStackModuleItem::GetVersionSelectionMenuEnabled)
        .ButtonContent()
        [
	        SNew(STextBlock)
	        .Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
	        .ColorAndOpacity(this, &SNiagaraStackModuleItem::GetVersionSelectorColor)
	        .Text(FEditorFontGlyphs::Random)
        ]
    ];
	
	// Add menu.
	HorizontalBox->AddSlot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SAssignNew(AddButton, SComboButton)
		.HasDownArrow(false)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnGetMenuContent(this, &SNiagaraStackModuleItem::RaiseActionMenuClicked)
		.ContentPadding(FMargin(2))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Visibility(this, &SNiagaraStackModuleItem::GetRaiseActionMenuVisibility)
		.IsEnabled(this, &SNiagaraStackModuleItem::GetButtonsEnabled)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];

	// Refresh button
	HorizontalBox->AddSlot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.IsFocusable(false)
		.ToolTipText(LOCTEXT("RefreshTooltip", "Refresh this module"))
		.Visibility(this, &SNiagaraStackModuleItem::GetRefreshVisibility)
		.IsEnabled(this, &SNiagaraStackModuleItem::GetButtonsEnabled)
		.OnClicked(this, &SNiagaraStackModuleItem::RefreshClicked)
		.Content()
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
			.Text(FEditorFontGlyphs::Refresh)
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}

FSlateColor SNiagaraStackModuleItem::GetVersionSelectorColor() const
{
	UNiagaraScript* Script = ModuleItem->GetModuleNode().FunctionScript;
	
	if (Script && Script->IsVersioningEnabled())
	{
		FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(ModuleItem->GetModuleNode().SelectedScriptVersion);
		if (ScriptData && ScriptData->Version < Script->GetExposedVersion())
		{
			return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.IconColor.VersionUpgrade");
		}
	}
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.FlatButtonColor");
}

TSharedRef<SWidget> SNiagaraStackModuleItem::AddContainerForRowWidgets(TSharedRef<SWidget> RowWidgets)
{	
	return SNew(SDropTarget)
	.OnAllowDrop(this, &SNiagaraStackModuleItem::OnModuleItemAllowDrop)
	.OnDropped(this, &SNiagaraStackModuleItem::OnModuleItemDrop)
	.HorizontalImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.DropTarget.BorderHorizontal"))
	.VerticalImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.DropTarget.BorderVertical"))
	.Content()
	[
		RowWidgets
	];
}

bool SNiagaraStackModuleItem::GetButtonsEnabled() const
{
	return ModuleItem->GetOwnerIsEnabled() && ModuleItem->GetIsEnabled();
}

EVisibility SNiagaraStackModuleItem::GetRaiseActionMenuVisibility() const
{
	return CanRaiseActionMenu() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SNiagaraStackModuleItem::GetRefreshVisibility() const
{
	bool bShowRefresh = false;
	if (UNiagaraGraph* Graph = ModuleItem->GetModuleNode().GetCalledGraph())
	{
		FString OutVal;
		bShowRefresh = Graph->GetPropertyMetadata(TEXT("DisplayNameArg0"), OutVal);
	}
	return bShowRefresh && ModuleItem->CanRefresh() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SNiagaraStackModuleItem::GetVersionSelectionMenuVisibility() const
{	
	UNiagaraScript* Script = ModuleItem->GetModuleNode().FunctionScript;
	if (Script && Script->IsVersioningEnabled() && Script->GetAllAvailableVersions().Num() > 1)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

bool SNiagaraStackModuleItem::GetVersionSelectionMenuEnabled() const
{
	// if the module is inherited we do not allow version changes, in that case only the parent emitter can define the module version
	return ModuleItem->CanMoveAndDelete();
}

FText SNiagaraStackModuleItem::GetVersionSelectionMenuTooltip() const
{
	FText BaseText;
	if (ModuleItem->CanMoveAndDelete())
	{
		BaseText = LOCTEXT("VersionTooltip", "Change the version of this module script.");
	}
	else
	{
		BaseText = LOCTEXT("VersionTooltipDisabled", "The version of this module script can only be changed in the parent emitter.");
	}
	
	if (UNiagaraScript* Script = ModuleItem->GetModuleNode().FunctionScript; Script && Script->IsVersioningEnabled())
	{
		if (FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(ModuleItem->GetModuleNode().SelectedScriptVersion))
		{
			// add the version information if possible
			FText VersionText = FText::Format(FText::FromString("{0}.{1}"), ScriptData->Version.MajorVersion, ScriptData->Version.MinorVersion);
			return FText::Format(LOCTEXT("VersionTooltipFormat", "{0}\nCurrently used module version: {1}"), BaseText, VersionText);
		}
	}
	return BaseText;
}

FReply SNiagaraStackModuleItem::ScratchButtonPressed() const
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchModuleViewModel =
		ModuleItem->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(ModuleItem->GetModuleNode().FunctionScript);
	if (ScratchModuleViewModel.IsValid())
	{
		ModuleItem->GetSystemViewModel()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchModuleViewModel.ToSharedRef());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (SelectedActions.Num() == 1 && (InSelectionType == ESelectInfo::OnKeyPress || InSelectionType == ESelectInfo::OnMouseClick))
	{
		TSharedPtr<FNiagaraMenuAction> Action = StaticCastSharedPtr<FNiagaraMenuAction>(SelectedActions[0]);
		if (Action.IsValid())
		{
			FSlateApplication::Get().DismissAllMenus();
			Action->ExecuteAction();
		}
	}
}

TSharedRef<SWidget> SNiagaraStackModuleItem::RaiseActionMenuClicked()
{
	if (CanRaiseActionMenu())
	{
		UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(&ModuleItem->GetModuleNode());
		if (AssignmentNode != nullptr)
		{
			return AssignmentNode->CreateAddParameterMenu(AddButton);
		}
	}
	return SNullWidget::NullWidget;
}

void SNiagaraStackModuleItem::SwitchToVersion(FNiagaraAssetVersion Version)
{
	ModuleItem->ChangeScriptVersion(Version.VersionGuid);
}

TSharedRef<SWidget> SNiagaraStackModuleItem::GetVersionSelectorDropdownMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	UNiagaraScript* Script = ModuleItem->GetModuleNode().FunctionScript;
	TArray<FNiagaraAssetVersion> AssetVersions = Script->GetAllAvailableVersions();
	for (FNiagaraAssetVersion& Version : AssetVersions)
	{
		if (!Version.bIsVisibleInVersionSelector)
        {
        	continue;
        }
		FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(Version.VersionGuid);
		bool bIsSelected = ModuleItem->GetModuleNode().SelectedScriptVersion == Version.VersionGuid;
		
		FText Tooltip = LOCTEXT("NiagaraSelectVersion_Tooltip", "Select this version to use for the module");
		if (!ScriptData->VersionChangeDescription.IsEmpty())
		{
			Tooltip = FText::Format(LOCTEXT("NiagaraSelectVersionChangelist_Tooltip", "Select this version to use for the module. Change description for this version:\n{0}"), ScriptData->VersionChangeDescription);
		}
		
		FUIAction UIAction(FExecuteAction::CreateSP(this, &SNiagaraStackModuleItem::SwitchToVersion, Version),
        FCanExecuteAction(),
        FIsActionChecked::CreateLambda([bIsSelected]() { return bIsSelected; }));
        FText Format = (Version == Script->GetExposedVersion()) ? FText::FromString("{0}.{1}*") : FText::FromString("{0}.{1}");
        FText Label = FText::Format(Format, Version.MajorVersion, Version.MinorVersion);
		MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);	
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SExpanderArrow> SNiagaraStackModuleItem::CreateCustomActionExpander(const FCustomExpanderData& ActionMenuData)
{
	return SNew(SNiagaraActionMenuExpander, ActionMenuData);
}

bool SNiagaraStackModuleItem::CanRaiseActionMenu() const
{
	return Cast<UNiagaraNodeAssignment>(&ModuleItem->GetModuleNode()) != nullptr;
}

FReply SNiagaraStackModuleItem::RefreshClicked()
{
	ModuleItem->Refresh();
	return FReply::Handled();
}

FReply SNiagaraStackModuleItem::OnModuleItemDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragDropOperation = InDragDropEvent.GetOperation();
	if (DragDropOperation)
	{
		UNiagaraStackEntry::FDropRequest DropRequest(DragDropOperation.ToSharedRef(), EItemDropZone::OntoItem, UNiagaraStackEntry::EDragOptions::None, UNiagaraStackEntry::EDropOptions::None);
		TOptional<UNiagaraStackEntry::FDropRequestResponse> DropResponse = ModuleItem->Drop(DropRequest);
		return (DropResponse.IsSet() && (DropResponse->DropZone == EItemDropZone::OntoItem)) ? FReply::Handled() : FReply::Unhandled();
	}

	return FReply::Unhandled();
}

bool SNiagaraStackModuleItem::OnModuleItemAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	UNiagaraStackEntry::FDropRequest AllowDropRequest(DragDropOperation.ToSharedRef(), EItemDropZone::OntoItem, UNiagaraStackEntry::EDragOptions::None, UNiagaraStackEntry::EDropOptions::None);
	TOptional<UNiagaraStackEntry::FDropRequestResponse> AllowDropResponse = ModuleItem->CanDrop(AllowDropRequest);
	return AllowDropResponse.IsSet() && AllowDropResponse->DropZone == EItemDropZone::OntoItem;
}

void ReassignModuleScript(UNiagaraStackModuleItem* ModuleItem, FAssetData NewModuleScriptAsset)
{
	UNiagaraScript* NewModuleScript = Cast<UNiagaraScript>(NewModuleScriptAsset.GetAsset());
	if (NewModuleScript != nullptr)
	{
		ModuleItem->ReassignModuleScript(NewModuleScript);
	}
}

void SNiagaraStackModuleItem::CollectModuleActions(FGraphActionListBuilderBase& ModuleActions)
{
	TArray<FAssetData> ModuleAssets;
	FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions ModuleScriptFilterOptions;
	ModuleScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::Module;
	ModuleScriptFilterOptions.TargetUsageToMatch = ModuleItem->GetOutputNode()->GetUsage();
	ModuleScriptFilterOptions.bIncludeNonLibraryScripts = bLibraryOnly == false;
	FNiagaraEditorUtilities::GetFilteredScriptAssets(ModuleScriptFilterOptions, ModuleAssets);
	for (const FAssetData& ModuleAsset : ModuleAssets)
	{
		FText Category;
		ModuleAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Category), Category);
		if (Category.IsEmptyOrWhitespace())
		{
			Category = LOCTEXT("ModuleNotCategorized", "Uncategorized Modules");
		}

		bool bIsInLibrary = FNiagaraEditorUtilities::IsScriptAssetInLibrary(ModuleAsset);

		FText DisplayName = FNiagaraEditorUtilities::FormatScriptName(ModuleAsset.AssetName, bIsInLibrary);

		FText AssetDescription;
		ModuleAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Description), AssetDescription);
		FText Description = FNiagaraEditorUtilities::FormatScriptDescription(AssetDescription, ModuleAsset.GetSoftObjectPath(), bIsInLibrary);

		FText Keywords;
		ModuleAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Keywords), Keywords);

		TSharedPtr<FNiagaraMenuAction> ModuleAction(new FNiagaraMenuAction(Category, DisplayName, Description, 0, Keywords,
			FNiagaraMenuAction::FOnExecuteStackAction::CreateStatic(&ReassignModuleScript, ModuleItem, ModuleAsset)));
		ModuleActions.AddAction(ModuleAction);
	}
}

void SNiagaraStackModuleItem::ShowReassignModuleScriptMenu()
{
	TSharedPtr<SNiagaraLibraryOnlyToggleHeader> LibraryOnlyToggle;
	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	TSharedRef<SBorder> MenuWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			.WidthOverride(300)
			.HeightOverride(400)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(1.0f)
				[
					SAssignNew(LibraryOnlyToggle, SNiagaraLibraryOnlyToggleHeader)
					.HeaderLabelText(LOCTEXT("ReassignModuleLabel", "Select a new module"))
					.LibraryOnly(this, &SNiagaraStackModuleItem::GetLibraryOnly)
					.LibraryOnlyChanged(this, &SNiagaraStackModuleItem::SetLibraryOnly)
				]
				+SVerticalBox::Slot()
				.FillHeight(15)
				[
					SAssignNew(GraphActionMenu, SGraphActionMenu)
					.OnActionSelected_Static(OnActionSelected)
					.OnCollectAllActions(this, &SNiagaraStackModuleItem::CollectModuleActions)
					.ShowFilterTextBox(true)
				]
			]
		];

	LibraryOnlyToggle->SetActionMenu(GraphActionMenu.ToSharedRef());

	FGeometry ThisGeometry = GetCachedGeometry();
	bool bAutoAdjustForDpiScale = false; // Don't adjust for dpi scale because the push menu command is expecting an unscaled position.
	FVector2D MenuPosition = FSlateApplication::Get().CalculatePopupWindowPosition(ThisGeometry.GetLayoutBoundingRect(), MenuWidget->GetDesiredSize(), bAutoAdjustForDpiScale);
	FSlateApplication::Get().PushMenu(AsShared(), FWidgetPath(), MenuWidget, MenuPosition, FPopupTransitionEffect::ContextMenu);
}

bool SNiagaraStackModuleItem::GetLibraryOnly() const
{
	return bLibraryOnly;
}

void SNiagaraStackModuleItem::SetLibraryOnly(bool bInLibraryOnly)
{
	bLibraryOnly = bInLibraryOnly;
}

#undef LOCTEXT_NAMESPACE