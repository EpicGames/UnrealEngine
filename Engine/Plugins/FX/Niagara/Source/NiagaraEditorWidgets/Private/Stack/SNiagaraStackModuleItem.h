// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraStackItem.h"
#include "Styling/SlateTypes.h"
#include "Layout/Visibility.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

class UNiagaraStackModuleItem;
class UNiagaraStackViewModel;
class SNiagaraStackDisplayName;
struct FGraphActionListBuilderBase;
class SComboButton;

class SNiagaraStackModuleItem : public SNiagaraStackItem
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackModuleItem) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackModuleItem& InModuleItem, UNiagaraStackViewModel* InStackViewModel);

	void FillRowContextMenu(class FMenuBuilder& MenuBuilder);

	//~ SWidget interface
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	virtual void AddCustomRowWidgets(TSharedRef<SHorizontalBox> HorizontalBox) override;

	virtual TSharedRef<SWidget> AddContainerForRowWidgets(TSharedRef<SWidget> RowWidgets) override;

private:
	bool GetButtonsEnabled() const;

	EVisibility GetRaiseActionMenuVisibility() const;

	EVisibility GetRefreshVisibility() const;

	EVisibility GetVersionSelectionMenuVisibility() const;
	bool GetVersionSelectionMenuEnabled() const;
	FText GetVersionSelectionMenuTooltip() const;

	FReply ScratchButtonPressed() const;
	
	TSharedRef<SWidget> RaiseActionMenuClicked();
	
	TSharedRef<SWidget> GetVersionSelectorDropdownMenu();

	bool CanRaiseActionMenu() const;

	static TSharedRef<SExpanderArrow> CreateCustomActionExpander(const struct FCustomExpanderData& ActionMenuData);

	FReply RefreshClicked();

	FReply OnModuleItemDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

	FSlateColor GetVersionSelectorColor() const;

	bool OnModuleItemAllowDrop(TSharedPtr<class FDragDropOperation> DragDropOperation);

	void CollectModuleActions(FGraphActionListBuilderBase& ModuleActions);

	void ShowReassignModuleScriptMenu();

	bool GetLibraryOnly() const;

	void SetLibraryOnly(bool bInLibraryOnly);

	void SwitchToVersion(FNiagaraAssetVersion Version);

private:
	UNiagaraStackModuleItem* ModuleItem;

	TSharedPtr<SComboButton> AddButton;

	static bool bLibraryOnly;

};