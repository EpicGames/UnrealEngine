// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStackRowPerfWidget.h"

#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"

#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"

#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "SNiagaraStackRowPerfWidget"

IConsoleVariable* SNiagaraStackRowPerfWidget::StatEnabledVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.EditorPerformanceStatsEnabled"));

void SNiagaraStackRowPerfWidget::Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry, ENiagaraPerformanceStateScopeMode InStatScopeMode)
{
	StackEntry = InStackEntry;
	StatScopeMode = InStatScopeMode;

	SetToolTipText(CreateTooltipText());
	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SNiagaraStackRowPerfWidget::IsVisible));
	
	TSharedRef<SWidget> PerfWidget =
		SNew(SBox)
        .HeightOverride(16)
		.WidthOverride(70)
        .HAlign(HAlign_Right)
        .VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			  .HAlign(HAlign_Right)
			  .VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
	            [
	                // displaying evaluation type
		            SNew(STextBlock)
		            .Margin(FMargin(0, 0, 3, 1))
		            .Justification(ETextJustify::Right)
		            .Font(FNiagaraEditorWidgetsStyle::Get().GetFontStyle("NiagaraEditor.Stack.Stats.EvalTypeFont"))
		            .ColorAndOpacity(FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.EvalTypeColor"))
		            .Text(this, &SNiagaraStackRowPerfWidget::GetEvalTypeDisplayText)
	            ]
	            + SHorizontalBox::Slot()
                [
	                // displaying perf cost
		            SNew(STextBlock)
		            .Justification(ETextJustify::Right)
		            .Margin(FMargin(0, 0, 0, 1))
		            .Font(this, &SNiagaraStackRowPerfWidget::GetPerformanceDisplayTextFont)
		            .ColorAndOpacity(this, &SNiagaraStackRowPerfWidget::GetPerformanceDisplayTextColor)
		            .Text(this, &SNiagaraStackRowPerfWidget::GetPerformanceDisplayText)
                ]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(70 - GetFullBarWidth(), 0, 0, 2))
            .VAlign(VAlign_Bottom)
            [
				SNew(SWrapBox)
				.UseAllottedSize(true)
			    +SWrapBox::Slot()
                [
                    // Placeholder brush to fill the remaining space
                    SNew(SBox)
                    .HeightOverride(2)
                    .WidthOverride(this, &SNiagaraStackRowPerfWidget::GetPlaceholderBrushWidth)
					.Visibility(this, &SNiagaraStackRowPerfWidget::GetPerfBarVisibility)
                    [
                        SNew(SColorBlock)
                        .Color(this, &SNiagaraStackRowPerfWidget::GetPlaceholderBrushColor)
                    ]
                ]
                +SWrapBox::Slot()
                [
                    // Colored visualization brush
                    SNew(SBox)
                    .HeightOverride(2)
                    .WidthOverride(this, &SNiagaraStackRowPerfWidget::GetVisualizationBrushWidth)
					.Visibility(this, &SNiagaraStackRowPerfWidget::GetPerfBarVisibility)
                    [
                        SNew(SColorBlock)
                        .Color(this, &SNiagaraStackRowPerfWidget::GetVisualizationBrushColor)
                    ]
                ]
            ]
		];

	ChildSlot
	[
		PerfWidget
	];
}

void SNiagaraStackRowPerfWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!HasPerformanceData())
	{
		return;
	}
	GroupOverallTime = CalculateGroupOverallTime(IsEmitterStack() ? GetEmitter().Emitter->GetUniqueEmitterName() : "Main");
	if (IsSystemStack())
	{
		EmitterTimeTotal = 0;
		for (const FNiagaraEmitterHandle& Handle : StackEntry->GetSystemViewModel()->GetSystem().GetEmitterHandles())
		{
			EmitterTimeTotal += CalculateGroupOverallTime(Handle.GetInstance().Emitter->GetUniqueEmitterName());
		}
	}
	if (IsGpuEmitter())
	{
		if (GetUsage() == ENiagaraScriptUsage::ParticleUpdateScript)
		{
			static const FString GpuStatName = TEXT("GPU_Stage_") + UNiagaraSimulationStageBase::ParticleSpawnUpdateName.ToString();
			GroupOverallTime = CalculateGroupOverallTime(GpuStatName);
		}
		if (GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			FString SimStageName = "";
			UNiagaraStackSimulationStageGroup* SimStageGroup = Cast<UNiagaraStackSimulationStageGroup>(StackEntry);
			if (SimStageGroup)
			{
				UNiagaraSimulationStageBase* SimStage = SimStageGroup->GetSimulationStage();
				SimStageName = SimStage->SimulationStageName.ToString();
			}
			
			GroupOverallTime = CalculateGroupOverallTime("GPU_Stage_" + SimStageName);
		}
	}
	
	StackEntryTime = CalculateStackEntryTime();
	if (GetUsage() == ENiagaraScriptUsage::ParticleSpawnScript && IsInterpolatedSpawnEnabled())
	{
		UpdateInSpawnTime = CalculateGroupOverallTime("MapUpdateMain");
	}
}

float SNiagaraStackRowPerfWidget::GetFullBarWidth() const
{
	return IsGroupHeaderEntry() ? 54 : 40;
}

EVisibility SNiagaraStackRowPerfWidget::GetPerfBarVisibility() const
{
	return StatScopeMode == ENiagaraPerformanceStateScopeMode::Detailed ? EVisibility::Visible : EVisibility::Hidden;
}

FOptionalSize SNiagaraStackRowPerfWidget::GetVisualizationBrushWidth() const
{
	if (IsGroupHeaderEntry())
	{
		if (GetUsage() == ENiagaraScriptUsage::ParticleSpawnScript && IsInterpolatedSpawnEnabled())
		{
			float Factor = GroupOverallTime == 0 ? 1 : (1 - UpdateInSpawnTime / GroupOverallTime);
			return GetFullBarWidth() * FMath::Min(Factor, 1.0f);
		}
		if (IsSystemStack())
		{
			float Factor = GroupOverallTime == 0 ? 1 : (1 - EmitterTimeTotal / GroupOverallTime);
			return GetFullBarWidth() * FMath::Min(Factor, 1.0f);
		}
		return GetFullBarWidth();
	}
	float PercentageFactor = GroupOverallTime == 0 ? 0 : StackEntryTime / GroupOverallTime;
	return GetFullBarWidth() * FMath::Min(PercentageFactor, 1.0f);
}

FOptionalSize SNiagaraStackRowPerfWidget::GetPlaceholderBrushWidth() const
{
	return GetFullBarWidth() - GetVisualizationBrushWidth().Get();
}

FLinearColor SNiagaraStackRowPerfWidget::GetVisualizationBrushColor() const
{
	ENiagaraScriptUsage Usage = GetUsage();
	if (Usage == ENiagaraScriptUsage::ParticleUpdateScript)
	{
		if (IsGpuEmitter())
		{
			return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleSpawn");
		}
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleUpdate");
	}
	if (Usage == ENiagaraScriptUsage::ParticleSpawnScript)
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleSpawn");
	}
	if (Usage == ENiagaraScriptUsage::SystemSpawnScript || Usage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		if (IsEmitterStack())
		{
			return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorEmitter");
		}
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorSystem");
	}
	if (Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleUpdate");
	}
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorDefault");
}

FLinearColor SNiagaraStackRowPerfWidget::GetPlaceholderBrushColor() const
{
	if (IsGroupHeaderEntry() && GetUsage() == ENiagaraScriptUsage::ParticleSpawnScript && IsInterpolatedSpawnEnabled())
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleUpdate");
	}
	if (IsGroupHeaderEntry() && GetUsage() == ENiagaraScriptUsage::SystemUpdateScript && IsSystemStack())
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimeUsageColorEmitter");
	}
	if (IsGroupHeaderEntry() && GetUsage() == ENiagaraScriptUsage::SystemSpawnScript && IsSystemStack())
	{
		return FLinearColor(FColor(241, 99, 6));
	}
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.RuntimePlaceholderColor");
}

bool SNiagaraStackRowPerfWidget::HasPerformanceData() const
{
#if STATS
	if (IsFinalized())
	{
		return false;
	}

	const bool IsPerfCaptureEnabled = StackEntry->GetSystemViewModel()->IsPerformanceStatsEnabled() && (StackEntry->GetSystemViewModel()->GetPerformanceStateScopeMode() >= StatScopeMode);
	if (IsPerfCaptureEnabled && (IsSystemStack() || IsEmitterStack()))
	{
		return (IsGroupHeaderEntry() || IsModuleEntry()) && (StackEntry->GetExecutionSubcategoryName() != UNiagaraStackEntry::FExecutionSubcategoryNames::Settings);
	}
	if (IsPerfCaptureEnabled && IsParticleStack())
	{
		return IsGpuEmitter() ? (IsGroupHeaderEntry() && GetUsage() != ENiagaraScriptUsage::ParticleSpawnScript) : (IsGroupHeaderEntry() || IsModuleEntry());
	}
	return false;
#else
	return false;
#endif
}

bool SNiagaraStackRowPerfWidget::IsSystemStack() const
{
	return StackEntry.IsValid() && StackEntry->GetExecutionCategoryName() == UNiagaraStackEntry::FExecutionCategoryNames::System;
}

bool SNiagaraStackRowPerfWidget::IsEmitterStack() const
{
	return StackEntry.IsValid() && StackEntry->GetEmitterViewModel().IsValid() && StackEntry->GetExecutionCategoryName() == UNiagaraStackEntry::FExecutionCategoryNames::Emitter;
}

bool SNiagaraStackRowPerfWidget::IsParticleStack() const
{
	return StackEntry.IsValid() && StackEntry->GetEmitterViewModel().IsValid() && StackEntry->GetExecutionCategoryName() == UNiagaraStackEntry::FExecutionCategoryNames::Particle;
}

EVisibility SNiagaraStackRowPerfWidget::IsVisible() const
{
	return HasPerformanceData() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SNiagaraStackRowPerfWidget::IsFinalized() const
{
	return !StackEntry.IsValid() || StackEntry->IsFinalized();
}

FText SNiagaraStackRowPerfWidget::GetPerformanceDisplayText() const
{
	// guard against the stack entry having been finalized
	if (IsFinalized())
	{
		return FText::FromString("N/A");
	}

	FNumberFormattingOptions TimeFormatOptions;
	TimeFormatOptions.MinimumIntegralDigits = 1;
	TimeFormatOptions.MinimumFractionalDigits = 3;
	TimeFormatOptions.MaximumFractionalDigits = 3;
	TimeFormatOptions.UseGrouping = false;
	
	if (IsGroupHeaderEntry())
	{
		if (GroupOverallTime == 0)
		{
			return FText::FromString("N/A");
		}
		if (GroupOverallTime >= 1000)
		{
			return FText::Format(FText::FromString("{0}s"), FText::AsNumber(GroupOverallTime / 1000, &TimeFormatOptions));
		}
		return FText::Format(FText::FromString("{0}ms"), FText::AsNumber(GroupOverallTime, &TimeFormatOptions));
	}
	if (GetDisplayMode() == ENiagaraStatDisplayMode::Absolute)
	{
		if (StackEntryTime >= 1000)
		{
			return FText::Format(FText::FromString("{0}s"), FText::AsNumber(StackEntryTime / 1000, &TimeFormatOptions));
		}
		return FText::Format(FText::FromString("{0}ms"), FText::AsNumber(StackEntryTime, &TimeFormatOptions));
	}
	FNumberFormattingOptions PercentFormatOptions;
	PercentFormatOptions.MinimumIntegralDigits = 1;
	PercentFormatOptions.MinimumFractionalDigits = 1;
	PercentFormatOptions.MaximumFractionalDigits = 1;
	PercentFormatOptions.RoundingMode = HalfToZero;
	float RuntimeFactor = GroupOverallTime == 0 ? 0 : StackEntryTime / GroupOverallTime;
	return FText::Format(FText::FromString("{0}%"), FText::AsNumber(RuntimeFactor * 100, &PercentFormatOptions));
}

FText SNiagaraStackRowPerfWidget::GetEvalTypeDisplayText() const
{
	if (!IsGroupHeaderEntry() || GroupOverallTime == 0)
	{
		return FText();
	}
	return FText::FromString(GetEvaluationType() == ENiagaraStatEvaluationType::Maximum ? "Max" : "Avg");
}

FSlateColor SNiagaraStackRowPerfWidget::GetPerformanceDisplayTextColor() const
{
	// guard against the stack entry having been finalized
	if (IsFinalized())
	{
		return FSlateColor(FLinearColor::Black);
	}
	if (IsEntrySelected())
	{
		return FSlateColor(FLinearColor::Black);
	}
	if (IsGroupHeaderEntry())
	{
		return FSlateColor(FLinearColor::White);
	}
	float RuntimeFactor = GroupOverallTime == 0 ? 0 : StackEntryTime / GroupOverallTime;
	if (RuntimeFactor < 0.25)
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.LowCostColor");
	}
	if (RuntimeFactor < 0.5)
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.MediumCostColor");
	}
	if (RuntimeFactor < 0.75)
	{
		return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.HighCostColor");
	}
	return FNiagaraEditorWidgetsStyle::Get().GetColor("NiagaraEditor.Stack.Stats.MaxCostColor");
}

FSlateFontInfo SNiagaraStackRowPerfWidget::GetPerformanceDisplayTextFont() const
{
	if (IsGroupHeaderEntry())
	{
		return FNiagaraEditorWidgetsStyle::Get().GetFontStyle("NiagaraEditor.Stack.Stats.GroupFont");
	}
	return FNiagaraEditorWidgetsStyle::Get().GetFontStyle("NiagaraEditor.Stack.Stats.DetailFont");
}

FText SNiagaraStackRowPerfWidget::CreateTooltipText() const
{
	if (IsModuleEntry())
	{
		if (IsEmitterStack())
		{
			return LOCTEXT("EmitterEntryTooltip", "This shows the module runtime cost in percent.\nThe displayed percentages relate only to the parent emitter script, not the whole system script.\nNote that the module calls do not add up to 100% because the emitter script itself has some overhead as well.");
		}
		if (IsSystemStack())
		{
			return LOCTEXT("SystemEntryTooltip", "This shows the module runtime cost in percent.\nThe displayed percentages relate to the full system script, which has the emitter scripts embedded.\nNote that even without emitter cost, the module calls do not add up to 100% because the system script itself has some overhead as well.");
		}
		return LOCTEXT("ModuleEntryTooltip", "This shows the module runtime cost in percent.\nThe displayed percentage relates only to the parent script, not the whole emitter.\nThe module cost does not include work from DI calls to other threads, e.g. async collision traces.");
	}
	if (IsGroupHeaderEntry())
	{
		if (IsSystemStack())
		{
			return LOCTEXT("GroupHeaderSystemTooltip", "This is the total runtime cost of the system script and its module calls.\nSince the emitter scripts are embedded in the system scripts, they are part of this cost as well and visualized in a different color.\nNote that the module calls do not add up to 100% because the system script itself has some overhead as well.");
		}
		if (IsEmitterStack())
		{
			return LOCTEXT("GroupHeaderEmitterTooltip", "This is the total runtime cost of the emitter script and its module calls.\nNote that the module calls do not add up to 100% because the script itself has some overhead as well.");
		}
		if (GetUsage() == ENiagaraScriptUsage::ParticleSpawnScript)
		{
			return LOCTEXT("GroupHeaderSpawnTooltip", "This is the total runtime cost of the spawn script and its module calls.\nNote that the module calls do not add up to 100% because the script itself has some overhead as well.\nWhen interpolated spawn is enabled, this also includes the cost of running the update script on the spawned particles.");
		}
		if (IsGpuEmitter())
		{
			if (GetUsage() == ENiagaraScriptUsage::ParticleUpdateScript)
			{
				return LOCTEXT("GroupHeaderUpdateGPUTooltip", "This is the total GPU runtime cost of the combined spawn and update script call.\nThere are no individual module costs because the gpu stats cannot be measured on that level.");
			}
			if (GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
			{
				return LOCTEXT("GroupHeaderSimStageGPUTooltip", "This is the total GPU runtime cost of the simulation stage script.\nIf the simulation stage has more than one iteration then this is the combined sum of all iterations.");
			}
		}
		return LOCTEXT("GroupHeaderTooltip", "This is the total runtime cost of the script and its module calls.\nNote that the module calls do not add up to 100% because the script itself has some overhead as well.");
	}
	return FText();
}

bool SNiagaraStackRowPerfWidget::IsGroupHeaderEntry() const
{
	return StackEntry.IsValid() && StackEntry->IsA(UNiagaraStackScriptItemGroup::StaticClass());
}

bool SNiagaraStackRowPerfWidget::IsModuleEntry() const
{
	return StackEntry.IsValid() && StackEntry->IsA(UNiagaraStackModuleItem::StaticClass());
}

bool SNiagaraStackRowPerfWidget::IsEntrySelected() const
{
	if (!StackEntry.IsValid())
	{
		return false;
	}
	if (!StackEntry->GetSystemViewModel()->GetSelectionViewModel())
	{
		return false;
	}
	return StackEntry->GetSystemViewModel()->GetSelectionViewModel()->ContainsEntry(StackEntry.Get());
}

FVersionedNiagaraEmitter SNiagaraStackRowPerfWidget::GetEmitter() const
{
	if (!StackEntry->GetEmitterViewModel())
	{
		return FVersionedNiagaraEmitter();
	}
	return StackEntry->GetEmitterViewModel().Get()->GetEmitter();
}

ENiagaraScriptUsage SNiagaraStackRowPerfWidget::GetUsage() const
{
	FName SubcategoryName = StackEntry->GetExecutionSubcategoryName();
	if (SubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::SimulationStage)
	{
		return ENiagaraScriptUsage::ParticleSimulationStageScript;
	}
	if (SubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn)
	{
		return StackEntry->GetExecutionCategoryName() == UNiagaraStackEntry::FExecutionCategoryNames::Particle ? ENiagaraScriptUsage::ParticleSpawnScript : ENiagaraScriptUsage::SystemSpawnScript;
	}
	return StackEntry->GetExecutionCategoryName() == UNiagaraStackEntry::FExecutionCategoryNames::Particle ? ENiagaraScriptUsage::ParticleUpdateScript :  ENiagaraScriptUsage::SystemUpdateScript;
}

ENiagaraStatEvaluationType SNiagaraStackRowPerfWidget::GetEvaluationType() const
{
	TSharedPtr<FNiagaraSystemViewModel> ViewModel = StackEntry.IsValid() ? StackEntry->GetSystemViewModelPtr() : nullptr;
	return ViewModel.IsValid() ? ViewModel->StatEvaluationType : ENiagaraStatEvaluationType::Average;
}

ENiagaraStatDisplayMode SNiagaraStackRowPerfWidget::GetDisplayMode() const
{
	TSharedPtr<FNiagaraSystemViewModel> ViewModel = StackEntry.IsValid() ? StackEntry->GetSystemViewModelPtr() : nullptr;
	return ViewModel.IsValid() ? ViewModel->StatDisplayMode : ENiagaraStatDisplayMode::Percent;
}

bool SNiagaraStackRowPerfWidget::IsInterpolatedSpawnEnabled() const
{
	FVersionedNiagaraEmitterData* EmitterData = GetEmitter().GetEmitterData();
	if (!EmitterData)
	{
		return false;
	}
	return EmitterData->UsesInterpolatedSpawning();
}

bool SNiagaraStackRowPerfWidget::IsGpuEmitter() const
{
	FVersionedNiagaraEmitterData* EmitterData = GetEmitter().GetEmitterData();
	return EmitterData && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim;
}

float SNiagaraStackRowPerfWidget::CalculateGroupOverallTime(FString StatScopeName) const
{
#if STATS
	if (!HasPerformanceData())
	{
		return 0;
	}
	if (IsParticleStack())
	{
		FVersionedNiagaraEmitterData* EmitterData = GetEmitter().GetEmitterData();
		if (!EmitterData)
		{
			return 0;
		}
		TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(StatScopeName);
		if (IsGpuEmitter())
		{
			return EmitterData->GetStatData().GetRuntimeStat(StatId.GetName(), ENiagaraScriptUsage::ParticleGPUComputeScript, GetEvaluationType()) / 1000.0f;
		}
		return FPlatformTime::ToMilliseconds(EmitterData->GetStatData().GetRuntimeStat(StatId.GetName(), GetUsage(), GetEvaluationType()));
	}
	if (IsSystemStack() || IsEmitterStack())
	{
		UNiagaraSystem& System = StackEntry->GetSystemViewModel()->GetSystem();
		TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(StatScopeName);
        return FPlatformTime::ToMilliseconds(System.GetStatData().GetRuntimeStat(StatId.GetName(), GetUsage(), GetEvaluationType()));
	}
	return 0;
#else
	return 0;
#endif
}

float SNiagaraStackRowPerfWidget::CalculateStackEntryTime() const
{
#if STATS
	if (!HasPerformanceData())
	{
		return 0;
	}
	if (IsGroupHeaderEntry())
	{
		return GroupOverallTime;
	}

	UNiagaraStackModuleItem* ModuleItem = Cast<UNiagaraStackModuleItem>(StackEntry);
	UNiagaraNodeFunctionCall& FunctionNode = ModuleItem->GetModuleNode();
	if (IsParticleStack())
	{
		FVersionedNiagaraEmitterData* EmitterData = GetEmitter().GetEmitterData();
		if (!EmitterData)
		{
			return 0;
		}

		FString StatScopeName = FunctionNode.GetFunctionName() + "_Emitter";
		TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(StatScopeName);
		FString StatName = StatId.GetName().ToString();
		return FPlatformTime::ToMilliseconds(EmitterData->GetStatData().GetRuntimeStat(StatId.GetName(), GetUsage(), GetEvaluationType()));
	}
	if (IsEmitterStack())
	{
		UNiagaraEmitter* Emitter = GetEmitter().Emitter;
		if (!Emitter)
		{
			return 0;
		}
		FString StatScopeName = FunctionNode.GetFunctionName() + "_" + Emitter->GetUniqueEmitterName();
		TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(StatScopeName);
		FString StatName = StatId.GetName().ToString();
		UNiagaraSystem& System = StackEntry->GetSystemViewModel()->GetSystem();
		return FPlatformTime::ToMilliseconds(System.GetStatData().GetRuntimeStat(StatId.GetName(), GetUsage(), GetEvaluationType()));
	}
	if (IsSystemStack())
	{
		TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(FunctionNode.GetFunctionName());
		FString StatName = StatId.GetName().ToString();
		UNiagaraSystem& System = StackEntry->GetSystemViewModel()->GetSystem();
		return FPlatformTime::ToMilliseconds(System.GetStatData().GetRuntimeStat(StatId.GetName(), GetUsage(), GetEvaluationType()));
	}
	return 0;
#else
	return 0;
#endif
}

#undef LOCTEXT_NAMESPACE
