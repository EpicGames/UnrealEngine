// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitter.h"
#include "Types/SlateStructs.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UNiagaraStackEntry;
enum class ENiagaraPerformanceStateScopeMode : int;

class SNiagaraStackRowPerfWidget: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackRowPerfWidget) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, UNiagaraStackEntry* InStackEntry, ENiagaraPerformanceStateScopeMode InStatScopeMode);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
private:
	float GetFullBarWidth() const;
	EVisibility GetPerfBarVisibility() const;
	FOptionalSize GetVisualizationBrushWidth() const;
	FOptionalSize GetPlaceholderBrushWidth() const;
	FLinearColor GetVisualizationBrushColor() const;
	FLinearColor GetPlaceholderBrushColor() const;
	bool HasPerformanceData() const;
	bool IsSystemStack() const;
	bool IsEmitterStack() const;
	bool IsParticleStack() const;
	EVisibility IsVisible() const;
	FText GetPerformanceDisplayText() const;
	FText GetEvalTypeDisplayText() const;
	FSlateColor GetPerformanceDisplayTextColor() const;
	FSlateFontInfo GetPerformanceDisplayTextFont() const;
	FText CreateTooltipText() const;

	bool IsGroupHeaderEntry() const;
	bool IsModuleEntry() const;
	bool IsEntrySelected() const;
	FVersionedNiagaraEmitter GetEmitter() const;
	ENiagaraScriptUsage GetUsage() const;
	ENiagaraStatEvaluationType GetEvaluationType() const;
	ENiagaraStatDisplayMode GetDisplayMode() const;
	bool IsInterpolatedSpawnEnabled() const;
	bool IsGpuEmitter() const;
	bool IsFinalized() const;
	
	float CalculateGroupOverallTime(FString StatScopeName) const;
	float CalculateStackEntryTime() const;

	mutable TOptional<FText> IconToolTipCache;
	TWeakObjectPtr<UNiagaraStackEntry> StackEntry;
	static IConsoleVariable* StatEnabledVar;
	ENiagaraPerformanceStateScopeMode StatScopeMode;

	float GroupOverallTime = 0;
	float StackEntryTime = 0;
	float UpdateInSpawnTime = 0;
	float EmitterTimeTotal = 0;
};