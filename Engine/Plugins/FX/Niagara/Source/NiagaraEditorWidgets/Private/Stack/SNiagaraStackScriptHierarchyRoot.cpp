// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackScriptHierarchyRoot.h"

#include "NiagaraEditorWidgetsStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "NiagaraEditor"

void SNiagaraStackHierarchySection::Construct(const FArguments& InArgs, const UHierarchySection* InSection)
{
	Section = InSection;
	
	ChildSlot
	[
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.OnCheckStateChanged(InArgs._OnCheckStateChanged)
		.IsChecked(InArgs._IsChecked)
		.ToolTipText(this, &SNiagaraStackHierarchySection::GetTooltipText)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "SmallText")
			.Text(this, &SNiagaraStackHierarchySection::GetSectionNameAsText)
		]
	];
}

FText SNiagaraStackHierarchySection::GetSectionNameAsText() const
{
	return Section.IsValid() ? Section->GetSectionNameAsText() : LOCTEXT("AllSection", "All");
}

FText SNiagaraStackHierarchySection::GetTooltipText() const
{
	return Section.IsValid() ? Section->GetTooltip() : FText::GetEmpty();
}

void SNiagaraStackScriptHierarchyRoot::Construct(const FArguments& InArgs, UNiagaraStackScriptHierarchyRoot* InModuleHierarchyRoot)
{
	ModuleHierarchyRoot = InModuleHierarchyRoot;

	ModuleHierarchyRoot->OnStructureChanged().AddSP(this, &SNiagaraStackScriptHierarchyRoot::HierarchyStructureChanged);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 2)
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemText")
			.ToolTipText_UObject(InModuleHierarchyRoot, &UNiagaraStackEntry::GetTooltipText)
			.Text_UObject(InModuleHierarchyRoot, &UNiagaraStackEntry::GetDisplayName)
			.IsEnabled_UObject(InModuleHierarchyRoot, &UNiagaraStackEntry::GetOwnerIsEnabled)
			.Visibility(this, &SNiagaraStackScriptHierarchyRoot::GetLabelVisibility)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4, 2)
		[
			SAssignNew(SectionSelectorBox, SWrapBox)
			.UseAllottedSize(true)
			.InnerSlotPadding(FVector2D(4, 4))
		]
	];

	ConstructSectionButtons();
}

EVisibility SNiagaraStackScriptHierarchyRoot::GetLabelVisibility() const
{
	return ModuleHierarchyRoot->GetShouldDisplayLabel() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SNiagaraStackScriptHierarchyRoot::IsSectionChecked(const UHierarchySection* NiagaraHierarchySection) const
{
	return ModuleHierarchyRoot->GetActiveHierarchySection() == NiagaraHierarchySection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackScriptHierarchyRoot::OnCheckStateChanged(ECheckBoxState CheckBoxState, const UHierarchySection* NiagaraHierarchySection)
{
	if(CheckBoxState == ECheckBoxState::Checked)
	{
		ModuleHierarchyRoot->SetActiveHierarchySection(NiagaraHierarchySection);
	}
}

void SNiagaraStackScriptHierarchyRoot::ConstructSectionButtons()
{
	SectionSelectorBox->ClearChildren();
	for (const UHierarchySection* Section : ModuleHierarchyRoot->GerHierarchySectionData())
	{
		SectionSelectorBox->AddSlot()
		[
			SNew(SNiagaraStackHierarchySection, Section)
			.IsChecked(this, &SNiagaraStackScriptHierarchyRoot::IsSectionChecked, Section)
			.OnCheckStateChanged(this, &SNiagaraStackScriptHierarchyRoot::OnCheckStateChanged, Section)
		];
	}

	SectionSelectorBox->AddSlot()
	[
		SNew(SNiagaraStackHierarchySection, nullptr)
		.OnCheckStateChanged(this, &SNiagaraStackScriptHierarchyRoot::OnCheckStateChanged, (const UHierarchySection*) nullptr)
		.IsChecked(this, &SNiagaraStackScriptHierarchyRoot::IsSectionChecked, (const UHierarchySection*) nullptr)
	];
}

void SNiagaraStackScriptHierarchyRoot::HierarchyStructureChanged(ENiagaraStructureChangedFlags StructureChangedFlags)
{
	ConstructSectionButtons();
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
