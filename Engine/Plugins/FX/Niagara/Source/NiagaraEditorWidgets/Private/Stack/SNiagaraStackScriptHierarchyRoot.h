// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewModels/Stack/NiagaraStackScriptHierarchyRoot.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SCheckBox.h"

class SNiagaraStackHierarchySection : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackHierarchySection) {}
	SLATE_EVENT(FOnCheckStateChanged, OnCheckStateChanged)
	SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)
SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const UHierarchySection* InSection);

private:
	FText GetSectionNameAsText() const;
	FText GetTooltipText() const;
private:
	TWeakObjectPtr<const UHierarchySection> Section;
};

class SNiagaraStackScriptHierarchyRoot : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackScriptHierarchyRoot) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackScriptHierarchyRoot* InModuleHierarchyRoot);

private:
	void ConstructSectionButtons();
	
	EVisibility GetLabelVisibility() const;

	ECheckBoxState IsSectionChecked(const UHierarchySection* NiagaraHierarchySection) const;
	void OnCheckStateChanged(ECheckBoxState CheckBoxState, const UHierarchySection* NiagaraHierarchySection);

	void HierarchyStructureChanged(ENiagaraStructureChangedFlags StructureChangedFlags);

private:
	TWeakObjectPtr<UNiagaraStackScriptHierarchyRoot> ModuleHierarchyRoot;

	TSharedPtr<SWrapBox> SectionSelectorBox;
};