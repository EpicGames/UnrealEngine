// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackTableRow.h"

#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "Styling/AppStyle.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraStackCommandContext.h"
#include "Stack/SNiagaraStackIndent.h"
#include "Stack/SNiagaraStackIssueIcon.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Framework/Application/SlateApplication.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "Styling/StyleColors.h"
#include "ScopedTransaction.h"
#include "NiagaraEmitterEditorData.h"
#include "SNiagaraStackNote.h"
#include "Framework/Commands/GenericCommands.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "ViewModels/Stack/NiagaraStackClipboardUtilities.h"
#include "ViewModels/Stack/NiagaraStackNote.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"

#define LOCTEXT_NAMESPACE "NiagaraStackTableRow"

SNiagaraStackTableRow::~SNiagaraStackTableRow()
{
	if(StackEntry && StackEntry->IsFinalized() == false)
	{
		StackEntry->OnPulseRequested().Unbind();
	}
}

void SNiagaraStackTableRow::Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel, UNiagaraStackEntry* InStackEntry, TSharedRef<FNiagaraStackCommandContext> InStackCommandContext, const TSharedRef<STreeView<UNiagaraStackEntry*>>& InOwnerTree)
{
	DefaultContentPadding = InArgs._ContentPadding;
	bIsCategoryIconHighlighted = InArgs._IsCategoryIconHighlighted;
	bShowExecutionCategoryIcon = InArgs._ShowExecutionCategoryIcon;
	NameColumnWidth = InArgs._NameColumnWidth;
	ValueColumnWidth = InArgs._ValueColumnWidth;
	NameColumnWidthChanged = InArgs._OnNameColumnWidthChanged;
	ValueColumnWidthChanged = InArgs._OnValueColumnWidthChanged;
	IssueIconVisibility = InArgs._IssueIconVisibility;
	StackViewModel = InStackViewModel;
	StackEntry = InStackEntry;
	StackCommandContext = InStackCommandContext;
	OwnerTree = InOwnerTree;

	Reset();

	ExpandedImage = FCoreStyle::Get().GetBrush("TreeArrow_Expanded");
	CollapsedImage = FCoreStyle::Get().GetBrush("TreeArrow_Collapsed");

	PulseAnimation.AddCurve(0.f, 0.5f, ECurveEaseFunction::CubicInOut);

	InStackEntry->OnPulseRequested().BindSP(this, &SNiagaraStackTableRow::Pulse);
	
	IndicatorColor = InArgs._IndicatorColor;

	ExecutionCategoryToolTipText = (InStackEntry->GetExecutionSubcategoryName() != NAME_None)
		? FText::Format(LOCTEXT("ExecutionCategoryToolTipFormat", "{0} - {1}"), FText::FromName(InStackEntry->GetExecutionCategoryName()), FText::FromName(InStackEntry->GetExecutionSubcategoryName()))
		: FText::FromName(InStackEntry->GetExecutionCategoryName());

	ConstructInternal(
		STableRow<UNiagaraStackEntry*>::FArguments()
			.Style(InArgs._Style)
			.OnDragDetected(InArgs._OnDragDetected)
			.OnDragLeave(InArgs._OnDragLeave)
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
		, OwnerTree.ToSharedRef());
}

void SNiagaraStackTableRow::Reset()
{
	ChildSlot[SNullWidget::NullWidget];
	ContentPadding = DefaultContentPadding;
	NameMinWidth.Reset();
	NameMaxWidth.Reset();
	NameVerticalAlignment = VAlign_Center;
	NameHorizontalAlignment = HAlign_Fill;
	ValueMinWidth.Reset();
	ValueMaxWidth.Reset();
	ValueVerticalAlignment = VAlign_Center;
	ValueHorizontalAlignment = HAlign_Fill;
	OnFillRowContextMenuHanders.Empty();
}

void SNiagaraStackTableRow::SetOverrideNameWidth(TOptional<float> InMinWidth, TOptional<float> InMaxWidth)
{
	NameMinWidth = InMinWidth;
	NameMaxWidth = InMaxWidth;
}

void SNiagaraStackTableRow::SetOverrideNameAlignment(EHorizontalAlignment InHAlign, EVerticalAlignment InVAlign)
{
	NameHorizontalAlignment = InHAlign;
	NameVerticalAlignment = InVAlign;
}

void SNiagaraStackTableRow::SetOverrideValueWidth(TOptional<float> InMinWidth, TOptional<float> InMaxWidth)
{
	ValueMinWidth = InMinWidth;
	ValueMaxWidth = InMaxWidth;
}

void SNiagaraStackTableRow::SetOverrideValueAlignment(EHorizontalAlignment InHAlign, EVerticalAlignment InVAlign)
{
	ValueHorizontalAlignment = InHAlign;
	ValueVerticalAlignment = InVAlign;
}

FMargin SNiagaraStackTableRow::GetContentPadding() const
{
	return ContentPadding;
}

void SNiagaraStackTableRow::SetContentPadding(FMargin InContentPadding)
{
	ContentPadding = InContentPadding;
}

void SNiagaraStackTableRow::SetNameAndValueContent(TSharedRef<SWidget> InNameWidget, TSharedPtr<SWidget> InValueWidget)
{
	FSlateColor IconColor = FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetColorNameForExecutionCategory(StackEntry->GetExecutionCategoryName()));
	if (bIsCategoryIconHighlighted)
	{
		IconColor = FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(StackEntry->GetExecutionCategoryName()));
	}

	FName IconName = FNiagaraStackEditorWidgetsUtilities::GetIconNameForExecutionSubcategory(StackEntry->GetExecutionSubcategoryName(), bIsCategoryIconHighlighted);
	const FSlateBrush* IconBrush = nullptr;

	if(IconName != NAME_None)
	{
		IconBrush = FNiagaraEditorWidgetsStyle::Get().GetBrush(IconName);
	}
	
	TSharedRef<SHorizontalBox> NameContent = SNew(SHorizontalBox)
	.Clipping(EWidgetClipping::OnDemand)
	// Indent
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(0)
	[
		SNew(SNiagaraStackIndent, StackEntry, ENiagaraStackIndentMode::Name)
	]
	// Expand button
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(0)
	[
		SNew(SButton)
		.ButtonStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.SimpleButton")
		.Visibility(this, &SNiagaraStackTableRow::GetExpanderVisibility)
		.OnClicked(this, &SNiagaraStackTableRow::ExpandButtonClicked)
		.ContentPadding(FMargin(1, 2, 1, 2))
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.Image(this, &SNiagaraStackTableRow::GetExpandButtonImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	]
	// Execution sub-category icon
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(FMargin(1, 1, 2, 1))
	.VAlign(EVerticalAlignment::VAlign_Center)
	.HAlign(EHorizontalAlignment::HAlign_Center)
	[
		SNew(SBox)
		.WidthOverride(FNiagaraEditorWidgetsStyle::Get().GetFloat("NiagaraEditor.Stack.IconHighlightedSize"))
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.ToolTipText(ExecutionCategoryToolTipText)
		.Visibility(this, &SNiagaraStackTableRow::GetExecutionCategoryIconVisibility)
		.IsEnabled_UObject(StackEntry, &UNiagaraStackEntry::GetIsEnabledAndOwnerIsEnabled)
		[
			SNew(SImage)
			.Visibility(this, &SNiagaraStackTableRow::GetExecutionCategoryIconVisibility)
			.Image(IconBrush ? IconBrush : FCoreStyle::Get().GetDefaultBrush())
			.ColorAndOpacity(IconColor)
		]
	]
	// Name content
	+ SHorizontalBox::Slot()
	.HAlign(NameHorizontalAlignment)
	.VAlign(NameVerticalAlignment)
	.Padding(FMargin(0, ContentPadding.Top, 0, ContentPadding.Bottom))
	[
		InNameWidget
	];
	
	TSharedPtr<SWidget> ChildContent;
	if (InValueWidget.IsValid())
	{
		ChildContent = SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(1.0f)
		.HitDetectionSplitterHandleSize(5.0f)

		+ SSplitter::Slot()
		.Value(NameColumnWidth)
		.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SNiagaraStackTableRow::OnNameColumnWidthChanged))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SBox)
				.Padding(FMargin(ContentPadding.Left, 0, 5, 0))
				.MinDesiredWidth(NameMinWidth.IsSet() ? NameMinWidth.GetValue() : FOptionalSize())
				.MaxDesiredWidth(NameMaxWidth.IsSet() ? NameMaxWidth.GetValue() : FOptionalSize())
				[
					NameContent
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SNiagaraStackInlineNote, StackEntry)
				.Visibility(this, &SNiagaraStackTableRow::GetInlineNoteVisibility)
			]
		]

		// Value
		+ SSplitter::Slot()
		.Value(ValueColumnWidth)
		.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SNiagaraStackTableRow::OnValueColumnWidthChanged))
		[
			SNew(SBox)
			.Padding(FMargin(4, ContentPadding.Top, ContentPadding.Right, ContentPadding.Bottom))
			.HAlign(ValueHorizontalAlignment)
			.VAlign(ValueVerticalAlignment)
			.MinDesiredWidth(ValueMinWidth.IsSet() ? ValueMinWidth.GetValue() : FOptionalSize())
			.MaxDesiredWidth(ValueMaxWidth.IsSet() ? ValueMaxWidth.GetValue() : FOptionalSize())
			[
				InValueWidget.ToSharedRef()
			]
		];
	}
	else
	{
		ChildContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SBox)
			.Padding(FMargin(ContentPadding.Left, 0, ContentPadding.Right, 0))
			.MinDesiredWidth(NameMinWidth.IsSet() ? NameMinWidth.GetValue() : FOptionalSize())
			.MaxDesiredWidth(NameMaxWidth.IsSet() ? NameMaxWidth.GetValue() : FOptionalSize())
			[
				NameContent
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SNiagaraStackInlineNote, StackEntry)
			.Visibility(this, &SNiagaraStackTableRow::GetInlineNoteVisibility)
		];
	}

	FName AccentColorName = FNiagaraStackEditorWidgetsUtilities::GetColorNameForExecutionCategory(StackEntry->GetExecutionCategoryName());
	TOptional<FSlateColor> AccentColor = (AccentColorName != NAME_None) ? FNiagaraEditorWidgetsStyle::Get().GetColor(AccentColorName) : FStyleColors::Transparent;

	TSharedRef<SOverlay> RowOverlay = SNew(SOverlay)
		.Visibility(this, &SNiagaraStackTableRow::GetRowVisibility)
		+ SOverlay::Slot()
		[
			// Row content
			SNew(SBorder)
			.BorderImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("Niagara.TableViewRowBorder"))
			.ToolTipText_UObject(StackEntry, &UNiagaraStackEntry::GetTooltipText)
			.Padding(FMargin(5, 0, 0, 1))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.0f)
				[
					ChildContent.ToSharedRef()
				]
				// Issue Icon
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3, 0)
				[
					SNew(SNiagaraStackIssueIcon, StackViewModel, StackEntry)
					.Visibility(IssueIconVisibility)
				]
			]
		];

	if (AccentColor.IsSet())
	{
		RowOverlay->AddSlot()
		.HAlign(HAlign_Left)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(AccentColor.GetValue())
			.Padding(0)
			[
				SNew(SBox)
				.WidthOverride(4)
			]
		];
	}

	if (IndicatorColor.IsSet())
	{
		RowOverlay->AddSlot()
		.HAlign(HAlign_Right)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(IndicatorColor.GetValue())
			.Padding(0)
			[
				SNew(SBox)
				.WidthOverride(2)
			]
		];
	}

	// Search result indicator.
	RowOverlay->AddSlot()
	[
		SNew(SBorder)
		.BorderImage(FNiagaraEditorWidgetsStyle::Get().GetBrush("NiagaraEditor.Stack.SearchResult"))
		.Visibility(this, &SNiagaraStackTableRow::GetSearchResultBorderVisibility)
	];

	ChildSlot
	[
		RowOverlay
	];
}

void SNiagaraStackTableRow::AddFillRowContextMenuHandler(FOnFillRowContextMenu FillRowContextMenuHandler)
{
	OnFillRowContextMenuHanders.Add(FillRowContextMenuHandler);
}

FReply SNiagaraStackTableRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(StackEntry))
		{
			RendererItem->OpenSourceCodeLocation();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SNiagaraStackTableRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TSharedPtr<ITypedTableView<UNiagaraStackEntry*>> OwnerTable = OwnerTablePtr.Pin();
		if (OwnerTable.IsValid())
		{
			if (OwnerTable->GetSelectedItems().Contains(StackEntry) == false)
			{
				OwnerTable->Private_ClearSelection();
				OwnerTable->Private_SetItemSelection(StackEntry, true, true);
				OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
			}
		}

		FMenuBuilder MenuBuilder(true, StackCommandContext->GetCommands());
		for (FOnFillRowContextMenu& OnFillRowContextMenuHandler : OnFillRowContextMenuHanders)
		{
			OnFillRowContextMenuHandler.ExecuteIfBound(MenuBuilder);
		}

		FNiagaraStackEditorWidgetsUtilities::AddStackEntryAssetContextMenuActions(MenuBuilder, *StackEntry);
		StackCommandContext->AddEditMenuItems(MenuBuilder);

		TArray<UNiagaraStackEntry*> EntriesToProcess;
		TArray<UNiagaraStackEntry*> NavigationEntries;
		StackViewModel->GetPathForEntry(StackEntry, EntriesToProcess);
		for (UNiagaraStackEntry* Parent : EntriesToProcess)
		{
			UNiagaraStackItemGroup* GroupParent = Cast<UNiagaraStackItemGroup>(Parent);
			UNiagaraStackItem* ItemParent = Cast<UNiagaraStackItem>(Parent);
			if (GroupParent != nullptr || ItemParent != nullptr)
			{
				MenuBuilder.BeginSection("StackRowNavigation", LOCTEXT("NavigationMenuSection", "Navigation"));
				{
					if (GroupParent != nullptr)
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("TopOfSection", "Top of Section"),
							FText::Format(LOCTEXT("NavigateToFormatted", "Navigate to {0}"), Parent->GetDisplayName()),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackTableRow::NavigateTo, Parent)));
					}
					if (ItemParent != nullptr)
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("TopOfModule", "Top of Module"),
							FText::Format(LOCTEXT("NavigateToFormatted", "Navigate to {0}"), Parent->GetDisplayName()),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackTableRow::NavigateTo, Parent)));
					}
				}
				MenuBuilder.EndSection();
			}
		}

		MenuBuilder.BeginSection("StackActions", LOCTEXT("StackActions", "Stack Actions"));
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ExpandAllItems", "Expand All"),
			LOCTEXT("ExpandAllItemsToolTip", "Expand all items under this header."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackTableRow::ExpandChildren)));
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CollapseAllItems", "Collapse All"),
			LOCTEXT("CollapseAllItemsToolTip", "Collapse all items under this header."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraStackTableRow::CollapseChildren)));

		MenuBuilder.EndSection();
		
		FNiagaraStackEditorWidgetsUtilities::AddStackNoteContextMenuActions(MenuBuilder, *StackEntry);
		
		if(StackEntry->GetEmitterViewModel().IsValid() && StackEntry->GetEmitterViewModel())
		{			
			if(StackEntry->SupportsSummaryView())
			{
				FHierarchyElementIdentity Identity = StackEntry->DetermineSummaryIdentity();
				if(Identity.IsValid())
				{
					FUIAction ShowHideSummaryViewAction(
			FExecuteAction::CreateSP(this, &SNiagaraStackTableRow::ToggleShowInSummaryView),
						FCanExecuteAction::CreateSP(this, &SNiagaraStackTableRow::CanToggleShowInSummary),
						FIsActionChecked::CreateSP(this, &SNiagaraStackTableRow::IsStackEntryInSummary));
			
					MenuBuilder.BeginSection("SummaryViewActions", LOCTEXT("SummaryViewActions", "Emitter Summary"));
						MenuBuilder.AddMenuEntry(
							TAttribute<FText>::CreateLambda([this]
							{
								bool bIsInSummaryView = StackEntry->IsInSummaryView();
								FText BaseText = LOCTEXT("SummaryViewShow", "{0} Emitter Summary");
								return FText::FormatOrdered(BaseText, bIsInSummaryView ? LOCTEXT("HideSummaryItem", "Remove from") : LOCTEXT("AddSummaryItem", "Add to"));
							}),
							TAttribute<FText>::CreateSP(this, &SNiagaraStackTableRow::GetToggleShowSummaryActionTooltip),
							FSlateIcon(),
							ShowHideSummaryViewAction,
							NAME_None, EUserInterfaceActionType::ToggleButton);

					FUIAction ShowInSummaryViewAction(
			FExecuteAction::CreateSP(this, &SNiagaraStackTableRow::NavigateToSummaryView),
						FCanExecuteAction::CreateSP(this, &SNiagaraStackTableRow::IsStackEntryInSummary));
					
						MenuBuilder.AddMenuEntry(
								LOCTEXT("ShowInSummaryEditorLabel", "Show in Summary Editor"),
								LOCTEXT("ShowInSummaryEditorTooltip", "Summon the Summary Editor and navigate to the selected item"),
								FSlateIcon(),
								ShowInSummaryViewAction,
								NAME_None, EUserInterfaceActionType::Button);
					MenuBuilder.EndSection();
				}						
			}
		}

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		return FReply::Handled();
	}
	
	return STableRow<UNiagaraStackEntry*>::OnMouseButtonUp(MyGeometry, MouseEvent);
}

int32 SNiagaraStackTableRow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = PaintSelection(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	const FSlateBrush* BrushResource = GetBorderImage();
		
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);

	if ( BrushResource && BrushResource->DrawAs != ESlateBrushDrawType::NoDrawType )
	{
		FLinearColor PulseAlpha = FLinearColor(1.f, 1.f, 1.f, 1.f);
		if(PulseAnimation.IsPlaying())
		{
			float Lerp = PulseAnimation.GetLerp();
			PulseAlpha = FMath::Lerp(FAppStyle::Get().GetSlateColor("Colors.Hover2").GetSpecifiedColor(), PulseAlpha, Lerp);
		}
		
		const bool bShowDisabledEffect = GetShowDisabledEffect();
		const ESlateDrawEffect DrawEffects = (bShowDisabledEffect && !bEnabled) ? ESlateDrawEffect::DisabledEffect : ESlateDrawEffect::None;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			BrushResource,
			DrawEffects,
			BrushResource->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint() * GetBorderBackgroundColor().GetColor(InWidgetStyle) * PulseAlpha
		);
	}

	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled );
	LayerId = PaintDropIndicator(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return LayerId;
}

void SNiagaraStackTableRow::Pulse()
{
	PulseAnimation.Play(AsShared());
}

void SetExpansionStateRecursive(UNiagaraStackEntry* StackEntry, bool bIsExpanded)
{
	if (StackEntry->GetCanExpand())
	{
		StackEntry->SetIsExpanded(bIsExpanded);
	}

	TArray<UNiagaraStackEntry*> Children;
	StackEntry->GetUnfilteredChildren(Children);
	for (UNiagaraStackEntry* Child : Children)
	{
		SetExpansionStateRecursive(Child, bIsExpanded);
	}
}

void SNiagaraStackTableRow::CollapseChildren()
{
	bool bIsExpanded = false;
	SetExpansionStateRecursive(StackEntry, bIsExpanded);
}

void SNiagaraStackTableRow::ExpandChildren()
{
	bool bIsExpanded = true;
	SetExpansionStateRecursive(StackEntry, bIsExpanded);
}

EVisibility SNiagaraStackTableRow::GetRowVisibility() const
{
	return StackEntry->GetShouldShowInStack()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SNiagaraStackTableRow::GetExecutionCategoryIconVisibility() const
{
	return bShowExecutionCategoryIcon && (StackEntry->GetExecutionSubcategoryName() != NAME_None)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SNiagaraStackTableRow::GetExpanderVisibility() const
{
	if (StackEntry->GetCanExpand())
	{
		TArray<UNiagaraStackEntry*> Children;
		StackEntry->GetFilteredChildren(Children);
		return Children.Num() > 0
			? EVisibility::Visible
			: EVisibility::Hidden;
	}
	else
	{
		return StackEntry->KeepExpanderIndentation() ? EVisibility::Hidden : EVisibility::Collapsed;
	}
}

EVisibility SNiagaraStackTableRow::GetInlineNoteVisibility() const
{
	if(StackEntry->GetStackNote() == nullptr || StackEntry->GetStackNote()->GetShouldShowInStack())
	{
		return EVisibility::Collapsed;
	}
		
	return StackEntry->GetStackNote()->GetTargetStackNoteData().GetValue().bInlineNote == true ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SNiagaraStackTableRow::ExpandButtonClicked()
{
	const bool bWillBeExpanded = !StackEntry->GetIsExpanded();
	// Recurse the expansion if "shift" is being pressed
	const FModifierKeysState ModKeyState = FSlateApplication::Get().GetModifierKeys();
	if (ModKeyState.IsShiftDown())
	{
		StackEntry->SetIsExpanded_Recursive(bWillBeExpanded);
	}
	else
	{
		StackEntry->SetIsExpanded(bWillBeExpanded);
	}
	return FReply::Handled();
}

const FSlateBrush* SNiagaraStackTableRow::GetExpandButtonImage() const
{
	return StackEntry->GetIsExpanded() ? ExpandedImage : CollapsedImage;
}

void SNiagaraStackTableRow::OnNameColumnWidthChanged(float Width)
{
	NameColumnWidthChanged.ExecuteIfBound(Width);
}

void SNiagaraStackTableRow::OnValueColumnWidthChanged(float Width)
{
	ValueColumnWidthChanged.ExecuteIfBound(Width);
}

EVisibility SNiagaraStackTableRow::GetSearchResultBorderVisibility() const
{
	return StackViewModel->GetCurrentFocusedEntry() == StackEntry ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

FSlateColor SNiagaraStackTableRow::GetInnerBackgroundColor() const
{
	FSlateColor Color(FLinearColor(0.f, 0.f, 0.f, 1.f));
	if (PulseAnimation.IsPlaying())
	{
		float Lerp = PulseAnimation.GetLerp();
		return FMath::Lerp(FAppStyle::Get().GetSlateColor("Colors.Hover2").GetSpecifiedColor(), Color.GetSpecifiedColor(), Lerp);
	}

	return Color;
}

void SNiagaraStackTableRow::NavigateTo(UNiagaraStackEntry* Item)
{
	OwnerTree->RequestNavigateToItem(Item, 0);
}

void SNiagaraStackTableRow::ToggleShowInSummaryView() const
{
	if(StackEntry->GetEmitterViewModel().IsValid())
	{
		FHierarchyElementIdentity Identity = StackEntry->DetermineSummaryIdentity();
		if(Identity.IsValid())
		{
			bool bItemExistsInHierarchy = StackEntry->GetEmitterViewModel()->GetSummaryHierarchyViewModel()->GetHierarchyRootViewModel()->FindViewModelForChild(Identity, true) != nullptr;
			if(bItemExistsInHierarchy)
			{
				StackEntry->GetEmitterViewModel()->GetSummaryHierarchyViewModel()->DeleteElementWithIdentity(Identity);
			}
			else
			{
				UClass* HierarchyClass = DetermineHierarchyClassForSummaryView();
				check(HierarchyClass);
				UHierarchyElement* Element = StackEntry->GetEmitterViewModel()->GetSummaryHierarchyViewModel()->GetHierarchyRoot()->AddChild<UHierarchyElement>(HierarchyClass);
				Element->SetIdentity(Identity);
			}
		}
	}
}

bool SNiagaraStackTableRow::IsStackEntryInSummary() const
{
	return StackEntry->IsInSummaryView();
}

bool SNiagaraStackTableRow::CanToggleShowInSummary() const
{	
	return StackEntry->IsAnyParentInSummaryView() == false && StackEntry->IsAnyChildInSummaryView() == false && StackEntry->ExistsInParentEmitterSummary() == false;
}

FText SNiagaraStackTableRow::GetToggleShowSummaryActionTooltip() const
{
	if(CanToggleShowInSummary())
	{
		return LOCTEXT("CanToggleSummaryStatus", "You can add or remove this item to or from summary view.\nTo view the emitter summary, please collapse the emitter node.");
	}

	FString ResultMessage;
	FText CantToggleMessage = LOCTEXT("CantToggleSummaryViewBaseMessage", "You can not add or remove this item due to:\n");
	FHierarchyElementIdentity Identity = StackEntry->DetermineSummaryIdentity();
	
	bool bExistsInParentEmitter = StackEntry->ExistsInParentEmitterSummary();
	bool bIsAnyParentInSummaryView = StackEntry->IsAnyParentInSummaryView();
	bool bIsAnyChildInSummaryView = StackEntry->IsAnyChildInSummaryView(true);
	
	FText StackEntryAddedInParentEmitter = LOCTEXT("CantToggleSummaryDueToAddedInParentEmitter", "This item was added in the parent emitter and can't me modified.\n");
	FText ParentStackEntryIsInSummaryText = LOCTEXT("CantToggleSummaryDueToParentStackEntryInSummary", "One of the parent items of this item was already added to the summary view, which prevents you from adding this separately.\nIf you wish to add or remove it, remove the parent item first.\n");
	FText ChildStackEntryIsInSummaryText = LOCTEXT("CantToggleSummaryDueToChildStackEntryInSummary", "One of the child items of this item was already added to the summary view, which prevents you from adding this separately.\nIf you wish to add or remove it, remove the child item first.\n");

	ResultMessage.Append(CantToggleMessage.ToString());
	if(bExistsInParentEmitter)
	{
		ResultMessage.Append(StackEntryAddedInParentEmitter.ToString());
	}

	if(bIsAnyParentInSummaryView)
	{
		ResultMessage.Append(ParentStackEntryIsInSummaryText.ToString());
	}

	if(bIsAnyChildInSummaryView)
	{
		ResultMessage.Append(ChildStackEntryIsInSummaryText.ToString());
	}

	ResultMessage.RemoveFromEnd("\n");
	return FText::FromString(ResultMessage);
}

bool SNiagaraStackTableRow::DoesItemExistInParentSummary() const
{
	return StackEntry->ExistsInParentEmitterSummary();
}

void SNiagaraStackTableRow::NavigateToSummaryView() const
{
	StackEntry->GetSystemViewModel()->FocusTab(FName("NiagaraSystemEditor_SummaryViewHierarchyEditor"), true);
	StackEntry->GetEmitterViewModel()->GetSummaryHierarchyViewModel()->SetActiveHierarchySection(nullptr);
	StackEntry->GetEmitterViewModel()->GetSummaryHierarchyViewModel()->NavigateToElementInHierarchy(StackEntry->DetermineSummaryIdentity());
}

TSubclassOf<UHierarchyElement> SNiagaraStackTableRow::DetermineHierarchyClassForSummaryView() const
{
	if(UNiagaraStackFunctionInput* FunctionInput = Cast<UNiagaraStackFunctionInput>(StackEntry))
	{
		if(FunctionInput->GetInputFunctionCallNode().IsA<UNiagaraNodeAssignment>())
		{
			return UNiagaraHierarchyAssignmentInput::StaticClass();
		}
		
		return UNiagaraHierarchyModuleInput::StaticClass();
	}
	else if(StackEntry->IsA<UNiagaraStackModuleItem>())
	{
		return UNiagaraHierarchyModule::StaticClass();
	}
	else if(StackEntry->IsA<UNiagaraStackRendererItem>())
	{
		return UNiagaraHierarchyRenderer::StaticClass();
	}
	else if(StackEntry->IsA<UNiagaraStackPropertyRow>())
	{
		return UNiagaraHierarchyObjectProperty::StaticClass();
	}
	else if(StackEntry->IsA<UNiagaraStackSimulationStageGroup>())
	{
		return UNiagaraHierarchySimStage::StaticClass();
	}
	else if(StackEntry->IsA<UNiagaraStackSimulationStagePropertiesItem>())
	{
		return UNiagaraHierarchySimStageProperties::StaticClass();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
