// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Views/STreeView.h"
#include "Styling/SlateTypes.h"
#include "Layout/Visibility.h"
#include "Animation/CurveSequence.h"

class UHierarchyElement;
class UNiagaraStackViewModel;
class UNiagaraStackEntry;
class FNiagaraStackCommandContext;

class SNiagaraStackTableRow: public STableRow<UNiagaraStackEntry*>
{
public:
	DECLARE_DELEGATE_OneParam(FOnColumnWidthChanged, float);
	DECLARE_DELEGATE_OneParam(FOnFillRowContextMenu, FMenuBuilder&);

public:
	SLATE_BEGIN_ARGS(SNiagaraStackTableRow)
		: _ContentPadding(FMargin(2, 0, 2, 0))
		, _IsCategoryIconHighlighted(false)
		, _ShowExecutionCategoryIcon(false)
	{}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
		SLATE_ARGUMENT(FMargin, ContentPadding)
		SLATE_ARGUMENT(TOptional<FSlateColor>, IndicatorColor)
		SLATE_ARGUMENT(bool, IsCategoryIconHighlighted)
		SLATE_ARGUMENT(bool, ShowExecutionCategoryIcon)
		SLATE_ATTRIBUTE(float, NameColumnWidth)
		SLATE_ATTRIBUTE(float, ValueColumnWidth)
		SLATE_ATTRIBUTE(EVisibility, IssueIconVisibility)
		SLATE_EVENT(FOnColumnWidthChanged, OnNameColumnWidthChanged)
		SLATE_EVENT(FOnColumnWidthChanged, OnValueColumnWidthChanged)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_EVENT(FOnTableRowDragLeave, OnDragLeave)
		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop);
	SLATE_END_ARGS();

	virtual ~SNiagaraStackTableRow() override;
	
	void Construct(const FArguments& InArgs, UNiagaraStackViewModel* InStackViewModel, UNiagaraStackEntry* InStackEntry, TSharedRef<FNiagaraStackCommandContext> InStackCommandContext, const TSharedRef<STreeView<UNiagaraStackEntry*>>& InOwnerTree);

	void Reset();

	void SetOverrideNameWidth(TOptional<float> InMinWidth, TOptional<float> InMaxWidth);

	void SetOverrideNameAlignment(EHorizontalAlignment InHAlign, EVerticalAlignment InVAlign);

	void SetOverrideValueWidth(TOptional<float> InMinWidth, TOptional<float> InMaxWidth);

	void SetOverrideValueAlignment(EHorizontalAlignment InHAlign, EVerticalAlignment InVAlign);

	FMargin GetContentPadding() const;

	void SetContentPadding(FMargin InContentPadding);
	
	void SetNameAndValueContent(TSharedRef<SWidget> InNameWidget, TSharedPtr<SWidget> InValueWidget);

	void AddFillRowContextMenuHandler(FOnFillRowContextMenu FillRowContextMenuHandler);

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
private:
	void Pulse();

	void CollapseChildren();
	void ExpandChildren();
	
	EVisibility GetRowVisibility() const;

	EVisibility GetExecutionCategoryIconVisibility() const;

	EVisibility GetExpanderVisibility() const;

	EVisibility GetInlineNoteVisibility() const;

	FReply ExpandButtonClicked();

	const FSlateBrush* GetExpandButtonImage() const;

	void OnNameColumnWidthChanged(float Width);

	void OnValueColumnWidthChanged(float Width);

	EVisibility GetSearchResultBorderVisibility() const;
	
	FSlateColor GetInnerBackgroundColor() const;

	void NavigateTo(UNiagaraStackEntry* Item);

	void ToggleShowInSummaryView() const;
	bool IsStackEntryInSummary() const;
	bool CanToggleShowInSummary() const;
	FText GetToggleShowSummaryActionTooltip() const;
	bool DoesItemExistInParentSummary() const;

	void NavigateToSummaryView() const;
	
	TSubclassOf<UHierarchyElement> DetermineHierarchyClassForSummaryView() const;
private:
	UNiagaraStackViewModel* StackViewModel;
	UNiagaraStackEntry* StackEntry;
	TSharedPtr<STreeView<UNiagaraStackEntry*>> OwnerTree;

	TAttribute<float> NameColumnWidth;
	TAttribute<float> ValueColumnWidth;
	FOnColumnWidthChanged NameColumnWidthChanged;
	FOnColumnWidthChanged ValueColumnWidthChanged;

	TAttribute<EVisibility> IssueIconVisibility;

	const FSlateBrush* ExpandedImage;
	const FSlateBrush* CollapsedImage;

	TOptional<FSlateColor> IndicatorColor;

	FText ExecutionCategoryToolTipText;

	FMargin DefaultContentPadding;
	FMargin ContentPadding;

	EHorizontalAlignment NameHorizontalAlignment;
	EVerticalAlignment NameVerticalAlignment;

	TOptional<float> NameMinWidth;
	TOptional<float> NameMaxWidth;

	EHorizontalAlignment ValueHorizontalAlignment;
	EVerticalAlignment ValueVerticalAlignment;

	TOptional<float> ValueMinWidth;
	TOptional<float> ValueMaxWidth;

	bool bIsCategoryIconHighlighted;
	bool bShowExecutionCategoryIcon;

	TArray<FOnFillRowContextMenu> OnFillRowContextMenuHanders;

	TSharedPtr<FNiagaraStackCommandContext> StackCommandContext;

	FCurveSequence PulseAnimation;
};