// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraStackValueCollection.h"

#include "NiagaraEditorWidgetsStyle.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "NiagaraStack"

void SNiagaraStackValueCollection::Construct(const FArguments& InArgs, UNiagaraStackValueCollection* PropertyCollectionBase)
{
	PropertyCollection = PropertyCollectionBase;
	PropertyCollection->OnStructureChanged().AddSP(this, &SNiagaraStackValueCollection::InputCollectionStructureChanged);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 2)
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.Stack.ItemText")
			.ToolTipText_UObject(PropertyCollection, &UNiagaraStackEntry::GetTooltipText)
			.Text_UObject(PropertyCollection, &UNiagaraStackEntry::GetDisplayName)
			.IsEnabled_UObject(PropertyCollection, &UNiagaraStackEntry::GetOwnerIsEnabled)
			.Visibility(this, &SNiagaraStackValueCollection::GetLabelVisibility)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4, 2)
		[
			SAssignNew(SectionSelectorBox, SWrapBox)
			.UseAllottedSize(true)
			.InnerSlotPadding(FVector2D(4, 4))
		]
	];

	ConstructSectionButtons();
}

EVisibility SNiagaraStackValueCollection::GetLabelVisibility() const
{
	return PropertyCollection->GetShouldDisplayLabel() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SNiagaraStackValueCollection::ConstructSectionButtons()
{
	SectionSelectorBox->ClearChildren();
	for (FText Section : PropertyCollection->GetSections())
	{
		SectionSelectorBox->AddSlot()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.OnCheckStateChanged(this, &SNiagaraStackValueCollection::OnSectionChecked, Section)
			.IsChecked(this, &SNiagaraStackValueCollection::GetSectionCheckState, Section)
			.ToolTipText(this, &SNiagaraStackValueCollection::GetTooltipText, Section)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(Section)
			]
		];
	}
}

void SNiagaraStackValueCollection::InputCollectionStructureChanged(ENiagaraStructureChangedFlags StructureChangedFlags)
{
	ConstructSectionButtons();
}

ECheckBoxState SNiagaraStackValueCollection::GetSectionCheckState(FText Section) const
{
	return Section.EqualTo(PropertyCollection->GetActiveSection()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraStackValueCollection::OnSectionChecked(ECheckBoxState CheckState, FText Section)
{
	if (CheckState == ECheckBoxState::Checked)
	{
		PropertyCollection->SetActiveSection(Section);
	}
}

FText SNiagaraStackValueCollection::GetTooltipText(FText Section) const
{
	return PropertyCollection->GetTooltipForSection(Section.ToString());
}

#undef LOCTEXT_NAMESPACE
