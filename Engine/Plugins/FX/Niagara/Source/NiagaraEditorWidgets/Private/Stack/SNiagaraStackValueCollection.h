// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorCommon.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SExpanderArrow.h"

class SWrapBox;
enum class ECheckBoxState : uint8;

class SNiagaraStackValueCollection : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStackValueCollection) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraStackValueCollection* PropertyCollectionBase);

private:
	EVisibility GetLabelVisibility() const;

	void ConstructSectionButtons();

	void InputCollectionStructureChanged(ENiagaraStructureChangedFlags StructureChangedFlags);

	ECheckBoxState GetSectionCheckState(FText Section) const;

	void OnSectionChecked(ECheckBoxState CheckState, FText Section);

	FText GetTooltipText(FText Section) const;

private:
	UNiagaraStackValueCollection* PropertyCollection = nullptr;

	TSharedPtr<SWrapBox> SectionSelectorBox;
};
