// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stack/SNiagaraSummaryViewToggle.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "Stack/SNiagaraStackInheritanceIcon.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"

#define LOCTEXT_NAMESPACE "SNiagaraSummaryViewToggle"

void SNiagaraSummaryViewToggle::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	EmitterHandleViewModel = InEmitterHandleViewModel;

	if(EmitterHandleViewModel.IsValid() == false)
	{
		return;
	}
	
	ChildSlot
	[
		SNew(SButton)
		.OnClicked(this, &SNiagaraSummaryViewToggle::ToggleShowSummaryForUI)
		.ToolTipText(this, &SNiagaraSummaryViewToggle::GetSummaryToggleTooltip)
		.ContentPadding(1.f)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		[
			SNew(SImage)
			.Image(this, &SNiagaraSummaryViewToggle::GetSummaryToggleImage)
		]
	];
}

void SNiagaraSummaryViewToggle::ToggleShowSummary() const
{
	EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEditorData().ToggleShowSummaryView();
}

FReply SNiagaraSummaryViewToggle::ToggleShowSummaryForUI() const
{
	ToggleShowSummary();
	return FReply::Handled();
}

const FSlateBrush* SNiagaraSummaryViewToggle::GetSummaryToggleImage() const
{
	return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEditorData().ShouldShowSummaryView() ? FAppStyle::GetBrush("TreeArrow_Collapsed") : FAppStyle::GetBrush("TreeArrow_Expanded");  
	// return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEditorData().ShouldShowSummaryView() ? FAppStyle::GetBrush("DetailsView.PulldownArrow.Down") : FAppStyle::GetBrush("DetailsView.PulldownArrow.Up");  
}

FText SNiagaraSummaryViewToggle::GetSummaryToggleTooltip() const
{
	return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEditorData().ShouldShowSummaryView()
		? LOCTEXT("ToggleSummaryViewButtonTooltip_IsShowingSummaryView", "This emitter is displayed in summary view. To see the full emitter, return to the full view first.")
		: LOCTEXT("ToggleSummaryViewButtonTooltip_IsShowingDefaultView", "This emitter is displayed in default view. Collapse the node to see the summary view instead.");
			
		
}

#undef LOCTEXT_NAMESPACE
