// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"

class SNiagaraSummaryViewToggle: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSummaryViewToggle) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);

private:
	void ToggleShowSummary() const;
	FReply ToggleShowSummaryForUI() const;

	const FSlateBrush* GetSummaryToggleImage() const;
	FText GetSummaryToggleTooltip() const;

private:
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;
};
