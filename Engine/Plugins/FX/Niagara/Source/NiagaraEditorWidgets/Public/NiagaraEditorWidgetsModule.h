// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorModule.h"
#include "Modules/ModuleInterface.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/ObjectKey.h"
#include "Templates/SharedPointer.h"

class UObject;
class FCurveEditor;
struct FCurveEditorTreeItemID;
class SCurveEditorTree;

class FNiagaraStackCurveEditorOptions
{
public:
	FNiagaraStackCurveEditorOptions();

	bool GetNeedsInitializeView() const;
	void InitializeView(float InViewMinInput, float InViewMaxInput, float InViewMinOutput, float InViewMaxOutput);

	float GetViewMinInput() const;
	float GetViewMaxInput() const;
	void SetInputViewRange(float InViewMinInput, float InViewMaxInput);

	float GetViewMinOutput() const;
	float GetViewMaxOutput() const;
	void SetOutputViewRange(float InViewMinOutput, float InViewMaxOutput);

	bool GetIsGradientVisible() const;
	void SetIsGradientVisible(bool bInIsGradientVisible);

	float GetTimelineLength() const;

	float GetHeight() const;
	void SetHeight(float InHeight);

private:
	float ViewMinInput;
	float ViewMaxInput;
	float ViewMinOutput;
	float ViewMaxOutput;
	bool bIsGradientVisible;
	bool bNeedsInitializeView;
	float Height;
};

/** A module containing widgets for editing niagara data. */
class FNiagaraEditorWidgetsModule : public IModuleInterface
{
private:
	class FNiagaraEditorWidgetProvider : public INiagaraEditorWidgetProvider
	{
	public:
		virtual TSharedRef<SWidget> CreateStackView(UNiagaraStackViewModel& StackViewModel) const override;
		virtual TSharedRef<SWidget> CreateSystemOverview(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, const FAssetData& EditedAsset) const override;
		virtual TSharedRef<SWidget> CreateStackIssueIcon(UNiagaraStackViewModel& StackViewModel, UNiagaraStackEntry& StackEntry) const override;
		virtual TSharedRef<SWidget> CreateScriptScratchPadManager(UNiagaraScratchPadViewModel& ScriptScratchPadViewModel) const override;
		virtual TSharedRef<SWidget> CreateCurveOverview(TSharedRef<FNiagaraSystemViewModel> SystemViewModel) const override;
		virtual TSharedRef<SWidget> CreateCurveKeySelector(TSharedPtr<FCurveEditor> InCurveEditor, const TArray<FCurveEditorTreeItemID>& InCurveTreeItemIds, TSharedPtr<SCurveEditorTree> InCurveEditorTree) const override;
		virtual TSharedRef<SWidget> CreateCurveThumbnail(const FRichCurve& Curve) const override;
		virtual FLinearColor GetColorForExecutionCategory(FName ExecutionCategory) const override;
	};

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FNiagaraEditorWidgetsModule& Get();

	TSharedRef<FNiagaraStackCurveEditorOptions> GetOrCreateStackCurveEditorOptionsForObject(UObject* Object, float DefaultHeight);

private:
	void ReinitializeStyle();

private:
	TMap<FObjectKey, TSharedRef<FNiagaraStackCurveEditorOptions>> ObjectToStackCurveEditorOptionsMap;

	TSharedPtr<FNiagaraEditorWidgetProvider> WidgetProvider;

	IConsoleCommand* ReinitializeStyleCommand;
};