// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

class FNiagaraLWCTileShiftPositionsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraLWCTileShiftPositionsCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraLWCTileShiftPositionsCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;
	static constexpr uint32 MaxPositions = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARASHADER_API)
		SHADER_PARAMETER_UAV(RWBuffer<float>,	FloatBuffer)
		SHADER_PARAMETER(uint32,				FloatBufferStride)
		SHADER_PARAMETER(uint32,				NumInstances)

		SHADER_PARAMETER_SRV(Buffer<int>,		CountBuffer)
		SHADER_PARAMETER(uint32,				CountBufferOffset)

		SHADER_PARAMETER(uint32,				NumPositions)
		SHADER_PARAMETER_SCALAR_ARRAY(uint32,	PositionComponentOffsets, [MaxPositions])
		SHADER_PARAMETER(FVector3f,				TileShift)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	NIAGARASHADER_API static void Execute(FRHIComputeCommandList& RHICmdList, const FParameters& Parameters);
};
