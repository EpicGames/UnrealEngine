// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraStatelessSimulationShader.h"

#include "NiagaraStatelessModuleShaderParameters.h"

#include "RenderGraphFwd.h"
#include "ShaderParameterStruct.h"

namespace NiagaraStateless
{
	#define STATELESS_MODULE_COMBINE2(A, B) A##B
	#define STATELESS_MODULE_COMBINE(A, B) STATELESS_MODULE_COMBINE2(A, B)
	#define ADD_STATELESS_MODULE(NAME) SHADER_PARAMETER_STRUCT_INCLUDE(NAME, STATELESS_MODULE_COMBINE(ModuleParam,__LINE__))

	class FSimulationShaderDefaultCS : public FSimulationShader
	{
	public:
		DECLARE_EXPORTED_GLOBAL_SHADER(FSimulationShaderDefaultCS, NIAGARASHADER_API);
		SHADER_USE_PARAMETER_STRUCT(FSimulationShaderDefaultCS, FSimulationShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARASHADER_API)
			SHADER_PARAMETER_STRUCT_INCLUDE(FCommonShaderParameters, CommonParameters)
			// Initialize Modules
			ADD_STATELESS_MODULE(FInitializeParticleModule_ShaderParameters)
			ADD_STATELESS_MODULE(FInitialMeshOrientationModule_ShaderParameters)
			ADD_STATELESS_MODULE(FShapeLocationModule_ShaderParameters)
			// Pre Solve Modules
			ADD_STATELESS_MODULE(FSolveVelocitiesAndForcesModule_ShaderParameters)
			// Post Solve Modules
			ADD_STATELESS_MODULE(FCameraOffsetModule_ShaderParameters)
			ADD_STATELESS_MODULE(FDynamicMaterialParametersModule_ShaderParameters)
			ADD_STATELESS_MODULE(FMeshIndexModule_ShaderParameters)
			ADD_STATELESS_MODULE(FMeshRotationRateModule_ShaderParameters)
			ADD_STATELESS_MODULE(FScaleColorModule_ShaderParameters)
			ADD_STATELESS_MODULE(FScaleRibbonWidthModule_ShaderParameters)
			ADD_STATELESS_MODULE(FScaleSpriteSizeModule_ShaderParameters)
			ADD_STATELESS_MODULE(FScaleSpriteSizeBySpeedModule_ShaderParameters)
			ADD_STATELESS_MODULE(FScaleMeshSizeModule_ShaderParameters)
			ADD_STATELESS_MODULE(FScaleMeshSizeBySpeedModule_ShaderParameters)
			ADD_STATELESS_MODULE(FSpriteFacingAndAlignmentModule_ShaderParameters)
			ADD_STATELESS_MODULE(FSpriteRotationRateModule_ShaderParameters)
			ADD_STATELESS_MODULE(FSubUVAnimationModule_ShaderParameters)

			SHADER_PARAMETER(int, Permutation_UniqueIDComponent)
			SHADER_PARAMETER(int, Permutation_MaterialRandomComponent)
			SHADER_PARAMETER(int, Permutation_PositionComponent)
			SHADER_PARAMETER(int, Permutation_CameraOffsetComponent)
			SHADER_PARAMETER(int, Permutation_ColorComponent)
			SHADER_PARAMETER(int, Permutation_DynamicMaterialParameterComponent)
			SHADER_PARAMETER(int, Permutation_DynamicMaterialParameter1Component)
			SHADER_PARAMETER(int, Permutation_DynamicMaterialParameter2Component)
			SHADER_PARAMETER(int, Permutation_DynamicMaterialParameter3Component)
			SHADER_PARAMETER(int, Permutation_MeshIndexComponent)
			SHADER_PARAMETER(int, Permutation_MeshOrientationComponent)
			SHADER_PARAMETER(int, Permutation_RibbonWidthComponent)
			SHADER_PARAMETER(int, Permutation_ScaleComponent)
			SHADER_PARAMETER(int, Permutation_SpriteSizeComponent)
			SHADER_PARAMETER(int, Permutation_SpriteFacingComponent)
			SHADER_PARAMETER(int, Permutation_SpriteAlignmentComponent)
			SHADER_PARAMETER(int, Permutation_SpriteRotationComponent)
			SHADER_PARAMETER(int, Permutation_SubImageIndexComponent)
			SHADER_PARAMETER(int, Permutation_VelocityComponent)
			SHADER_PARAMETER(int, Permutation_PreviousPositionComponent)
			SHADER_PARAMETER(int, Permutation_PreviousCameraOffsetComponent)
			SHADER_PARAMETER(int, Permutation_PreviousMeshOrientationComponent)
			SHADER_PARAMETER(int, Permutation_PreviousRibbonWidthComponent)
			SHADER_PARAMETER(int, Permutation_PreviousScaleComponent)
			SHADER_PARAMETER(int, Permutation_PreviousSpriteSizeComponent)
			SHADER_PARAMETER(int, Permutation_PreviousSpriteFacingComponent)
			SHADER_PARAMETER(int, Permutation_PreviousSpriteAlignmentComponent)
			SHADER_PARAMETER(int, Permutation_PreviousSpriteRotationComponent)
			SHADER_PARAMETER(int, Permutation_PreviousVelocityComponent)
		END_SHADER_PARAMETER_STRUCT()
	};

	#undef ADD_STATELESS_MODULE
	#undef STATELESS_MODULE_COMBINE
	#undef STATELESS_MODULE_COMBINE2
}
