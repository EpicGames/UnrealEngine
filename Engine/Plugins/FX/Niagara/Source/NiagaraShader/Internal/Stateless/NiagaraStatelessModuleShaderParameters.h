// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "ShaderParameterStruct.h"
#include "NiagaraStatelessBuiltDistribution.h"

namespace NiagaraStateless
{
	BEGIN_SHADER_PARAMETER_STRUCT(FInitializeParticleModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(uint32,									InitializeParticle_ModuleFlags)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	InitializeParticle_InitialPosition)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	InitializeParticle_InitialColor)

		SHADER_PARAMETER(FVector2f, InitializeParticle_SpriteSizeScale)			// Unset / Uniform / Random Uniform / Non-Uniform / Random Non-Uniform
		SHADER_PARAMETER(FVector2f, InitializeParticle_SpriteSizeBias)
		SHADER_PARAMETER(float,		InitializeParticle_SpriteRotationScale)				// Unset / Random / Direct Set Deg / Direct Set Normalized
		SHADER_PARAMETER(float,		InitializeParticle_SpriteRotationBias)
		//SHADER_PARAMETER(FVector3f, InitializeParticle_SpriteUVMode)			// Unset / Random / Random X / Random Y / Random XY / Direct Set

		SHADER_PARAMETER(FVector3f,	InitializeParticle_MeshScaleScale)			// Unset / Uniform / Random Uniform / Non-Uniform / Random Non-Uniform
		SHADER_PARAMETER(FVector3f,	InitializeParticle_MeshScaleBias)

		SHADER_PARAMETER(float,		InitializeParticle_RibbonWidthScale)		// Unset / Direct Set
		SHADER_PARAMETER(float,		InitializeParticle_RibbonWidthBias)
		//SHADER_PARAMETER(FVector3f, InitializeParticle_RibbonFacingVector)		// Unset / Direct Set
		//SHADER_PARAMETER(FVector3f, InitializeParticle_RibbonTwist)				// Unset / Direct Set
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FInitialMeshOrientationModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FVector3f, InitialMeshOrientation_RotationScale)
		SHADER_PARAMETER(FVector3f, InitialMeshOrientation_RotationBias)
	END_SHADER_PARAMETER_STRUCT()
		

	BEGIN_SHADER_PARAMETER_STRUCT(FRotateAroundPointModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(float,		RotateAroundPoint_RateScale)
		SHADER_PARAMETER(float,		RotateAroundPoint_RateBias)
		SHADER_PARAMETER(float,		RotateAroundPoint_RadiusScale)
		SHADER_PARAMETER(float,		RotateAroundPoint_RadiusBias)
		SHADER_PARAMETER(float,		RotateAroundPoint_InitialPhaseScale)
		SHADER_PARAMETER(float,		RotateAroundPoint_InitialPhaseBias)
		SHADER_PARAMETER(FVector3f,	RotateAroundPoint_CenterScale)
		SHADER_PARAMETER(FVector3f,	RotateAroundPoint_CenterBias)
		SHADER_PARAMETER(FVector3f, RotateAroundPoint_RotationAxisScale)
		SHADER_PARAMETER(FVector3f, RotateAroundPoint_RotationAxisBias)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FDynamicMaterialParametersModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(uint32,									DynamicMaterialParameters_ChannelMask)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter0X)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter0Y)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter0Z)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter0W)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter1X)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter1Y)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter1Z)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter1W)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter2X)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter2Y)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter2Z)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter2W)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter3X)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter3Y)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter3Z)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	DynamicMaterialParameters_Parameter3W)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FCameraOffsetModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	CameraOffset_Distribution)
		SHADER_PARAMETER(float,										CameraOffset_Scale)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FScaleColorModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	ScaleColor_Distribution)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FScaleMeshSizeModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	ScaleMeshSize_Distribution)
		SHADER_PARAMETER(FVector3f,									ScaleMeshSize_CurveScale)
		SHADER_PARAMETER(int32,										ScaleMeshSize_CurveScaleOffset)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FScaleMeshSizeBySpeedModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FVector3f,									ScaleMeshSizeBySpeed_ScaleFactorBias)
		SHADER_PARAMETER(FVector3f,									ScaleMeshSizeBySpeed_ScaleFactorScale)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	ScaleMeshSizeBySpeed_ScaleDistribution)
		SHADER_PARAMETER(float,										ScaleMeshSizeBySpeed_VelocityNorm)
	END_SHADER_PARAMETER_STRUCT()
		
	BEGIN_SHADER_PARAMETER_STRUCT(FMeshIndexModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(int32,		MeshIndex_Index)
		SHADER_PARAMETER(int32,		MeshIndex_TableOffset)
		SHADER_PARAMETER(int32,		MeshIndex_TableNumElements)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FScaleRibbonWidthModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	ScaleRibbonWidth_Distribution)
		SHADER_PARAMETER(float,										ScaleRibbonWidth_CurveScale)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FMeshRotationRateModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(int32,										MeshRotationRate_ModuleEnabled)
		SHADER_PARAMETER(FVector3f,									MeshRotationRate_Scale)
		SHADER_PARAMETER(FVector3f,									MeshRotationRate_Bias)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	MeshRotationRate_RateScaleParameters)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FScaleSpriteSizeModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	ScaleSpriteSize_Distribution)
		SHADER_PARAMETER(FVector2f,									ScaleSpriteSize_CurveScale)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FScaleSpriteSizeBySpeedModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FVector2f,									ScaleSpriteSizeBySpeed_ScaleFactorBias)
		SHADER_PARAMETER(FVector2f,									ScaleSpriteSizeBySpeed_ScaleFactorScale)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	ScaleSpriteSizeBySpeed_ScaleDistribution)
		SHADER_PARAMETER(float,										ScaleSpriteSizeBySpeed_VelocityNorm)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FSpriteFacingAndAlignmentModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FVector3f,	SpriteFacingAndAlignment_SpriteFacing)
		SHADER_PARAMETER(FVector3f,	SpriteFacingAndAlignment_SpriteAlignment)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FSpriteRotationRateModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(int32,										SpriteRotationRate_Enabled)
		SHADER_PARAMETER(float,										SpriteRotationRate_Scale)
		SHADER_PARAMETER(float,										SpriteRotationRate_Bias)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType,	SpriteRotationRate_RateScaleParameters)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FSubUVAnimationModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(int,		SubUVAnimation_Mode)
		SHADER_PARAMETER(float,		SubUVAnimation_NumFrames)
		SHADER_PARAMETER(float,		SubUVAnimation_InitialFrameScale)
		SHADER_PARAMETER(float,		SubUVAnimation_InitialFrameBias)
		SHADER_PARAMETER(float,		SubUVAnimation_InitialFrameRateChange)
		SHADER_PARAMETER(float,		SubUVAnimation_AnimFrameStart)
		SHADER_PARAMETER(float,		SubUVAnimation_AnimFrameRange)
		SHADER_PARAMETER(float,		SubUVAnimation_RateScale)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FShapeLocationModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FUintVector4,	ShapeLocation_Mode)
		SHADER_PARAMETER(FVector4f,		ShapeLocation_Parameters0)
		SHADER_PARAMETER(FVector4f,		ShapeLocation_Parameters1)
		SHADER_PARAMETER(FVector4f,		ShapeLocation_Parameters2)
		SHADER_PARAMETER(FNiagaraStatelessBuiltDistributionType, ShapeLocation_TransformOffset)
		SHADER_PARAMETER(FVector3f,		ShapeLocation_TransformOffsetScale)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FSolveVelocitiesAndForcesModule_ShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_OwnerAccelerationScale)
		//SHADER_PARAMETER(float,			SolveVelocitiesAndForces_OwnerDragScale)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_OwnerVelocityScale)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_MassScale)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_MassBias)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_DragScale)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_DragBias)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_VelocityScale)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_VelocityBias)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_WindScale)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_WindBias)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_AccelerationScale)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_AccelerationBias)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_GravityScale)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_GravityBias)

		SHADER_PARAMETER(uint32,		SolveVelocitiesAndForces_ConeVelocityEnabled)
		SHADER_PARAMETER(FQuat4f,		SolveVelocitiesAndForces_ConeQuat)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_ConeVelocityScale)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_ConeVelocityBias)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_ConeAngleScale)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_ConeAngleBias)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_ConeVelocityFalloff)

		SHADER_PARAMETER(uint32,		SolveVelocitiesAndForces_PointVelocityEnabled)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_PointVelocityScale)
		SHADER_PARAMETER(float,			SolveVelocitiesAndForces_PointVelocityBias)
		SHADER_PARAMETER(FVector3f,		SolveVelocitiesAndForces_PointOrigin)

		SHADER_PARAMETER(uint32,				SolveVelocitiesAndForces_NoiseEnabled)
		SHADER_PARAMETER(float,					SolveVelocitiesAndForces_NoiseStrength)
		SHADER_PARAMETER(float,					SolveVelocitiesAndForces_NoiseFrequency)
		SHADER_PARAMETER(FVector3f,				SolveVelocitiesAndForces_NoiseFieldOffsetScale)
		SHADER_PARAMETER(FVector3f,				SolveVelocitiesAndForces_NoiseFieldOffsetBias)
		SHADER_PARAMETER(float,					SolveVelocitiesAndForces_NoiseLUTNormalize)
		SHADER_PARAMETER(uint32,				SolveVelocitiesAndForces_NoiseLUTRowMask)
		SHADER_PARAMETER(uint32,				SolveVelocitiesAndForces_NoiseLUTRowWidth)
		SHADER_PARAMETER_SRV(ByteAddressBuffer,	SolveVelocitiesAndForces_NoiseLUT)
	END_SHADER_PARAMETER_STRUCT()
}
