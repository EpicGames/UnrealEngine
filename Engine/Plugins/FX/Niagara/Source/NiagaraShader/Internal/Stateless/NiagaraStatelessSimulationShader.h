// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "GlobalShader.h"
#include "RenderGraphFwd.h"
#include "ShaderParameterStruct.h"

namespace NiagaraStateless
{
	static constexpr uint32 MaxGpuSpawnInfos = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FSpawnInfoShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER_SCALAR_ARRAY(uint32,	SpawnInfo_NumActive, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(uint32,	SpawnInfo_ParticleOffset, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(uint32,	SpawnInfo_UniqueOffset, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(float,	SpawnInfo_Time, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(float,	SpawnInfo_Rate, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(float,	SpawnInfo_Probability, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(float,	SpawnInfo_LifetimeScale, [MaxGpuSpawnInfos])
		SHADER_PARAMETER_SCALAR_ARRAY(float,	SpawnInfo_LifetimeBias, [MaxGpuSpawnInfos])
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FCommonShaderParameters, NIAGARASHADER_API)
		SHADER_PARAMETER(uint32,				Common_RandomSeed)
		SHADER_PARAMETER(float,					Common_SimulationTime)
		SHADER_PARAMETER(float,					Common_SimulationDeltaTime)
		SHADER_PARAMETER(float,					Common_SimulationInvDeltaTime)
		SHADER_PARAMETER(FVector3f,				Common_LocalToWorldScale)
		SHADER_PARAMETER_ARRAY(FQuat4f,			Common_ToSimulationRotations, [3])

		SHADER_PARAMETER(uint32,				Common_OutputBufferStride)
		SHADER_PARAMETER(uint32,				Common_GPUCountBufferOffset)
		SHADER_PARAMETER_UAV(RWBuffer<float>,	Common_FloatOutputBuffer)
		//SHADER_PARAMETER_UAV(RWBuffer<half>,	Common_HalfOutputBuffer)
		SHADER_PARAMETER_UAV(RWBuffer<int>,		Common_IntOutputBuffer)
		SHADER_PARAMETER_UAV(RWBuffer<int>,		Common_GPUCountBuffer)
		SHADER_PARAMETER_SRV(ByteAddressBuffer,	Common_StaticDataBuffer)
		SHADER_PARAMETER_SRV(ByteAddressBuffer,	Common_DynamicDataBuffer)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSpawnInfoShaderParameters,	SpawnParameters)
	END_SHADER_PARAMETER_STRUCT()

	class FSimulationShader : public FGlobalShader
	{
	public:
		using FParameters = FCommonShaderParameters;
		static constexpr uint32 ThreadGroupSize = 64;

		FSimulationShader() {}
		FSimulationShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FGlobalShader(Initializer) { }

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
			OutEnvironment.SetDefine(TEXT("NIAGARA_MAX_GPU_SPAWN_INFOS"), NiagaraStateless::MaxGpuSpawnInfos);
			OutEnvironment.SetDefine(TEXT("GPU_SIMULATION"), 1);
		}
	};
}
