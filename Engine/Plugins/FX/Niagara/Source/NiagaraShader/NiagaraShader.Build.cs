// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class NiagaraShader : ModuleRules
{
    public NiagaraShader(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "../../Shaders/Shared"));

		PrivateDependencyModuleNames.AddRange(
            new string[] {
                "CoreUObject",
                "Engine",
                "Renderer",
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "RenderCore",
                "VectorVM",
                "RHI",
                "Projects",
                "NiagaraCore",
                "NiagaraVertexFactories",
            }
        );

        if (Target.bBuildEditor == true)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "TargetPlatform",
                });

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "DerivedDataCache",
                });
        }

        PublicIncludePathModuleNames.AddRange(
            new string[] {
            });

		PrivateIncludePaths.AddRange(new string[] {
		});

		PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "DerivedDataCache",
                "Niagara",
            });
    }
}
