// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAsyncGpuTraceProvider.h"

#include "Containers/StridedView.h"
#include "GlobalShader.h"
#include "NiagaraAsyncGpuTraceProviderGsdf.h"
#include "NiagaraAsyncGpuTraceProviderHwrt.h"
#include "NiagaraSettings.h"
#include "NiagaraShaderParticleID.h"

#include "PrimitiveSceneInfo.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "SceneInterface.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////

class FNiagaraClearAsyncGpuTraceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraClearAsyncGpuTraceCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraClearAsyncGpuTraceCS, FGlobalShader);

	static constexpr uint32 kThreadGroupSizeX = 32;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), kThreadGroupSizeX);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARASHADER_API)
		SHADER_PARAMETER_UAV(StructuredBuffer<FNiagaraAsyncGpuTraceResult>, Results)
		SHADER_PARAMETER(uint32, TraceCount)
		SHADER_PARAMETER(uint32, ResultsOffset)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraClearAsyncGpuTraceCS, "/Plugin/FX/Niagara/Private/NiagaraAsyncGpuTraceUtils.usf", "NiagaraClearAsyncGpuTraceCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////

class FNiagaraUpdateCollisionGroupMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraUpdateCollisionGroupMapCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraUpdateCollisionGroupMapCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(Buffer<UINT>, RWHashTable)
		SHADER_PARAMETER(uint32, HashTableSize)
		SHADER_PARAMETER_UAV(Buffer<UINT>, RWHashToCollisionGroups)
		SHADER_PARAMETER_SRV(Buffer<UINT>, NewPrimIdCollisionGroupPairs)
		SHADER_PARAMETER(uint32, NumNewPrims)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), THREAD_COUNT);
	}

	static constexpr uint32 THREAD_COUNT = 64;
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraUpdateCollisionGroupMapCS, "/Plugin/FX/Niagara/Private/NiagaraRayTraceCollisionGroupShaders.usf", "UpdatePrimIdToCollisionGroupMap", SF_Compute);

////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////

FNiagaraAsyncGpuTraceProvider::EProviderType FNiagaraAsyncGpuTraceProvider::ResolveSupportedType(EProviderType InType, const FProviderPriorityArray& Priorities)
{
	switch (InType)
	{
#if RHI_RAYTRACING
		case ENDICollisionQuery_AsyncGpuTraceProvider::HWRT:
			return FNiagaraAsyncGpuTraceProviderHwrt::IsSupported() ? InType : ENDICollisionQuery_AsyncGpuTraceProvider::None;
#endif

		case ENDICollisionQuery_AsyncGpuTraceProvider::GSDF:
			return FNiagaraAsyncGpuTraceProviderGsdf::IsSupported() ? InType : ENDICollisionQuery_AsyncGpuTraceProvider::None;

		case ENDICollisionQuery_AsyncGpuTraceProvider::Default:
		{
			ENDICollisionQuery_AsyncGpuTraceProvider::Type CurrentType = ENDICollisionQuery_AsyncGpuTraceProvider::None;

			for (auto ProviderType : Priorities)
			{
				if (ENDICollisionQuery_AsyncGpuTraceProvider::Default == ProviderType)
				{
					continue;
				}

				EProviderType ResolvedType = ResolveSupportedType(ProviderType, Priorities);
				if (ResolvedType != ENDICollisionQuery_AsyncGpuTraceProvider::None)
				{
					if (CurrentType == ENDICollisionQuery_AsyncGpuTraceProvider::None)
					{
						CurrentType = ResolvedType;
					}
					else if (CurrentType != ResolvedType)
					{
						// we've got multiple valid types so just return 'Default'
						return ENDICollisionQuery_AsyncGpuTraceProvider::Default;
					}
				}
			}

			return CurrentType;
		}
	}

	return ENDICollisionQuery_AsyncGpuTraceProvider::None;
}

template<typename T>
bool CheckProviderIsRequestedAndSupported(FNiagaraAsyncGpuTraceProvider::EProviderType InType, const FNiagaraAsyncGpuTraceProvider::FProviderPriorityArray& Priorities)
{
	if ((InType == T::Type) ||
		(InType == ENDICollisionQuery_AsyncGpuTraceProvider::Default && Priorities.Contains(T::Type)))
	{
		return T::IsSupported();
	}

	return false;
}

bool FNiagaraAsyncGpuTraceProvider::RequiresGlobalDistanceField(EProviderType InType, const FProviderPriorityArray& Priorities)
{
	return CheckProviderIsRequestedAndSupported<FNiagaraAsyncGpuTraceProviderGsdf>(InType, Priorities);
}

bool FNiagaraAsyncGpuTraceProvider::RequiresRayTracingScene(EProviderType InType, const FProviderPriorityArray& Priorities)
{
#if RHI_RAYTRACING
	return CheckProviderIsRequestedAndSupported<FNiagaraAsyncGpuTraceProviderHwrt>(InType, Priorities);
#else
	return false;
#endif
}

TArray<TUniquePtr<FNiagaraAsyncGpuTraceProvider>> FNiagaraAsyncGpuTraceProvider::CreateSupportedProviders(EShaderPlatform ShaderPlatform, FNiagaraGpuComputeDispatchInterface* Dispatcher, const FProviderPriorityArray& Priorities)
{
	TArray<TUniquePtr<FNiagaraAsyncGpuTraceProvider>> Providers;

	for (auto ProviderType : Priorities)
	{
		switch (ProviderType)
		{
#if RHI_RAYTRACING
		case ENDICollisionQuery_AsyncGpuTraceProvider::HWRT:
				if (FNiagaraAsyncGpuTraceProviderHwrt::IsSupported())
				{
					Providers.Emplace(MakeUnique<FNiagaraAsyncGpuTraceProviderHwrt>(ShaderPlatform, Dispatcher));
				}
				break;
#endif

			case ENDICollisionQuery_AsyncGpuTraceProvider::GSDF:
				if (FNiagaraAsyncGpuTraceProviderGsdf::IsSupported())
				{
					Providers.Emplace(MakeUnique<FNiagaraAsyncGpuTraceProviderGsdf>(ShaderPlatform, Dispatcher));
				}
				break;
		}
	}

	return Providers;
}

void FNiagaraAsyncGpuTraceProvider::ClearResults(FRHICommandList& RHICmdList, EShaderPlatform ShaderPlatform, const FDispatchRequest& Request)
{
	SCOPED_DRAW_EVENT(RHICmdList, NiagaraClearAsyncGpuTraceResults);

	if (Request.MaxTraceCount == 0)
	{
		return;
	}

	TShaderMapRef<FNiagaraClearAsyncGpuTraceCS> TraceShader(GetGlobalShaderMap(ShaderPlatform));

	FNiagaraClearAsyncGpuTraceCS::FParameters Params;
	Params.Results = Request.ResultsBuffer->UAV;
	Params.ResultsOffset = Request.ResultsOffset;
	Params.TraceCount = Request.MaxTraceCount;

	SetComputePipelineState(RHICmdList, TraceShader.GetComputeShader());

	SetShaderParameters(RHICmdList, TraceShader, TraceShader.GetComputeShader(), Params);

	const FIntVector ThreadGroupCount = FComputeShaderUtils::GetGroupCount(Request.MaxTraceCount, FNiagaraClearAsyncGpuTraceCS::kThreadGroupSizeX);
	DispatchComputeShader(RHICmdList, TraceShader.GetShader(), ThreadGroupCount.X, ThreadGroupCount.Y, ThreadGroupCount.Z);

	UnsetShaderUAVs(RHICmdList, TraceShader, TraceShader.GetComputeShader());
}

void FNiagaraAsyncGpuTraceProvider::BuildCollisionGroupHashMap(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FSceneInterface* Scene, const TMap<FPrimitiveComponentId, uint32>& CollisionGroupMap, FCollisionGroupHashMap& Result)
{
	const int32 CollisionMapEntryCount = CollisionGroupMap.Num();

	if (!CollisionMapEntryCount)
	{
		Result.HashTableSize = 0;
		Result.PrimIdHashTable.Release();
		Result.HashToCollisionGroups.Release();
		return;
	}

	SCOPED_DRAW_EVENT(RHICmdList, NiagaraUpdateCollisionGroupsMap);

	uint32 MinBufferInstances = FNiagaraUpdateCollisionGroupMapCS::THREAD_COUNT;
	uint32 NeededInstances = FMath::Max((uint32)CollisionMapEntryCount, MinBufferInstances);
	uint32 AllocInstances = Align(NeededInstances, FNiagaraUpdateCollisionGroupMapCS::THREAD_COUNT);

	//We can probably be smarter here but for now just push the whole map over to the GPU each time it's dirty.
	//Ideally this should be a small amount of data and updated infrequently.
	FReadBuffer NewPrimIdCollisionGroupPairs;
	NewPrimIdCollisionGroupPairs.Initialize(RHICmdList, TEXT("NewPrimIdCollisionGroupPairs"), sizeof(uint32) * 2, AllocInstances, EPixelFormat::PF_R32G32_UINT, BUF_Volatile);

	uint32* PrimIdCollisionGroupPairPtr = (uint32*)RHICmdList.LockBuffer(NewPrimIdCollisionGroupPairs.Buffer, 0, AllocInstances * sizeof(uint32) * 2, RLM_WriteOnly);
	FMemory::Memset(PrimIdCollisionGroupPairPtr, 0, AllocInstances * sizeof(uint32) * 2);

	for (auto Entry : CollisionGroupMap)
	{
		FPrimitiveComponentId PrimId = Entry.Key;
		uint32 CollisionGroup = Entry.Value;

		const FPrimitiveSceneInfo* const PrimitiveSceneInfo = Scene->GetPrimitiveSceneInfo(PrimId);
		const uint32 GPUSceneInstanceIndex = PrimitiveSceneInfo ? PrimitiveSceneInfo->GetInstanceSceneDataOffset() : INDEX_NONE;

		PrimIdCollisionGroupPairPtr[0] = GPUSceneInstanceIndex;
		PrimIdCollisionGroupPairPtr[1] = CollisionGroup;
		PrimIdCollisionGroupPairPtr += 2;
	}
	RHICmdList.UnlockBuffer(NewPrimIdCollisionGroupPairs.Buffer);

	//Init the hash table if needed
	if (Result.HashTableSize < AllocInstances)
	{
		Result.HashTableSize = AllocInstances;

		Result.PrimIdHashTable.Release();
		Result.PrimIdHashTable.Initialize(RHICmdList, TEXT("NiagaraPrimIdHashTable"), sizeof(uint32), AllocInstances, BUF_Static);

		Result.HashToCollisionGroups.Release();
		Result.HashToCollisionGroups.Initialize(RHICmdList, TEXT("NiagaraPrimIdHashToCollisionGroups"), sizeof(uint32), AllocInstances, EPixelFormat::PF_R32_UINT, BUF_Static);
	}

	RHICmdList.Transition(FRHITransitionInfo(Result.PrimIdHashTable.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	RHICmdList.Transition(FRHITransitionInfo(Result.HashToCollisionGroups.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

	//First we have to clear the buffers. Can probably do this better.
	NiagaraFillGPUIntBuffer(RHICmdList, FeatureLevel, Result.PrimIdHashTable.UAV, Result.PrimIdHashTable.NumBytes / sizeof(uint32), 0);
	RHICmdList.Transition(FRHITransitionInfo(Result.PrimIdHashTable.UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));
	NiagaraFillGPUIntBuffer(RHICmdList, FeatureLevel, Result.HashToCollisionGroups.UAV, Result.HashToCollisionGroups.NumBytes / sizeof(uint32), INDEX_NONE);
	RHICmdList.Transition(FRHITransitionInfo(Result.HashToCollisionGroups.UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));

	TShaderMapRef<FNiagaraUpdateCollisionGroupMapCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();

	FNiagaraUpdateCollisionGroupMapCS::FParameters Params;
	Params.RWHashTable = Result.PrimIdHashTable.UAV;
	Params.HashTableSize = Result.HashTableSize;
	Params.RWHashToCollisionGroups = Result.HashToCollisionGroups.UAV;
	Params.NewPrimIdCollisionGroupPairs = NewPrimIdCollisionGroupPairs.SRV;
	Params.NumNewPrims = CollisionGroupMap.Num();

	// To simplify the shader code, the size of the table must be a multiple of the thread count.
	check(Result.HashTableSize % FNiagaraUpdateCollisionGroupMapCS::THREAD_COUNT == 0);

	SetComputePipelineState(RHICmdList, ShaderRHI);
	SetShaderParameters(RHICmdList, ComputeShader, ShaderRHI, Params);
	RHICmdList.DispatchComputeShader(FMath::DivideAndRoundUp(CollisionGroupMap.Num(), (int32)FNiagaraUpdateCollisionGroupMapCS::THREAD_COUNT), 1, 1);
	UnsetShaderUAVs(RHICmdList, ComputeShader, ShaderRHI);

	RHICmdList.Transition(FRHITransitionInfo(Result.PrimIdHashTable.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));
	RHICmdList.Transition(FRHITransitionInfo(Result.HashToCollisionGroups.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));
}

void FNiagaraAsyncGpuTraceProvider::PostRenderOpaque(FRHICommandList& RHICmdList, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash)
{
}
