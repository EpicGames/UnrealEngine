// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAsyncGpuTraceProviderGsdf.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "FXRenderingUtils.h"
#include "GlobalShader.h"
#include "NiagaraDistanceFieldHelper.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSettings.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphUtils.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "ShaderParameterStruct.h"

static int GNiagaraAsyncGpuTraceGsdfEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraAsyncGpuTraceGsdfEnabled(
	TEXT("fx.Niagara.AsyncGpuTrace.GlobalSdfEnabled"),
	GNiagaraAsyncGpuTraceGsdfEnabled,
	TEXT("If disabled AsyncGpuTrace will not be supported against Global SDF."),
	ECVF_Default
);

class FNiagaraRayMarchGlobalSdfCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraRayMarchGlobalSdfCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRayMarchGlobalSdfCS, FGlobalShader);

	static constexpr uint32 kMaxMarchCount = 16;
	static constexpr uint32 kThreadGroupSizeX = 32;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!FGlobalShader::ShouldCompilePermutation(Parameters))
		{
			return false;
		}

		if (!ShouldCompileDistanceFieldShaders(Parameters.Platform))
		{
			return false;
		}

		// the GSDF sampling is only supported with SM5+
		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_MARCH_COUNT"), kMaxMarchCount);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), kThreadGroupSizeX);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARASHADER_API)
		SHADER_PARAMETER_SRV(StructuredBuffer<FNiagaraAsyncGpuTrace>, Traces)
		SHADER_PARAMETER_SRV(Buffer<uint>, TraceCounts)
		SHADER_PARAMETER_UAV(StructuredBuffer<FNiagaraAsyncGpuTraceResult>, Results)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldParameters2, GlobalDistanceFieldParameters)
		SHADER_PARAMETER(uint32, TracesOffset)
		SHADER_PARAMETER(uint32, TraceCountsOffset)
		SHADER_PARAMETER(uint32, ResultsOffset)
		SHADER_PARAMETER(float, DistanceThreshold)
		END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraRayMarchGlobalSdfCS, "/Plugin/FX/Niagara/Private/NiagaraRayMarchingShaders.usf", "NiagaraRayMarchGlobalSdfCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////

const FNiagaraAsyncGpuTraceProvider::EProviderType FNiagaraAsyncGpuTraceProviderGsdf::Type = ENDICollisionQuery_AsyncGpuTraceProvider::GSDF;

FNiagaraAsyncGpuTraceProviderGsdf::FNiagaraAsyncGpuTraceProviderGsdf(EShaderPlatform InShaderPlatform, FNiagaraGpuComputeDispatchInterface* Dispatcher)
	: FNiagaraAsyncGpuTraceProvider(InShaderPlatform, Dispatcher)
{
}

bool FNiagaraAsyncGpuTraceProviderGsdf::IsSupported()
{
	return GNiagaraAsyncGpuTraceGsdfEnabled
		&& DoesProjectSupportDistanceFields();
}

bool FNiagaraAsyncGpuTraceProviderGsdf::IsAvailable() const
{
	if (!GNiagaraAsyncGpuTraceGsdfEnabled)
	{
		return false;
	}

	// check the feature level of the scene.  Gsdf sampling requires SM5+
	if (Dispatcher->GetSceneInterface()->GetFeatureLevel() < ERHIFeatureLevel::SM5)
	{
		return false;
	}

	return true;
}

void FNiagaraAsyncGpuTraceProviderGsdf::PostRenderOpaque(FRHICommandList& RHICmdList, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash)
{
	if (const FGlobalDistanceFieldParameterData* DistanceFieldData = UE::FXRenderingUtils::GetGlobalDistanceFieldParameterData(Views))
	{
		m_DistanceFieldData = *DistanceFieldData;
	}
	else
	{
		m_DistanceFieldData = {};
	}
	m_ViewUniformBuffer = Views[0].ViewUniformBuffer;
}

void FNiagaraAsyncGpuTraceProviderGsdf::IssueTraces(FRHICommandList& RHICmdList, const FDispatchRequest& Request, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionHashMap)
{
	check(IsAvailable());

	if (Request.MaxTraceCount == 0)
	{
		return;
	}

	SCOPED_DRAW_EVENT(RHICmdList, NiagaraIssueTracesGsdf);

	TShaderMapRef<FNiagaraRayMarchGlobalSdfCS> TraceShader(GetGlobalShaderMap(ShaderPlatform));

	FNiagaraRayMarchGlobalSdfCS::FParameters Params;
	Params.View = GetShaderBinding(m_ViewUniformBuffer);
	Params.Traces = Request.TracesBuffer->SRV;
	Params.TraceCounts = Request.TraceCountsBuffer->SRV;
	Params.Results = Request.ResultsBuffer->UAV;
	FNiagaraDistanceFieldHelper::SetGlobalDistanceFieldParameters(&m_DistanceFieldData, Params.GlobalDistanceFieldParameters);
	Params.TracesOffset = Request.TracesOffset;
	Params.TraceCountsOffset = Request.TraceCountsOffset;
	Params.ResultsOffset = Request.ResultsOffset;
	Params.DistanceThreshold = 1.0f;

	SetComputePipelineState(RHICmdList, TraceShader.GetComputeShader());

	SetShaderParameters(RHICmdList, TraceShader, TraceShader.GetComputeShader(), Params);

	const FIntVector ThreadGroupCount = FComputeShaderUtils::GetGroupCount(Request.MaxTraceCount, FNiagaraRayMarchGlobalSdfCS::kThreadGroupSizeX);
	DispatchComputeShader(RHICmdList, TraceShader.GetShader(), ThreadGroupCount.X, ThreadGroupCount.Y, ThreadGroupCount.Z);

	UnsetShaderUAVs(RHICmdList, TraceShader, TraceShader.GetComputeShader());
}

void FNiagaraAsyncGpuTraceProviderGsdf::Reset()
{
	m_DistanceFieldData = FGlobalDistanceFieldParameterData();
	m_ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>();
}
