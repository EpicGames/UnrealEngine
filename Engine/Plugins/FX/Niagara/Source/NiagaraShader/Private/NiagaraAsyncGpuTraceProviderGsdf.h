// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalDistanceFieldParameters.h"
#include "NiagaraAsyncGpuTraceProvider.h"

class FViewUniformShaderParameters;

class FNiagaraAsyncGpuTraceProviderGsdf : public FNiagaraAsyncGpuTraceProvider
{
public:
	static const EProviderType Type;

	FNiagaraAsyncGpuTraceProviderGsdf(EShaderPlatform InShaderPlatform, FNiagaraGpuComputeDispatchInterface* Dispatcher);

	static bool IsSupported();
	virtual void PostRenderOpaque(FRHICommandList& RHICmdList, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash) override;
	virtual bool IsAvailable() const override;
	virtual void IssueTraces(FRHICommandList& RHICmdList, const FDispatchRequest& Request, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash) override;
	virtual void Reset() override;
	virtual EProviderType GetType() const override { return Type; }

private:
	FGlobalDistanceFieldParameterData m_DistanceFieldData;
	TUniformBufferRef<FViewUniformShaderParameters> m_ViewUniformBuffer;
};
