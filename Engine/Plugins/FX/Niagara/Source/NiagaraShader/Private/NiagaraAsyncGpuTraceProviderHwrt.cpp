// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAsyncGpuTraceProviderHwrt.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include "SceneView.h"

#if RHI_RAYTRACING

#include "Containers/StridedView.h"
#include "FXRenderingUtils.h"
#include "GlobalShader.h"
#include "GlobalRenderResources.h"
#include "MeshPassProcessor.h"
#include "RayTracingMeshDrawCommands.h"
#include "NiagaraSettings.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "Templates/Function.h"
#include "RayTracingPayloadType.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"

static int GNiagaraAsyncGpuTraceHwrtEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraAsyncGpuTraceHwrtEnabled(
	TEXT("fx.Niagara.AsyncGpuTrace.HWRayTraceEnabled"),
	GNiagaraAsyncGpuTraceHwrtEnabled,
	TEXT("If disabled AsyncGpuTrace will not be supported against the HW ray tracing scene."),
	ECVF_Default
);

static int GNiagaraAsyncGpuTraceHwrtInline = 1;
static FAutoConsoleVariableRef CVarNiagaraAsyncGpuTraceHwrtInline(
	TEXT("fx.Niagara.AsyncGpuTrace.HWRayTrace.Inline"),
	GNiagaraAsyncGpuTraceHwrtInline,
	TEXT("If disabled AsyncGpuTrace will not be supported against the HW ray tracing scene."),
	ECVF_Default);

bool SupportsNiagaraAsyncGpuTraceHwrtInline()
{
	return GRHISupportsInlineRayTracing && GNiagaraAsyncGpuTraceHwrtInline;
}

bool PlatformRequiresInlineSBTForAsyncGpuTraceHwrtInline(EShaderPlatform Platform)
{
	// PC can only fetch normals if InlineRayTracingBindingDataBuffer is valid. Otherwise we will fetch garbage
	// Other platforms can query the normal by pulling verts directly from the RT BVH
	return IsPCPlatform(Platform);
}

/// TODO
///  -get geometry masking working when an environmental mask is implemented

// c++ mirror of the struct defined in RayTracingCommon.ush
struct FVFXTracePayload
{
	float HitT;
	uint32 GPUSceneInstanceId;
	float Barycentrics[2];
	float WorldPosition[3];
	float WorldNormal[3];
};

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::VFX, sizeof(FVFXTracePayload));

class FNiagaraCollisionRayTrace : public FGlobalShader
{
public:
	FNiagaraCollisionRayTrace() = default;
	FNiagaraCollisionRayTrace(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}\

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_REF(FSceneUniformParameters, Scene)

		SHADER_PARAMETER_SRV(Buffer<UINT>, HashTable)
		SHADER_PARAMETER_SRV(Buffer<UINT>, HashToCollisionGroups)
		SHADER_PARAMETER(uint32, HashTableSize)

		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_SRV(StructuredBuffer, RayTracingSceneMetadata)

		SHADER_PARAMETER_SRV(Buffer<FNiagaraAsyncGpuTrace>, Rays)
		SHADER_PARAMETER(uint32, RaysOffset)
		SHADER_PARAMETER_UAV(Buffer<FNiagaraAsyncGpuTraceResult>, CollisionOutput)
		SHADER_PARAMETER(uint32, CollisionOutputOffset)
		SHADER_PARAMETER_SRV(Buffer<UINT>, RayTraceCounts)
		SHADER_PARAMETER(uint32, RayTraceCountsOffset)
		SHADER_PARAMETER(uint32, MaxRetraces)
		END_SHADER_PARAMETER_STRUCT()
};

class FNiagaraCollisionRayTraceRG : public FNiagaraCollisionRayTrace
{
	DECLARE_GLOBAL_SHADER(FNiagaraCollisionRayTraceRG)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FNiagaraCollisionRayTraceRG, FNiagaraCollisionRayTrace)

		class FFakeIndirectDispatch : SHADER_PERMUTATION_BOOL("NIAGARA_RAYTRACE_FAKE_INDIRECT");
	class FSupportsCollisionGroups : SHADER_PERMUTATION_BOOL("NIAGARA_SUPPORTS_COLLISION_GROUPS");
	using FPermutationDomain = TShaderPermutationDomain<FFakeIndirectDispatch, FSupportsCollisionGroups>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::VFX;
	}

	static TShaderRef< FNiagaraCollisionRayTraceRG> GetShader(FGlobalShaderMap* ShaderMap, bool SupportsCollisionGroups);
	static FRHIRayTracingShader* GetRayTracingShader(FGlobalShaderMap* ShaderMap, bool SupportCollisionGroups);
	static bool SupportsIndirectDispatch();
};

class FNiagaraCollisionRayTraceCS : public FNiagaraCollisionRayTrace
{
	DECLARE_GLOBAL_SHADER(FNiagaraCollisionRayTraceCS)
	SHADER_USE_PARAMETER_STRUCT(FNiagaraCollisionRayTraceCS, FNiagaraCollisionRayTrace);

	class FSupportsCollisionGroups : SHADER_PERMUTATION_BOOL("NIAGARA_SUPPORTS_COLLISION_GROUPS");
	using FPermutationDomain = TShaderPermutationDomain<FSupportsCollisionGroups>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static const uint32 DispatchSize = 32;
};

class FNiagaraCollisionRayTraceCH : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraCollisionRayTraceCH);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::VFX;
	}

	FNiagaraCollisionRayTraceCH() = default;
	FNiagaraCollisionRayTraceCH(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
};

class FNiagaraCollisionRayTraceMiss : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraCollisionRayTraceMiss);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::VFX;
	}

	FNiagaraCollisionRayTraceMiss() = default;
	FNiagaraCollisionRayTraceMiss(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
};

bool FNiagaraCollisionRayTraceRG::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
}

void FNiagaraCollisionRayTraceRG::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("NIAGARA_SUPPORTS_RAY_TRACING"), 1);
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
}

bool FNiagaraCollisionRayTraceCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsRayTracingEnabledForProject(Parameters.Platform) && RHISupportsRayTracing(Parameters.Platform) && RHISupportsInlineRayTracing(Parameters.Platform);
}

void FNiagaraCollisionRayTraceCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);
	OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);

	OutEnvironment.SetDefine(TEXT("RAY_TRACING_THREAD_GROUP_SIZE_X"), FNiagaraCollisionRayTraceCS::DispatchSize);
	OutEnvironment.SetDefine(TEXT("NIAGARA_SUPPORTS_RAY_TRACING"), 1);
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	OutEnvironment.SetDefine(TEXT("NIAGARA_RAYTRACE_FAKE_INDIRECT"), 0);
}

TShaderRef<FNiagaraCollisionRayTraceRG> FNiagaraCollisionRayTraceRG::GetShader(FGlobalShaderMap* ShaderMap, bool SupportsCollisionGroups)
{
	FPermutationDomain PermutationVector;
	PermutationVector.Set<FFakeIndirectDispatch>(!SupportsIndirectDispatch());
	PermutationVector.Set<FSupportsCollisionGroups>(SupportsCollisionGroups);

	return ShaderMap->GetShader<FNiagaraCollisionRayTraceRG>(PermutationVector);
}

FRHIRayTracingShader* FNiagaraCollisionRayTraceRG::GetRayTracingShader(FGlobalShaderMap* ShaderMap, bool SupportCollisionGroups)
{
	return GetShader(ShaderMap, SupportCollisionGroups).GetRayTracingShader();
}

bool FNiagaraCollisionRayTraceRG::SupportsIndirectDispatch()
{
	return GRHISupportsRayTracingDispatchIndirect;
}

bool FNiagaraCollisionRayTraceCH::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
}

void FNiagaraCollisionRayTraceCH::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("NIAGARA_SUPPORTS_RAY_TRACING"), 1);
}

FNiagaraCollisionRayTraceCH::FNiagaraCollisionRayTraceCH(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{}

bool FNiagaraCollisionRayTraceMiss::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
}

void FNiagaraCollisionRayTraceMiss::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("NIAGARA_SUPPORTS_RAY_TRACING"), 1);
}

FNiagaraCollisionRayTraceMiss::FNiagaraCollisionRayTraceMiss(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{}

IMPLEMENT_GLOBAL_SHADER(FNiagaraCollisionRayTraceCS, "/Plugin/FX/Niagara/Private/NiagaraRayTracingShaders.usf", "NiagaraCollisionRayTraceCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraCollisionRayTraceRG, "/Plugin/FX/Niagara/Private/NiagaraRayTracingShaders.usf", "NiagaraCollisionRayTraceRG", SF_RayGen);
IMPLEMENT_GLOBAL_SHADER(FNiagaraCollisionRayTraceCH, "/Plugin/FX/Niagara/Private/NiagaraRayTracingShaders.usf", "NiagaraCollisionRayTraceCH", SF_RayHitGroup);
IMPLEMENT_GLOBAL_SHADER(FNiagaraCollisionRayTraceMiss, "/Plugin/FX/Niagara/Private/NiagaraRayTracingShaders.usf", "NiagaraCollisionRayTraceMiss", SF_RayMiss);

////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////

static FRayTracingPipelineState* CreateNiagaraRayTracingPipelineState(
	EShaderPlatform Platform,
	FRHICommandList& RHICmdList,
	FRHIRayTracingShader* RayGenShader,
	FRHIRayTracingShader* ClosestHitShader,
	FRHIRayTracingShader* MissShader,
	uint32& OutMaxLocalBindingDataSize)
{
	FRayTracingPipelineStateInitializer Initializer;
	Initializer.MaxPayloadSizeInBytes = sizeof(FVFXTracePayload);

	FRHIRayTracingShader* RayGenShaderTable[] = { RayGenShader };
	Initializer.SetRayGenShaderTable(RayGenShaderTable);

	FRHIRayTracingShader* HitGroupTable[] = { ClosestHitShader };
	Initializer.SetHitGroupTable(HitGroupTable);

	FRHIRayTracingShader* MissTable[] = { MissShader };
	Initializer.SetMissShaderTable(MissTable);

	OutMaxLocalBindingDataSize = Initializer.GetMaxLocalBindingDataSize();

	return PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer);
}

static void BindNiagaraRayTracingMeshCommands(
	FRHICommandList& RHICmdList,
	FRHIShaderBindingTable* SBT,
	FRHIUniformBuffer* ViewUniformBuffer,
	TConstArrayView<FRayTracingShaderBindingData> DirtyShaderBindings,
	const FRayTracingMeshCommandStorage& MeshCommands,
	FRayTracingPipelineState* Pipeline,
	TFunctionRef<uint32(const FRayTracingMeshCommand&)> PackUserData)
{
	const int32 NumTotalBindings = DirtyShaderBindings.Num();

	const uint32 MergedBindingsSize = sizeof(FRayTracingLocalShaderBindings) * NumTotalBindings;

	const uint32 NumUniformBuffers = 1;
	const uint32 UniformBufferArraySize = sizeof(FRHIUniformBuffer*) * NumUniformBuffers;

	FConcurrentLinearBulkObjectAllocator Allocator;
	FRayTracingLocalShaderBindings* Bindings = nullptr;
	FRHIUniformBuffer** UniformBufferArray = nullptr;

	if (RHICmdList.Bypass())
	{
		Bindings = reinterpret_cast<FRayTracingLocalShaderBindings*>(Allocator.Malloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings)));
		UniformBufferArray = reinterpret_cast<FRHIUniformBuffer**>(Allocator.Malloc(UniformBufferArraySize, alignof(FRHIUniformBuffer*)));
	}
	else
	{
		Bindings = reinterpret_cast<FRayTracingLocalShaderBindings*>(RHICmdList.Alloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings)));
		UniformBufferArray = reinterpret_cast<FRHIUniformBuffer**>(RHICmdList.Alloc(UniformBufferArraySize, alignof(FRHIUniformBuffer*)));
	}

	UniformBufferArray[0] = ViewUniformBuffer;

	const uint32 NumShaderSlotsPerGeometrySegment = SBT->GetInitializer().NumShaderSlotsPerGeometrySegment;

	uint32 BindingIndex = 0;
	for (const FRayTracingShaderBindingData DirtyShaderBinding : DirtyShaderBindings)
	{
		const FRayTracingMeshCommand& MeshCommand = DirtyShaderBinding.GetRayTracingMeshCommand(MeshCommands);

		FRayTracingLocalShaderBindings Binding = {};
		Binding.RecordIndex = DirtyShaderBinding.SBTRecordIndex;
		Binding.Geometry = DirtyShaderBinding.RayTracingGeometry;
		Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
		Binding.UserData = PackUserData(MeshCommand);
		Binding.UniformBuffers = UniformBufferArray;
		Binding.NumUniformBuffers = NumUniformBuffers;

		Bindings[BindingIndex] = Binding;
		BindingIndex++;
	}

	const bool bCopyDataToInlineStorage = false; // Storage is already allocated from RHICmdList, no extra copy necessary
	RHICmdList.SetRayTracingHitGroups(
		SBT,
		Pipeline,
		NumTotalBindings,
		Bindings,
		bCopyDataToInlineStorage);
	RHICmdList.SetRayTracingMissShader(SBT, 0, Pipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
	RHICmdList.CommitShaderBindingTable(SBT);
}

////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////

const FNiagaraAsyncGpuTraceProvider::EProviderType FNiagaraAsyncGpuTraceProviderHwrt::Type = ENDICollisionQuery_AsyncGpuTraceProvider::HWRT;

FNiagaraAsyncGpuTraceProviderHwrt::FNiagaraAsyncGpuTraceProviderHwrt(EShaderPlatform InShaderPlatform, FNiagaraGpuComputeDispatchInterface* Dispatcher)
	: FNiagaraAsyncGpuTraceProvider(InShaderPlatform, Dispatcher)
{
}

bool FNiagaraAsyncGpuTraceProviderHwrt::IsSupported()
{
	return GNiagaraAsyncGpuTraceHwrtEnabled && IsRayTracingEnabled();
}

bool FNiagaraAsyncGpuTraceProviderHwrt::IsAvailable() const
{
	// Never allow HWRT to run if the dispatcher is outside of the scene renderer. The TLAS is only expected to be valid for a single frame 
	// and this risks using a stale TLAS from many frames ago which can contain pointers to resources that no longer exist
	if (!GNiagaraAsyncGpuTraceHwrtEnabled ||Dispatcher->IsOutsideSceneRenderer())
	{
		return false;
	}

	if (!Dispatcher->RequiresRayTracingScene())
	{
		return false;
	}

	if (PlatformRequiresInlineSBTForAsyncGpuTraceHwrtInline(GMaxRHIShaderPlatform))
	{
		FSceneInterface* Scene = Dispatcher->GetSceneInterface();
		if (UE::FXRenderingUtils::RayTracing::GetInlineRayTracingBindingDataBuffer(*Scene) == nullptr)
		{
			return false;
		}
	}

	return UE::FXRenderingUtils::RayTracing::HasRayTracingScene(*Dispatcher->GetSceneInterface());
}

void FNiagaraAsyncGpuTraceProviderHwrt::PostRenderOpaque(FRHICommandList& RHICmdList, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash)
{
	check(IsAvailable());
	check(Views.Num() > 0);

	const FSceneView& ReferenceView = Views[0]; // TODO: Should use appropriate view instead
	FSceneInterface* Scene = Dispatcher->GetSceneInterface();

	if (UE::FXRenderingUtils::RayTracing::HasRayTracingScene(*Scene))
	{
		TLASSRV = UE::FXRenderingUtils::RayTracing::GetRayTracingSceneView(RHICmdList, *Scene, ReferenceView);

		InlineRayTracingBindingDataBufferSRV = UE::FXRenderingUtils::RayTracing::GetInlineRayTracingBindingDataBuffer(*Scene);

		ViewUniformBuffer = ReferenceView.ViewUniformBuffer;

		if (!SupportsNiagaraAsyncGpuTraceHwrtInline() && GRHISupportsRayTracingShaders)
		{
			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);
			auto RayGenShader = FNiagaraCollisionRayTraceRG::GetRayTracingShader(ShaderMap, CollisionGroupHash != nullptr);
			auto ClosestHitShader = ShaderMap->GetShader<FNiagaraCollisionRayTraceCH>().GetRayTracingShader();
			auto MissShader = ShaderMap->GetShader<FNiagaraCollisionRayTraceMiss>().GetRayTracingShader();

			uint32 MaxLocalBindingDataSize = 0;
			RayTracingPipelineState = CreateNiagaraRayTracingPipelineState(
				ShaderPlatform,
				RHICmdList,
				RayGenShader,
				ClosestHitShader,
				MissShader,
				MaxLocalBindingDataSize);

			RayTracingSBT = UE::FXRenderingUtils::RayTracing::CreateShaderBindingTable(RHICmdList, Scene, MaxLocalBindingDataSize);

			// some options for what we want with our per MeshCommand user data.  For now we'll ignore it, but possibly
			// something we'd want to incorporate.  Some examples could be if the material is translucent, or possibly the physical material?
			auto BakeTranslucent = [&](const FRayTracingMeshCommand& MeshCommand) {	return (MeshCommand.bIsTranslucent != 0) & 0x1;	};
			auto BakeDefault = [&](const FRayTracingMeshCommand& MeshCommand) { return 0; };

			BindNiagaraRayTracingMeshCommands(
				RHICmdList,
				RayTracingSBT,
				ViewUniformBuffer,
				UE::FXRenderingUtils::RayTracing::GetVisibleRayTracingShaderBindings(ReferenceView),
				*UE::FXRenderingUtils::RayTracing::GetRayTracingMeshCommands(Scene),
				RayTracingPipelineState,
				BakeDefault);
		}
	}
	else
	{
		Reset();
	}
}

void FNiagaraAsyncGpuTraceProviderHwrt::IssueTraces(FRHICommandList& RHICmdList, const FDispatchRequest& Request, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash)
{
	check(IsAvailable());

	if (Request.MaxTraceCount == 0)
	{
		return;
	}

	SCOPED_DRAW_EVENT(RHICmdList, NiagaraIssueTracesHwrt);

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);

	FNiagaraCollisionRayTrace::FParameters Params;

	Params.View = GetShaderBinding(ViewUniformBuffer);
	Params.Scene = SceneUniformBufferRHI;

	if (CollisionGroupHash)
	{
		Params.HashTable = CollisionGroupHash->PrimIdHashTable.SRV;
		Params.HashTableSize = CollisionGroupHash->HashTableSize;
		Params.HashToCollisionGroups = CollisionGroupHash->HashToCollisionGroups.SRV;
	}

	FSceneInterface* Scene = Dispatcher->GetSceneInterface();
	Params.TLAS = TLASSRV;
	Params.Rays = Request.TracesBuffer->SRV;
	Params.RaysOffset = Request.TracesOffset;
	Params.CollisionOutput = Request.ResultsBuffer->UAV;
	Params.CollisionOutputOffset = Request.ResultsOffset;
	Params.MaxRetraces = Request.MaxRetraceCount;
	Params.RayTraceCountsOffset = Request.TraceCountsOffset;

	FRHIShaderResourceView* NullSRVBinding = GEmptyStructuredBufferWithUAV->ShaderResourceViewRHI.GetReference();
	bool bValidInlineRaytracingBindingDataBufferFound = InlineRayTracingBindingDataBufferSRV != nullptr;
	Params.RayTracingSceneMetadata = InlineRayTracingBindingDataBufferSRV ? InlineRayTracingBindingDataBufferSRV : NullSRVBinding;
	
	if (SupportsNiagaraAsyncGpuTraceHwrtInline())
	{
		check(bValidInlineRaytracingBindingDataBufferFound || !PlatformRequiresInlineSBTForAsyncGpuTraceHwrtInline(GMaxRHIShaderPlatform));

		FNiagaraCollisionRayTraceCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FNiagaraCollisionRayTraceCS::FSupportsCollisionGroups>(CollisionGroupHash != nullptr);
		TShaderRef<FNiagaraCollisionRayTraceCS> Shader = ShaderMap->GetShader<FNiagaraCollisionRayTraceCS>(PermutationVector);
		FRHIComputeShader* ShaderRHI = Shader.GetComputeShader();

		Params.RayTraceCounts = Request.TraceCountsBuffer->SRV;

		SetComputePipelineState(RHICmdList, ShaderRHI);
		SetShaderParameters(RHICmdList, Shader, ShaderRHI, Params);


		RHICmdList.DispatchComputeShader(FMath::DivideAndRoundUp(Request.MaxTraceCount, FNiagaraCollisionRayTraceCS::DispatchSize), 1, 1);

		UnsetShaderUAVs(RHICmdList, Shader, ShaderRHI);
	}
	else if (GRHISupportsRayTracingShaders)
	{
		check(RayTracingPipelineState);

		TShaderRef<FNiagaraCollisionRayTraceRG> RGShader = FNiagaraCollisionRayTraceRG::GetShader(ShaderMap, CollisionGroupHash != nullptr);
		if (FNiagaraCollisionRayTraceRG::SupportsIndirectDispatch())
		{
			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RGShader, Params);

			// Can we wrangle things so we can have one indirect dispatch with each internal dispatch pointing to potentially different Ray and Results buffers?
			// For now have a each as a unique dispatch.
			RHICmdList.RayTraceDispatchIndirect(
				RayTracingPipelineState,
				RGShader.GetRayTracingShader(),
				RayTracingSBT,
				GlobalResources,
				Request.TraceCountsBuffer->Buffer,
				Request.TraceCountsOffset * sizeof(uint32));
		}
		else
		{
			Params.RayTraceCounts = Request.TraceCountsBuffer->SRV;

			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RGShader, Params);

			RHICmdList.RayTraceDispatch(
				RayTracingPipelineState,
				RGShader.GetRayTracingShader(),
				RayTracingSBT,
				GlobalResources,
				Request.MaxTraceCount,
				1
			);
		}
	}
}

void FNiagaraAsyncGpuTraceProviderHwrt::Reset()
{
	RayTracingPipelineState = nullptr;
	RayTracingSBT = nullptr;
	TLASSRV = nullptr;
	ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>();
	InlineRayTracingBindingDataBufferSRV = nullptr;
}

#endif // RHI_RAYTRACING
