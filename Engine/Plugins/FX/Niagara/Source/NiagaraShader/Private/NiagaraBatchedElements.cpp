// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBatchedElements.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "SimpleElementShaders.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterUtils.h"

//////////////////////////////////////////////////////////////////////////

class FNiagaraSimpleElement2DArrayAttribute : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraSimpleElement2DArrayAttribute);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraSimpleElement2DArrayAttribute, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_2DARRAY_ATTRIBUTE_PS"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector4, InAttributeSlices)
		SHADER_PARAMETER(FMatrix44f, InColorWeights)
		SHADER_PARAMETER(float, InGamma)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InTextureSampler)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraSimpleElement2DArrayAttribute, "/Plugin/FX/Niagara/Private/NiagaraBatchedElements.usf", "MainPS", SF_Pixel);

void FBatchedElementNiagara2DArrayAttribute::BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture)
{
	FRHITexture* RHITexture = nullptr;
	FRHISamplerState* RHISamplerState = nullptr;
	if (GetTextureAndSampler != nullptr)
	{
		GetTextureAndSampler(RHITexture, RHISamplerState);
	}
	else if ( Texture != nullptr )
	{
		RHITexture = Texture->TextureRHI;
		RHISamplerState = Texture->SamplerStateRHI;
	}

	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FNiagaraSimpleElement2DArrayAttribute> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);

	FNiagaraSimpleElement2DArrayAttribute::FParameters PassParameters;
	PassParameters.InAttributeSlices = AttributeSlices;
	PassParameters.InColorWeights = FMatrix44f(ColorWeights);
	PassParameters.InGamma = InGamma;
	PassParameters.InTexture = RHITexture ? RHITexture : GWhiteTexture->TextureRHI.GetReference();
	PassParameters.InTextureSampler = RHISamplerState ? RHISamplerState : TStaticSamplerState<SF_Bilinear>::GetRHI();
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters);
}

//////////////////////////////////////////////////////////////////////////

class FNiagaraSimpleElementVolumeAttribute : public FGlobalShader
{
public:
	DECLARE_SHADER_TYPE(FNiagaraSimpleElementVolumeAttribute, Global);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraSimpleElementVolumeAttribute, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_VOLUME_ATTRIBUTE_PS"), 1);
	}


	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int,					InNumAttributes)
		SHADER_PARAMETER_ARRAY(FVector4f,		InAttributeUVAndChannel, [4])
		SHADER_PARAMETER(FVector2f,				InTileUV)
		SHADER_PARAMETER(FMatrix44f,			InColorWeights)
		SHADER_PARAMETER(float,					InGamma)
		SHADER_PARAMETER_TEXTURE(Texture2D,		InTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,	InTextureSampler)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraSimpleElementVolumeAttribute, "/Plugin/FX/Niagara/Private/NiagaraBatchedElements.usf", "MainPS", SF_Pixel);

void FBatchedElementNiagaraVolumeAttribute::BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture)
{
	FRHITexture* RHITexture = nullptr;
	FRHISamplerState* RHISamplerState = nullptr;
	if (GetTextureAndSampler != nullptr)
	{
		GetTextureAndSampler(RHITexture, RHISamplerState);
	}
	else if (Texture != nullptr)
	{
		RHITexture = Texture->TextureRHI;
		RHISamplerState = Texture->SamplerStateRHI;
	}

	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FNiagaraSimpleElementVolumeAttribute> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);

	FNiagaraSimpleElementVolumeAttribute::FParameters PassParameters;
	PassParameters.InNumAttributes	= AttributeUVs.Num();
	for (int32 i=0; i < 4; ++i)
	{
		const FVector3f AttributeUV = AttributeUVs.IsValidIndex(i) ? AttributeUVs[i] : FVector3f::ZeroVector;
		const int AttributeChannel = AttributeChannels.IsValidIndex(i) ? AttributeChannels[i] : 0;
		PassParameters.InAttributeUVAndChannel[i].X = AttributeUV.X;
		PassParameters.InAttributeUVAndChannel[i].Y = AttributeUV.Y;
		PassParameters.InAttributeUVAndChannel[i].Z = AttributeUV.Z;
		FMemory::Memcpy(&PassParameters.InAttributeUVAndChannel[i].W, &AttributeChannel, sizeof(int32));
	}
	PassParameters.InTileUV			= TileUVs;
	PassParameters.InColorWeights	= FMatrix44f(ColorWeights);
	PassParameters.InGamma			= InGamma;
	PassParameters.InTexture		= RHITexture ? RHITexture : GWhiteTexture->TextureRHI.GetReference();
	PassParameters.InTextureSampler	= RHISamplerState ? RHISamplerState : TStaticSamplerState<SF_Bilinear>::GetRHI();
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters);
}

//////////////////////////////////////////////////////////////////////////

class FNiagaraSimpleElementPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraSimpleElementPS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraSimpleElementPS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_SIMPLE_PS"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FMatrix44f, InColorWeights)
		SHADER_PARAMETER(float, InGamma)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InTextureSampler)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraSimpleElementPS, "/Plugin/FX/Niagara/Private/NiagaraBatchedElements.usf", "MainPS", SF_Pixel);

void FBatchedElementNiagaraInvertColorChannel::BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture)
{
	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FNiagaraSimpleElementPS> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_ALPHA, BO_Subtract, BF_One, BF_DestColor, BO_Subtract, BF_One, BF_DestAlpha>::GetRHI();

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);

	FNiagaraSimpleElementPS::FParameters PassParameters;
	PassParameters.InColorWeights	= FMatrix44f::Identity;
	PassParameters.InGamma			= InGamma;
	PassParameters.InTexture		= GWhiteTexture->TextureRHI;
	PassParameters.InTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters);
}

void FBatchedElementNiagaraSimple::BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture)
{
	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FNiagaraSimpleElementPS> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BlendState = bAlphaBlend ? TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI() : TStaticBlendState<>::GetRHI();

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);

	FNiagaraSimpleElementPS::FParameters PassParameters;
	PassParameters.InColorWeights = FMatrix44f(ColorTransform);
	PassParameters.InGamma = InGamma;
	PassParameters.InTexture = Texture ? Texture->TextureRHI : GWhiteTexture->TextureRHI;
	PassParameters.InTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters);
}
