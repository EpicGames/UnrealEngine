// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraClearCounts.h"

#include "GlobalShader.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

//////////////////////////////////////////////////////////////////////////

class FNiagaraClearCountsIntCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraClearCountsIntCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraClearCountsIntCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_CLEAR_COUNTS_INT_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	CountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	CountsAndValuesBuffer)
		SHADER_PARAMETER(uint32,						NumCounts)
	END_SHADER_PARAMETER_STRUCT()
};

class FNiagaraClearCountsUIntCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraClearCountsUIntCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraClearCountsUIntCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_CLEAR_COUNTS_UINT_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	CountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,	CountsAndValuesBuffer)
		SHADER_PARAMETER(uint32,						NumCounts)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraClearCountsIntCS, "/Plugin/FX/Niagara/Private/NiagaraClearCounts.usf", "ClearCountsIntCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraClearCountsUIntCS, "/Plugin/FX/Niagara/Private/NiagaraClearCounts.usf", "ClearCountsUIntCS", SF_Compute);

void NiagaraClearCounts::ClearCountsInt(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef UAV, TConstArrayView<TPair<uint32, int32>> IndexAndValueArray)
{
	const int32 NumInts = IndexAndValueArray.Num() * 2;
	FRDGBufferRef CountsAndValuesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumInts), TEXT("NiagaraClearCountsInt::CountsAndValuesBuffer"));
	GraphBuilder.QueueBufferUpload(CountsAndValuesBuffer, IndexAndValueArray.GetData(), sizeof(int32) * NumInts);

	TShaderMapRef<FNiagaraClearCountsIntCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(IndexAndValueArray.Num(), FNiagaraClearCountsIntCS::ThreadGroupSize);

	FNiagaraClearCountsIntCS::FParameters* Parameters = GraphBuilder.AllocParameters<FNiagaraClearCountsIntCS::FParameters>();
	Parameters->CountBuffer = UAV;
	Parameters->CountsAndValuesBuffer = GraphBuilder.CreateSRV(CountsAndValuesBuffer, PF_R32_SINT);
	Parameters->NumCounts = IndexAndValueArray.Num();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NiagaraClearCountsInt"),
		ERDGPassFlags::Compute,
		ComputeShader,
		Parameters,
		FIntVector(NumThreadGroups, 1, 1)
	);
}

void NiagaraClearCounts::ClearCountsUInt(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef UAV, TConstArrayView<TPair<uint32, uint32>> IndexAndValueArray)
{
	const int32 NumInts = IndexAndValueArray.Num() * 2;
	FRDGBufferRef CountsAndValuesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumInts), TEXT("NiagaraClearCountsInt::CountsAndValuesBuffer"));
	GraphBuilder.QueueBufferUpload(CountsAndValuesBuffer, IndexAndValueArray.GetData(), sizeof(int32) * NumInts);

	TShaderMapRef<FNiagaraClearCountsUIntCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(IndexAndValueArray.Num(), FNiagaraClearCountsUIntCS::ThreadGroupSize);

	FNiagaraClearCountsUIntCS::FParameters* Parameters = GraphBuilder.AllocParameters<FNiagaraClearCountsUIntCS::FParameters>();
	Parameters->CountBuffer = UAV;
	Parameters->CountsAndValuesBuffer = GraphBuilder.CreateSRV(CountsAndValuesBuffer, PF_R32_UINT);
	Parameters->NumCounts = IndexAndValueArray.Num();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NiagaraClearCountsUInt"),
		ERDGPassFlags::Compute,
		ComputeShader,
		Parameters,
		FIntVector(NumThreadGroups, 1, 1)
	);
}

//////////////////////////////////////////////////////////////////////////

class FNiagaraClearCountsIntCS_Legacy : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraClearCountsIntCS_Legacy);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraClearCountsIntCS_Legacy, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_CLEAR_COUNTS_INT_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<int>, CountBuffer)
		SHADER_PARAMETER_SRV(Buffer<int>,	CountsAndValuesBuffer)
		SHADER_PARAMETER(uint32,			NumCounts)
	END_SHADER_PARAMETER_STRUCT()
};

class FNiagaraClearCountsUIntCS_Legacy : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraClearCountsUIntCS_Legacy);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraClearCountsUIntCS_Legacy, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_CLEAR_COUNTS_UINT_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<uint>,	CountBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,		CountsAndValuesBuffer)
		SHADER_PARAMETER(uint32,				NumCounts)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraClearCountsIntCS_Legacy, "/Plugin/FX/Niagara/Private/NiagaraClearCounts.usf", "ClearCountsIntCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraClearCountsUIntCS_Legacy, "/Plugin/FX/Niagara/Private/NiagaraClearCounts.usf", "ClearCountsUIntCS", SF_Compute);

void NiagaraClearCounts::ClearCountsInt(FRHICommandList& RHICmdList, FRHIUnorderedAccessView* UAV, TConstArrayView<TPair<uint32, int32>> IndexAndValueArray)
{
	check(UAV != nullptr);
	check(IndexAndValueArray.Num() > 0);

	FReadBuffer IndexAndValueBuffer;
	IndexAndValueBuffer.Initialize(RHICmdList, TEXT("NiagaraClearCounts"), sizeof(int32), IndexAndValueArray.Num() * 2, PF_R32_SINT);
	{
		const uint32 BufferSize = IndexAndValueArray.Num() * sizeof(int32) * 2;
		void* GPUMemory = RHICmdList.LockBuffer(IndexAndValueBuffer.Buffer, 0, BufferSize, RLM_WriteOnly);
		FMemory::Memcpy(GPUMemory, IndexAndValueArray.GetData(), BufferSize);
		RHICmdList.UnlockBuffer(IndexAndValueBuffer.Buffer);
	}

	FNiagaraClearCountsIntCS_Legacy::FParameters ShaderParameters;
	ShaderParameters.CountBuffer			= UAV;
	ShaderParameters.CountsAndValuesBuffer	= IndexAndValueBuffer.SRV;
	ShaderParameters.NumCounts				= IndexAndValueArray.Num();

	TShaderMapRef<FNiagaraClearCountsIntCS_Legacy> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(IndexAndValueArray.Num(), FNiagaraClearCountsIntCS_Legacy::ThreadGroupSize);

	RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	SetComputePipelineState(RHICmdList, ShaderRHI);
	SetShaderParameters(RHICmdList, ComputeShader, ShaderRHI, ShaderParameters);
	RHICmdList.DispatchComputeShader(NumThreadGroups, 1, 1);
	UnsetShaderUAVs(RHICmdList, ComputeShader, ShaderRHI);
	RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));
}

void NiagaraClearCounts::ClearCountsUInt(FRHICommandList& RHICmdList, FRHIUnorderedAccessView* UAV, TConstArrayView<TPair<uint32, uint32>> IndexAndValueArray)
{
	check(UAV != nullptr);
	check(IndexAndValueArray.Num() > 0);

	FReadBuffer IndexAndValueBuffer;
	IndexAndValueBuffer.Initialize(RHICmdList, TEXT("NiagaraClearCounts"), sizeof(int32), IndexAndValueArray.Num() * 2, PF_R32_UINT);
	{
		const uint32 BufferSize = IndexAndValueArray.Num() * sizeof(int32) * 2;
		void* GPUMemory = RHICmdList.LockBuffer(IndexAndValueBuffer.Buffer, 0, BufferSize, RLM_WriteOnly);
		FMemory::Memcpy(GPUMemory, IndexAndValueArray.GetData(), BufferSize);
		RHICmdList.UnlockBuffer(IndexAndValueBuffer.Buffer);
	}

	FNiagaraClearCountsUIntCS_Legacy::FParameters ShaderParameters;
	ShaderParameters.CountBuffer = UAV;
	ShaderParameters.CountsAndValuesBuffer = IndexAndValueBuffer.SRV;
	ShaderParameters.NumCounts = IndexAndValueArray.Num();

	TShaderMapRef<FNiagaraClearCountsUIntCS_Legacy> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(IndexAndValueArray.Num(), FNiagaraClearCountsUIntCS_Legacy::ThreadGroupSize);

	RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	SetComputePipelineState(RHICmdList, ShaderRHI);
	SetShaderParameters(RHICmdList, ComputeShader, ShaderRHI, ShaderParameters);
	RHICmdList.DispatchComputeShader(NumThreadGroups, 1, 1);
	UnsetShaderUAVs(RHICmdList, ComputeShader, ShaderRHI);
	RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));
}
