// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDebugShaders.h"
#include "NiagaraCommon.h"

#include "FXRenderingUtils.h"
#include "RHI.h"
#include "ScreenRendering.h"
#include "CommonRenderResources.h"
#include "Modules/ModuleManager.h"
#include "PipelineStateCache.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "UnrealClient.h"

int GNiagaraGpuComputeDebug_ShowNaNInf = 1;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_ShowNaNInf(
	TEXT("fx.Niagara.GpuComputeDebug.ShowNaNInf"),
	GNiagaraGpuComputeDebug_ShowNaNInf,
	TEXT("When enabled will show NaNs as flashing colors."),
	ECVF_Default
);

int GNiagaraGpuComputeDebug_FourComponentMode = 0;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_FourComponentMode(
	TEXT("fx.Niagara.GpuComputeDebug.FourComponentMode"),
	GNiagaraGpuComputeDebug_FourComponentMode,
	TEXT("Adjust how we visualize four component types\n")
	TEXT("0 = Visualize RGB (defaut)\n")
	TEXT("1 = Visualize A\n"),
	ECVF_Default
);

float GNiagaraGpuComputeDebug_OccludedLineColorScale = 0.05f;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_OccludedLineColorScale(
	TEXT("fx.Niagara.GpuComputeDebug.OccludedLineColorScale"),
	GNiagaraGpuComputeDebug_OccludedLineColorScale,
	TEXT("Scalar value to adjust occluded lines, where 0 means transparent and 1 is opaque.  Default is 0.05 or 5%"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

namespace NiagaraDebugShaders
{
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const bool bTargetIsEditor = EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
		return (bTargetIsEditor && NIAGARA_COMPUTEDEBUG_ENABLED_IN_EDITOR) || (!bTargetIsEditor && NIAGARA_COMPUTEDEBUG_ENABLED_IN_COOKED);
	}
}

//////////////////////////////////////////////////////////////////////////

class FNiagaraVisualizeTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraVisualizeTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraVisualizeTexturePS, FGlobalShader);

	class FIntegerTexture : SHADER_PERMUTATION_BOOL("TEXTURE_INTEGER");
	class FTextureType : SHADER_PERMUTATION_INT("TEXTURE_TYPE", 4);

	using FPermutationDomain = TShaderPermutationDomain<FIntegerTexture, FTextureType>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return NiagaraDebugShaders::ShouldCompilePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector4,	NumTextureAttributes)
		SHADER_PARAMETER(int32,			NumAttributesToVisualize)
		SHADER_PARAMETER(FIntVector4,	AttributesToVisualize)
		SHADER_PARAMETER(FIntVector,	TextureDimensions)
		SHADER_PARAMETER(FVector4f,		PerChannelScale)
		SHADER_PARAMETER(FVector4f,		PerChannelBias)
		SHADER_PARAMETER(uint32,		DebugFlags)
		SHADER_PARAMETER(uint32,		TickCounter)
		SHADER_PARAMETER(uint32,		TextureSlice)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D,			Texture2DObject)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray,	Texture2DArrayObject)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D,			Texture3DObject)
		SHADER_PARAMETER_RDG_TEXTURE(TextureCube,		TextureCubeObject)
		SHADER_PARAMETER_SAMPLER(SamplerState,			TextureSampler)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraVisualizeTexturePS, "/Plugin/FX/Niagara/Private/NiagaraVisualizeTexture.usf", "Main", SF_Pixel);

//////////////////////////////////////////////////////////////////////////

class FNiagaraClearUAVCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraClearUAVCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraClearUAVCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 32;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_DEBUGDRAW_CLEARUAV_UINT_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector4,					ClearValue)
		SHADER_PARAMETER(uint32,						ClearSize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	BufferToClear)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraClearUAVCS, "/Plugin/FX/Niagara/Private/NiagaraDebugDraw.usf", "MainCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FNiagaraDebugDrawLineVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraDebugDrawLineVS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraDebugDrawLineVS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return NiagaraDebugShaders::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_DEBUGDRAW_DRAWLINE_VS"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GpuLineBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

class FNiagaraDebugDrawLinePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraDebugDrawLinePS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraDebugDrawLinePS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return NiagaraDebugShaders::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_DEBUGDRAW_DRAWLINE_PS"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, OutputInvResolution)
		SHADER_PARAMETER(FVector2f, OriginalViewRectMin)
		SHADER_PARAMETER(FVector2f, OriginalViewSize)
		SHADER_PARAMETER(FVector2f, OriginalBufferInvSize)
		SHADER_PARAMETER(float, OccludedColorScale)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraDebugDrawLineParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraDebugDrawLineVS::FParameters, VSParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraDebugDrawLinePS::FParameters, PSParameters)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraDebugDrawLineIndirectParameters, )
	RDG_BUFFER_ACCESS(AccessIndirectDrawArgsBuffer,	ERHIAccess::IndirectArgs)

	SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraDebugDrawLineVS::FParameters, VSParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraDebugDrawLinePS::FParameters, PSParameters)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER(FNiagaraDebugDrawLineVS, "/Plugin/FX/Niagara/Private/NiagaraDebugDraw.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FNiagaraDebugDrawLinePS, "/Plugin/FX/Niagara/Private/NiagaraDebugDraw.usf", "MainPS", SF_Pixel);

//////////////////////////////////////////////////////////////////////////

void NiagaraDebugShaders::ClearUAV(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef UAV, FUintVector4 ClearValues, uint32 UIntsToSet)
{
	check(UIntsToSet > 0);
	check(UAV != nullptr);

	TShaderMapRef<FNiagaraClearUAVCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp(UIntsToSet, FNiagaraClearUAVCS::ThreadGroupSize);

	FNiagaraClearUAVCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraClearUAVCS::FParameters>();
	PassParameters->BufferToClear = UAV;
	PassParameters->ClearValue = ClearValues;
	PassParameters->ClearSize = UIntsToSet;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NiagaraDebugShaders::ClearUAV"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FIntVector(NumThreadGroups, 1, 1)
	);
}

void NiagaraDebugShaders::DrawDebugLines(
	class FRDGBuilder& GraphBuilder, const class FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth,
	const uint32 LineInstanceCount, FRDGBufferRef LineBuffer
)
{
	const FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());

	TOptionalShaderMapRef<FNiagaraDebugDrawLineVS> VertexShader(ShaderMap);
	TOptionalShaderMapRef<FNiagaraDebugDrawLinePS> PixelShader(ShaderMap);
	if (VertexShader.IsNull() || PixelShader.IsNull())
	{
		return;
	}

	const FIntRect ViewRect = UE::FXRenderingUtils::GetRawViewRectUnsafe(View);

	FNiagaraDebugDrawLineParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraDebugDrawLineParameters>();
	PassParameters->VSParameters.View					= View.ViewUniformBuffer;
	PassParameters->VSParameters.GpuLineBuffer			= GraphBuilder.CreateSRV(LineBuffer, PF_R32_UINT);

	PassParameters->PSParameters.OutputInvResolution	= FVector2f(1.0f / View.UnscaledViewRect.Width(), 1.0f / View.UnconstrainedViewRect.Height());
	PassParameters->PSParameters.OriginalViewRectMin	= FVector2f(ViewRect.Min);
	PassParameters->PSParameters.OriginalViewSize		= FVector2f(ViewRect.Width(), ViewRect.Height());
	PassParameters->PSParameters.OriginalBufferInvSize	= FVector2f(1.f / SceneDepth->Desc.Extent.X, 1.f / SceneDepth->Desc.Extent.Y);
	PassParameters->PSParameters.OccludedColorScale		= GNiagaraGpuComputeDebug_OccludedLineColorScale;
	PassParameters->PSParameters.DepthTexture			= SceneDepth;
	PassParameters->PSParameters.DepthSampler			= TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->PSParameters.RenderTargets[0]		= FRenderTargetBinding(SceneColor, ERenderTargetLoadAction::ELoad);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("NiagaraDrawDebugLines"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, LineInstanceCount, PassParameters, ViewRect=View.UnscaledViewRect](FRHICommandListImmediate& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI(); // Premultiplied-alpha composition
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_LineList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VSParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PSParameters);
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);
			RHICmdList.DrawPrimitive(0, 2, LineInstanceCount);
		}
	);
}

void NiagaraDebugShaders::DrawDebugLines(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth, FRDGBufferRef ArgsBuffer, FRDGBufferRef LineBuffer)
{
	const FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());

	TOptionalShaderMapRef<FNiagaraDebugDrawLineVS> VertexShader(ShaderMap);
	TOptionalShaderMapRef<FNiagaraDebugDrawLinePS> PixelShader(ShaderMap);
	if (VertexShader.IsNull() || PixelShader.IsNull())
	{
		return;
	}

	const FIntRect ViewRect = UE::FXRenderingUtils::GetRawViewRectUnsafe(View);

	FNiagaraDebugDrawLineIndirectParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraDebugDrawLineIndirectParameters>();
	PassParameters->AccessIndirectDrawArgsBuffer		= ArgsBuffer;

	PassParameters->VSParameters.View					= View.ViewUniformBuffer;
	PassParameters->VSParameters.GpuLineBuffer			= GraphBuilder.CreateSRV(LineBuffer, PF_R32_UINT);

	PassParameters->PSParameters.OutputInvResolution	= FVector2f(1.0f / View.UnscaledViewRect.Width(), 1.0f / View.UnconstrainedViewRect.Height());
	PassParameters->PSParameters.OriginalViewRectMin	= FVector2f(ViewRect.Min);
	PassParameters->PSParameters.OriginalViewSize		= FVector2f(ViewRect.Width(), ViewRect.Height());
	PassParameters->PSParameters.OriginalBufferInvSize	= FVector2f(1.f / SceneDepth->Desc.Extent.X, 1.f / SceneDepth->Desc.Extent.Y);
	PassParameters->PSParameters.OccludedColorScale		= GNiagaraGpuComputeDebug_OccludedLineColorScale;
	PassParameters->PSParameters.DepthTexture			= SceneDepth;
	PassParameters->PSParameters.DepthSampler			= TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->PSParameters.RenderTargets[0]		= FRenderTargetBinding(SceneColor, ERenderTargetLoadAction::ELoad);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("NiagaraDrawDebugLines"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, ArgsBuffer, PassParameters, ViewRect=View.UnscaledViewRect](FRHICommandListImmediate& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI(); // Premultiplied-alpha composition
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_LineList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VSParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PSParameters);
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);
			RHICmdList.DrawPrimitiveIndirect(ArgsBuffer->GetIndirectRHICallBuffer(), 0);
		}
	);
}

void NiagaraDebugShaders::VisualizeTexture(
	class FRDGBuilder& GraphBuilder, const FSceneView& View, const FScreenPassRenderTarget& Output,
	const FIntPoint& Location, const int32& DisplayHeight,
	const FIntVector4& InAttributesToVisualize, FRDGTextureRef Texture, const FIntVector4& NumTextureAttributes, uint32 TickCounter,
	const FVector2D& PreviewDisplayRange
)
{
	FIntVector TextureSize = Texture->Desc.GetSize();
	if (NumTextureAttributes.X > 0)
	{
		check(NumTextureAttributes.Y > 0);
		TextureSize.X /= NumTextureAttributes.X;
		TextureSize.Y /= NumTextureAttributes.Y;
	}

	// Determine number of attributes to visualize
	int32 NumAttributesToVisualizeValue = 0;
	FIntVector4 AttributesToVisualize = InAttributesToVisualize;
	for (NumAttributesToVisualizeValue = 0; NumAttributesToVisualizeValue < 4; ++NumAttributesToVisualizeValue)
	{
		if (AttributesToVisualize[NumAttributesToVisualizeValue] == INDEX_NONE)
		{
			break;
		}
	}

	if (NumAttributesToVisualizeValue == 4)
	{
		switch (GNiagaraGpuComputeDebug_FourComponentMode)
		{
			// RGB only
			default:
			case 0:
				AttributesToVisualize[3] = INDEX_NONE;
				NumAttributesToVisualizeValue = 3;
				break;

			// Alpha only
			case 1:
				AttributesToVisualize[0] = AttributesToVisualize[3];
				AttributesToVisualize[1] = INDEX_NONE;
				AttributesToVisualize[2] = INDEX_NONE;
				AttributesToVisualize[3] = INDEX_NONE;
				NumAttributesToVisualizeValue = 1;
				break;
		}
	}

	// Set Shaders & State
	FNiagaraVisualizeTexturePS::FPermutationDomain PermutationVector;
	if (Texture->Desc.Dimension == ETextureDimension::Texture2D)
	{
		PermutationVector.Set<FNiagaraVisualizeTexturePS::FTextureType>(0);
	}
	else if (Texture->Desc.Dimension == ETextureDimension::Texture2DArray)
	{
		PermutationVector.Set<FNiagaraVisualizeTexturePS::FTextureType>(1);
		TextureSize.Z = NumAttributesToVisualizeValue == 0 ? Texture->Desc.ArraySize : 1;
	}
	else if (Texture->Desc.Dimension == ETextureDimension::Texture3D)
	{
		PermutationVector.Set<FNiagaraVisualizeTexturePS::FTextureType>(2);
	}
	else if (Texture->Desc.Dimension == ETextureDimension::TextureCube)
	{
		PermutationVector.Set<FNiagaraVisualizeTexturePS::FTextureType>(3);
		TextureSize.X *= 3;
	}
	else
	{
		// Should never get here, but let's not crash
		return;
	}

	switch (Texture->Desc.Format)
	{
		case PF_R32_UINT:
		case PF_R32_SINT:
		case PF_R16_UINT:
		case PF_R16_SINT:
		case PF_R16G16B16A16_UINT:
		case PF_R16G16B16A16_SINT:
			PermutationVector.Set<FNiagaraVisualizeTexturePS::FIntegerTexture>(true);
			break;
		default:
			PermutationVector.Set<FNiagaraVisualizeTexturePS::FIntegerTexture>(false);
			break;
	}

	auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
	TOptionalShaderMapRef<FNiagaraVisualizeTexturePS> PixelShader(ShaderMap, PermutationVector);
	if (PixelShader.IsNull())
	{
		return;
	}

	FIntPoint DisplaySize(TextureSize.X, TextureSize.Y);
	if (DisplayHeight > 0)
	{
		DisplaySize.Y = DisplayHeight;
		DisplaySize.X = int32(float(TextureSize.X) * (float(DisplaySize.Y) / float(TextureSize.Y)));
	}

	// Display slices
	const FIntPoint RenderTargetSize = View.Family->RenderTarget->GetSizeXY();

	const int32 AvailableWidth = RenderTargetSize.X - Location.X;
	const int32 SlicesWidth = FMath::Clamp(FMath::DivideAndRoundUp(AvailableWidth, DisplaySize.X + 1), 1, TextureSize.Z);

	const FVector2D::FReal DisplayScale = (PreviewDisplayRange.Y > PreviewDisplayRange.X) ? (1.0f / (PreviewDisplayRange.Y - PreviewDisplayRange.X)) : 1.0f;
	const FVector4f PerChannelScale(DisplayScale, DisplayScale, DisplayScale, DisplayScale);
	const FVector4f PerChannelBias(-PreviewDisplayRange.X, -PreviewDisplayRange.X, -PreviewDisplayRange.X, -PreviewDisplayRange.X);

	for (int32 iSlice = 0; iSlice < SlicesWidth; ++iSlice)
	{
		FNiagaraVisualizeTexturePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraVisualizeTexturePS::FParameters>();
		PassParameters->NumTextureAttributes		= NumTextureAttributes;
		PassParameters->NumAttributesToVisualize	= NumAttributesToVisualizeValue;
		PassParameters->AttributesToVisualize		= AttributesToVisualize;
		PassParameters->TextureDimensions			= TextureSize;
		PassParameters->PerChannelScale				= PerChannelScale;
		PassParameters->PerChannelBias				= PerChannelBias;
		PassParameters->DebugFlags					= GNiagaraGpuComputeDebug_ShowNaNInf != 0 ? 1 : 0;
		PassParameters->TickCounter					= TickCounter;
		PassParameters->TextureSlice				= iSlice;
		PassParameters->Texture2DObject				= Texture;
		PassParameters->Texture2DArrayObject		= Texture;
		PassParameters->Texture3DObject				= Texture;
		PassParameters->TextureCubeObject			= Texture;
		PassParameters->TextureSampler				= TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->RenderTargets[0]			= Output.GetRenderTargetBinding();

		const float OffsetX = float(iSlice) * DisplaySize.X + 1;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("NiagaraVisualizeTexture"),
			PassParameters,
			ERDGPassFlags::Raster,
			[VertexShader, PixelShader, PassParameters, Location=FIntPoint(Location.X+OffsetX, Location.Y), DisplaySize, TextureSize, RenderTargetSize](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.SetViewport(0, 0, 0.0f, RenderTargetSize.X, RenderTargetSize.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>("Renderer");
				RendererModule->DrawRectangle(
					RHICmdList,
					Location.X, Location.Y,						// Dest X, Y
					DisplaySize.X, DisplaySize.Y,				// Dest Width, Height
					0.0f, 0.0f,									// Source U, V
					TextureSize.X, TextureSize.Y,				// Source USize, VSize
					RenderTargetSize,							// TargetSize
					FIntPoint(TextureSize.X, TextureSize.Y),	// Source texture size
					VertexShader,
					EDRF_Default);
			}
		);
	}
}
