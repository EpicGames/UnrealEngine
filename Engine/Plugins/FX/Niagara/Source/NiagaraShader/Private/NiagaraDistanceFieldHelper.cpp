// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDistanceFieldHelper.h"
#include "GlobalDistanceFieldParameters.h"
#include "TextureFallbacks.h"

// todo - currently duplicated from SetupGlobalDistanceFieldParameters (GlobalDistanceField.cpp) because of problems getting it properly exported from the dll
void FNiagaraDistanceFieldHelper::SetGlobalDistanceFieldParameters(const FGlobalDistanceFieldParameterData* OptionalParameterData, FGlobalDistanceFieldParameters2& ShaderParameters)
{
	if (OptionalParameterData != nullptr)
	{
		ShaderParameters.GlobalDistanceFieldPageAtlasTexture = OrBlack3DIfNull(OptionalParameterData->PageAtlasTexture);
		ShaderParameters.GlobalDistanceFieldCoverageAtlasTexture = OrBlack3DIfNull(OptionalParameterData->CoverageAtlasTexture);
		ShaderParameters.GlobalDistanceFieldPageTableTexture = OrBlack3DUintIfNull(OptionalParameterData->PageTableTexture);
		ShaderParameters.GlobalDistanceFieldMipTexture = OrBlack3DIfNull(OptionalParameterData->MipTexture);

		for (int32 Index = 0; Index < GlobalDistanceField::MaxClipmaps; Index++)
		{
			ShaderParameters.GlobalVolumeTranslatedCenterAndExtent[Index] = OptionalParameterData->TranslatedCenterAndExtent[Index];
			ShaderParameters.GlobalVolumeTranslatedWorldToUVAddAndMul[Index] = OptionalParameterData->TranslatedWorldToUVAddAndMul[Index];
			ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVScale[Index] = OptionalParameterData->MipTranslatedWorldToUVScale[Index];
			ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVBias[Index] = OptionalParameterData->MipTranslatedWorldToUVBias[Index];
		}

		ShaderParameters.GlobalDistanceFieldMipFactor = OptionalParameterData->MipFactor;
		ShaderParameters.GlobalDistanceFieldMipTransition = OptionalParameterData->MipTransition;
		ShaderParameters.GlobalDistanceFieldClipmapSizeInPages = OptionalParameterData->ClipmapSizeInPages;
		ShaderParameters.GlobalDistanceFieldInvPageAtlasSize = (FVector3f)OptionalParameterData->InvPageAtlasSize;
		ShaderParameters.GlobalDistanceFieldInvCoverageAtlasSize = (FVector3f)OptionalParameterData->InvCoverageAtlasSize;
		ShaderParameters.GlobalVolumeDimension = OptionalParameterData->GlobalDFResolution;
		ShaderParameters.GlobalVolumeTexelSize = FMath::IsNearlyZero(OptionalParameterData->GlobalDFResolution) ? 0.0f : 1.0f / OptionalParameterData->GlobalDFResolution;
		ShaderParameters.MaxGlobalDFAOConeDistance = OptionalParameterData->MaxDFAOConeDistance;
		ShaderParameters.NumGlobalSDFClipmaps = OptionalParameterData->NumGlobalSDFClipmaps;
	}
	else
	{
		ShaderParameters.GlobalDistanceFieldPageAtlasTexture = GBlackVolumeTexture->TextureRHI;
		ShaderParameters.GlobalDistanceFieldCoverageAtlasTexture = GBlackVolumeTexture->TextureRHI;
		ShaderParameters.GlobalDistanceFieldPageTableTexture = GBlackUintVolumeTexture->TextureRHI;
		ShaderParameters.GlobalDistanceFieldMipTexture = GBlackVolumeTexture->TextureRHI;

		for (int32 Index = 0; Index < GlobalDistanceField::MaxClipmaps; Index++)
		{
			ShaderParameters.GlobalVolumeTranslatedCenterAndExtent[Index] = FVector4f::Zero();
			ShaderParameters.GlobalVolumeTranslatedWorldToUVAddAndMul[Index] = FVector4f::Zero();
			ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVScale[Index] = FVector4f::Zero();
			ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVBias[Index] = FVector4f::Zero();
		}

		ShaderParameters.GlobalDistanceFieldMipFactor = 0.0f;
		ShaderParameters.GlobalDistanceFieldMipTransition = 0.0f;
		ShaderParameters.GlobalDistanceFieldClipmapSizeInPages = 0;
		ShaderParameters.GlobalDistanceFieldInvPageAtlasSize = FVector3f::ZeroVector;
		ShaderParameters.GlobalDistanceFieldInvCoverageAtlasSize = FVector3f::ZeroVector;
		ShaderParameters.GlobalVolumeDimension = 0.0f;
		ShaderParameters.GlobalVolumeTexelSize = 0.0f;
		ShaderParameters.MaxGlobalDFAOConeDistance = 0.0f;
		ShaderParameters.NumGlobalSDFClipmaps = 0;
	}

	ShaderParameters.CoveredExpandSurfaceScale = 0.0f;
	ShaderParameters.NotCoveredExpandSurfaceScale = 0.0f;
	ShaderParameters.NotCoveredMinStepScale = 0.0f;
	ShaderParameters.DitheredTransparencyStepThreshold = 0.0f;
	ShaderParameters.DitheredTransparencyTraceThreshold = 0.0f;
	ShaderParameters.GlobalDistanceFieldCoverageAtlasTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	ShaderParameters.GlobalDistanceFieldPageAtlasTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	ShaderParameters.GlobalDistanceFieldMipTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}
