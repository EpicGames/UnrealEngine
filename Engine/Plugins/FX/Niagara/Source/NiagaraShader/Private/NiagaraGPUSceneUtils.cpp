// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGPUSceneUtils.h"
#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderUtils.h"

class FNiagaraUpdateMeshGPUSceneInstancesCS : public FGlobalShader
{
public:
	static constexpr uint32 ThreadGroupSize = 64;

	DECLARE_GLOBAL_SHADER(FNiagaraUpdateMeshGPUSceneInstancesCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraUpdateMeshGPUSceneInstancesCS, FGlobalShader);

	using FParameters = FNiagaraGPUSceneUtils::FUpdateMeshParticleInstancesParams;

	class FPreciseMotionVectors : SHADER_PERMUTATION_BOOL("ENABLE_PRECISE_MOTION_VECTORS");
	using FPermutationDomain = TShaderPermutationDomain<FPreciseMotionVectors>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UseGPUScene(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), ThreadGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraUpdateMeshGPUSceneInstancesCS, "/Plugin/FX/Niagara/Private/NiagaraUpdateMeshGPUSceneInstances.usf", "UpdateMeshInstancesCS", SF_Compute);

void FNiagaraGPUSceneUtils::AddUpdateMeshParticleInstancesPass(
	FRDGBuilder& GraphBuilder, 
	FUpdateMeshParticleInstancesParams& Params,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bPreciseMotionVectors
)
{
	FNiagaraUpdateMeshGPUSceneInstancesCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FNiagaraUpdateMeshGPUSceneInstancesCS::FPreciseMotionVectors>(bPreciseMotionVectors);

	TShaderMapRef<FNiagaraUpdateMeshGPUSceneInstancesCS> UpdateMeshInstancesCS(GetGlobalShaderMap(FeatureLevel), PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Niagara.UpdateMeshParticleInstances (%u Instances)", Params.ParticleCount),
		UpdateMeshInstancesCS,
		&Params,
		FComputeShaderUtils::GetGroupCountWrapped(Params.ParticleCount, FNiagaraUpdateMeshGPUSceneInstancesCS::ThreadGroupSize)
	);
}
