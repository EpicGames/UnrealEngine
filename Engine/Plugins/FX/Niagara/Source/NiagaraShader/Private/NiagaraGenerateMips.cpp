// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGenerateMips.h"
#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#if PLATFORM_ANDROID
#include "DataDrivenShaderPlatformInfo.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraGenerateMips)

class FNiagaraGenerateMipsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FNiagaraGenerateMipsCS)
	SHADER_USE_PARAMETER_STRUCT(FNiagaraGenerateMipsCS, FGlobalShader)

	class FGenMipsSRGB : SHADER_PERMUTATION_BOOL("GENMIPS_SRGB");
	class FGaussianBlur : SHADER_PERMUTATION_BOOL("GENMIPS_GAUSSIAN");
	using FPermutationDomain = TShaderPermutationDomain<FGenMipsSRGB, FGaussianBlur>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, SrcTexelSize)
		SHADER_PARAMETER(FVector2f, DstTexelSize)
		SHADER_PARAMETER(int32, KernelHWidth)
		SHADER_PARAMETER(FIntPoint, MipOutSize)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, MipOutUAV)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MipInSRV)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipInSampler)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE"), FComputeShaderUtils::kGolden2DGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraGenerateMipsCS, "/Plugin/FX/Niagara/Private/NiagaraGenerateMips.usf", "MainCS", SF_Compute);

void NiagaraGenerateMips::GenerateMips(FRDGBuilder& GraphBuilder, FRDGTextureRef RDGTexture, ENiagaraMipMapGenerationType GenType)
{
	RDG_RHI_EVENT_SCOPE(GraphBuilder, NiagaraGenerateMips);

	const FRDGTextureDesc& TextureDesc = RDGTexture->Desc;
	check(TextureDesc.Dimension == ETextureDimension::Texture2D);

	// Select compute shader variant (normal vs. sRGB etc.)
	bool bMipsSRGB = EnumHasAnyFlags(TextureDesc.Flags, TexCreate_SRGB);
#if PLATFORM_ANDROID
	if (IsVulkanPlatform(GMaxRHIShaderPlatform))
	{
		// Vulkan Android seems to skip sRGB->Lin conversion when sampling texture in compute
		bMipsSRGB = false;
	}
#endif

	const bool bBlurMips = GenType >= ENiagaraMipMapGenerationType::Blur1;

	FNiagaraGenerateMipsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FNiagaraGenerateMipsCS::FGenMipsSRGB>(bMipsSRGB);
	PermutationVector.Set<FNiagaraGenerateMipsCS::FGaussianBlur>(bBlurMips);
	TShaderMapRef<FNiagaraGenerateMipsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

	const int32 KernelHWidth = bBlurMips ? (int32(GenType) + 1 - int32(ENiagaraMipMapGenerationType::Blur1)) : 1;

	for ( int32 iDstMip=1; iDstMip < TextureDesc.NumMips; ++iDstMip)
	{
		const int32 iSrcMip = iDstMip - 1;
		const FIntPoint SrcMipSize(FMath::Max<int32>(TextureDesc.GetSize().X >> iSrcMip, 1), FMath::Max<int32>(TextureDesc.GetSize().Y >> iSrcMip, 1));
		const FIntPoint DstMipSize(FMath::Max<int32>(TextureDesc.GetSize().X >> iDstMip, 1), FMath::Max<int32>(TextureDesc.GetSize().Y >> iDstMip, 1));

		FNiagaraGenerateMipsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraGenerateMipsCS::FParameters>();
		PassParameters->SrcTexelSize	= FVector2f(1.0f / float(SrcMipSize.X), 1.0f / float(SrcMipSize.Y));
		PassParameters->DstTexelSize	 = FVector2f(1.0f / float(DstMipSize.X), 1.0f / float(DstMipSize.Y));
		PassParameters->KernelHWidth	 = KernelHWidth;
		PassParameters->MipOutSize		= DstMipSize;
		PassParameters->MipInSRV		= GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(RDGTexture, iSrcMip));
		PassParameters->MipInSampler	= GenType == ENiagaraMipMapGenerationType::Unfiltered ? TStaticSamplerState<SF_Point>::GetRHI() : TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->MipOutUAV		= GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RDGTexture, iDstMip));

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			{},
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(DstMipSize, FComputeShaderUtils::kGolden2DGroupSize)
		);
	}
}

