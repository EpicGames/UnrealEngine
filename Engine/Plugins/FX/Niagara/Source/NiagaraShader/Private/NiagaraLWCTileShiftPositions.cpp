// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraLWCTileShiftPositions.h"

#include "RenderGraphUtils.h"

IMPLEMENT_GLOBAL_SHADER(FNiagaraLWCTileShiftPositionsCS, "/Plugin/FX/Niagara/Private/NiagaraLWCHelper.usf", "TileShiftPositions", SF_Compute);

void FNiagaraLWCTileShiftPositionsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	OutEnvironment.SetDefine(TEXT("MaxPositions"), MaxPositions);
}

void FNiagaraLWCTileShiftPositionsCS::Execute(FRHIComputeCommandList& RHICmdList, const FParameters& Parameters)
{
	TShaderMapRef<FNiagaraLWCTileShiftPositionsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(Parameters.NumInstances, ThreadGroupSize);
	const FIntVector NumWrappedThreadGroups = FComputeShaderUtils::GetGroupCountWrapped(NumThreadGroups);
	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, NumWrappedThreadGroups);
}
