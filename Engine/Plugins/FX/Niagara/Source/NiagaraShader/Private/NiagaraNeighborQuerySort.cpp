// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNeighborQuerySort.h"

#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"

//////////////////////////////////////////////////////////////////////////
// Pass 2: Histogram

class FNiagaraNeighborQueryHistogramCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraNeighborQueryHistogramCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraNeighborQueryHistogramCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NEIGHBOR_QUERY_HISTOGRAM_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("NQ_DIST_BITS"), NiagaraNeighborQuerySort::NQDistBits);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	CellIdBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	CellCountBuffer)
		SHADER_PARAMETER(uint32,						NumSlots)
		SHADER_PARAMETER(uint32,						GroupsX)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraNeighborQueryHistogramCS, "/Plugin/FX/Niagara/Private/NiagaraNeighborQuerySort.usf", "HistogramCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////
// Pass 3: Parallel Prefix Sum (recursive reduce-then-scan)
//
// Two shader sub-passes per recursion level:
//   3a. LocalScan  — each group scans a tile, writes tile sum to BlockSumsBuffer
//   3b. Propagate  — each group adds its scanned block prefix to local results
//
// Block sums are scanned by recursively applying the same algorithm.

namespace PrefixSumConstants
{
	static constexpr uint32 ThreadGroupSize = 128;
	static constexpr uint32 ElementsPerThread = 4;
	static constexpr uint32 ElementsPerGroup = ThreadGroupSize * ElementsPerThread; // 512
}

class FNiagaraNeighborQueryPrefixSumLocalScanCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraNeighborQueryPrefixSumLocalScanCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraNeighborQueryPrefixSumLocalScanCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NEIGHBOR_QUERY_PREFIX_SUM_LOCAL_SCAN_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), PrefixSumConstants::ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("ELEMENTS_PER_THREAD"), PrefixSumConstants::ElementsPerThread);
		OutEnvironment.SetDefine(TEXT("ELEMENTS_PER_GROUP"), PrefixSumConstants::ElementsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	InputBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	BlockSumsBuffer)
		SHADER_PARAMETER(uint32,						NumElements)
		SHADER_PARAMETER(uint32,						GroupsX)
		SHADER_PARAMETER(uint32,						NumGroups)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraNeighborQueryPrefixSumLocalScanCS, "/Plugin/FX/Niagara/Private/NiagaraNeighborQuerySort.usf", "PrefixSumLocalScanCS", SF_Compute);

class FNiagaraNeighborQueryPrefixSumPropagateCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraNeighborQueryPrefixSumPropagateCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraNeighborQueryPrefixSumPropagateCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NEIGHBOR_QUERY_PREFIX_SUM_PROPAGATE_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), PrefixSumConstants::ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("ELEMENTS_PER_THREAD"), PrefixSumConstants::ElementsPerThread);
		OutEnvironment.SetDefine(TEXT("ELEMENTS_PER_GROUP"), PrefixSumConstants::ElementsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	BlockSumsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputBuffer)
		SHADER_PARAMETER(uint32,						NumElements)
		SHADER_PARAMETER(uint32,						GroupsX)
		SHADER_PARAMETER(uint32,						NumGroups)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraNeighborQueryPrefixSumPropagateCS, "/Plugin/FX/Niagara/Private/NiagaraNeighborQuerySort.usf", "PrefixSumPropagateCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////
// Pass 4: Scatter

class FNiagaraNeighborQueryScatterCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraNeighborQueryScatterCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraNeighborQueryScatterCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NEIGHBOR_QUERY_SCATTER_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("NQ_DIST_BITS"), NiagaraNeighborQuerySort::NQDistBits);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	CellIdBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	ParticleIdIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	CellOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	ParticleListBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	AcquireTagBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	AcquireTagListBuffer)
		SHADER_PARAMETER(uint32,						NumSlots)
		SHADER_PARAMETER(uint32,						MaxCellsPerParticle)
		SHADER_PARAMETER(uint32,						GroupsX)
		SHADER_PARAMETER(uint32,						bHasPersistentIDBuffers)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraNeighborQueryScatterCS, "/Plugin/FX/Niagara/Private/NiagaraNeighborQuerySort.usf", "ScatterCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////
// Dispatch functions

// Split a 1D group count into a 2D dispatch to stay within MaxDispatchThreadGroupsPerDimension (65535).
static FIntVector GetDispatchSize(uint32 NumGroups)
{
	if (NumGroups <= 65535)
	{
		return FIntVector(NumGroups, 1, 1);
	}
	const uint32 GroupsX = FMath::DivideAndRoundUp(NumGroups, (uint32)FMath::CeilToInt32(FMath::Sqrt((float)NumGroups)));
	const uint32 GroupsY = FMath::DivideAndRoundUp(NumGroups, GroupsX);
	return FIntVector(GroupsX, GroupsY, 1);
}

void NiagaraNeighborQuerySort::Histogram(FRDGBuilder& GraphBuilder, FRDGBufferSRVRef CellIdSRV, FRDGBufferUAVRef CellCountUAV, uint32 NumSlots)
{
	if (NumSlots == 0)
	{
		return;
	}

	TShaderMapRef<FNiagaraNeighborQueryHistogramCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp(NumSlots, FNiagaraNeighborQueryHistogramCS::ThreadGroupSize);
	const FIntVector DispatchSize = GetDispatchSize(NumThreadGroups);

	FNiagaraNeighborQueryHistogramCS::FParameters* Parameters = GraphBuilder.AllocParameters<FNiagaraNeighborQueryHistogramCS::FParameters>();
	Parameters->CellIdBuffer = CellIdSRV;
	Parameters->CellCountBuffer = CellCountUAV;
	Parameters->NumSlots = NumSlots;
	Parameters->GroupsX = DispatchSize.X;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NiagaraNeighborQuerySort::Histogram NumSlots(%u) DispatchCount(%ux%ux1) NumThreads(%ux1x1)", NumSlots, DispatchSize.X, DispatchSize.Y, FNiagaraNeighborQueryHistogramCS::ThreadGroupSize),
		ERDGPassFlags::Compute,
		ComputeShader,
		Parameters,
		DispatchSize
	);
}

// Internal recursive prefix sum implementation.
// InputBuffer is read as SRV, results written to OutputUAV.
// Recurses on block sums when NumElements > ElementsPerGroup.
static void PrefixSumRecursive(FRDGBuilder& GraphBuilder, FRDGBufferRef InputBuffer, FRDGBufferUAVRef OutputUAV, uint32 NumElements)
{
	if (NumElements == 0)
	{
		return;
	}

	using namespace PrefixSumConstants;

	const uint32 NumGroups = FMath::DivideAndRoundUp(NumElements, ElementsPerGroup);

	// Single tile — scan directly, no block sums needed
	if (NumGroups == 1)
	{
		TShaderMapRef<FNiagaraNeighborQueryPrefixSumLocalScanCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		// Create a dummy 1-element block sums buffer (required by shader signature)
		FRDGBufferRef DummyBlockSums = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), 1), TEXT("NiagaraNeighborQuerySort::DummyBlockSums"));

		auto* Parameters = GraphBuilder.AllocParameters<FNiagaraNeighborQueryPrefixSumLocalScanCS::FParameters>();
		Parameters->InputBuffer = GraphBuilder.CreateSRV(InputBuffer, PF_R32_SINT);
		Parameters->OutputBuffer = OutputUAV;
		Parameters->BlockSumsBuffer = GraphBuilder.CreateUAV(DummyBlockSums, PF_R32_SINT);
		Parameters->NumElements = NumElements;
		Parameters->GroupsX = 1;
		Parameters->NumGroups = 1;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NiagaraNeighborQuerySort::PrefixSum NumElements(%u) DispatchCount(1x1x1) NumThreads(%ux1x1)", NumElements, ThreadGroupSize),
			ERDGPassFlags::Compute,
			ComputeShader,
			Parameters,
			FIntVector(1, 1, 1)
		);
		return;
	}

	// Multi-tile: reduce-then-scan with recursive block sum scanning
	const FIntVector LocalScanDispatch = GetDispatchSize(NumGroups);

	FRDGBufferRef BlockSumsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumGroups), TEXT("NiagaraNeighborQuerySort::BlockSums"));

	// LocalScan — each group scans its tile, writes tile total to BlockSumsBuffer
	{
		TShaderMapRef<FNiagaraNeighborQueryPrefixSumLocalScanCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		auto* Parameters = GraphBuilder.AllocParameters<FNiagaraNeighborQueryPrefixSumLocalScanCS::FParameters>();
		Parameters->InputBuffer = GraphBuilder.CreateSRV(InputBuffer, PF_R32_SINT);
		Parameters->OutputBuffer = OutputUAV;
		Parameters->BlockSumsBuffer = GraphBuilder.CreateUAV(BlockSumsBuffer, PF_R32_SINT);
		Parameters->NumElements = NumElements;
		Parameters->GroupsX = LocalScanDispatch.X;
		Parameters->NumGroups = NumGroups;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NiagaraNeighborQuerySort::PrefixSum::LocalScan NumElements(%u) DispatchCount(%ux%ux1) NumThreads(%ux1x1)", NumElements, LocalScanDispatch.X, LocalScanDispatch.Y, ThreadGroupSize),
			ERDGPassFlags::Compute,
			ComputeShader,
			Parameters,
			LocalScanDispatch
		);
	}

	// Recursively scan the block sums
	FRDGBufferRef ScannedBlockSums = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumGroups), TEXT("NiagaraNeighborQuerySort::ScannedBlockSums"));
	FRDGBufferUAVRef ScannedBlockSumsUAV = GraphBuilder.CreateUAV(ScannedBlockSums, PF_R32_SINT);
	PrefixSumRecursive(GraphBuilder, BlockSumsBuffer, ScannedBlockSumsUAV, NumGroups);

	// Propagate — each group adds its scanned block prefix to local results
	{
		TShaderMapRef<FNiagaraNeighborQueryPrefixSumPropagateCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		const FIntVector PropagateDispatch = GetDispatchSize(NumGroups);

		auto* Parameters = GraphBuilder.AllocParameters<FNiagaraNeighborQueryPrefixSumPropagateCS::FParameters>();
		Parameters->BlockSumsBuffer = GraphBuilder.CreateSRV(ScannedBlockSums, PF_R32_SINT);
		Parameters->OutputBuffer = OutputUAV;
		Parameters->NumElements = NumElements;
		Parameters->GroupsX = PropagateDispatch.X;
		Parameters->NumGroups = NumGroups;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NiagaraNeighborQuerySort::PrefixSum::Propagate NumElements(%u) DispatchCount(%ux%ux1) NumThreads(%ux1x1)", NumElements, PropagateDispatch.X, PropagateDispatch.Y, ThreadGroupSize),
			ERDGPassFlags::Compute,
			ComputeShader,
			Parameters,
			PropagateDispatch
		);
	}
}

void NiagaraNeighborQuerySort::PrefixSum(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef CellCountUAV, FRDGBufferUAVRef CellOffsetUAV, uint32 NumCells)
{
	PrefixSumRecursive(GraphBuilder, CellCountUAV->GetParent(), CellOffsetUAV, NumCells);
}

void NiagaraNeighborQuerySort::Scatter(FRDGBuilder& GraphBuilder, FRDGBufferSRVRef CellIdSRV, FRDGBufferSRVRef ParticleIdIndexSRV, FRDGBufferUAVRef CellOffsetUAV, FRDGBufferUAVRef ParticleListUAV, FRDGBufferSRVRef AcquireTagSRV, FRDGBufferUAVRef AcquireTagListUAV, uint32 NumSlots, uint32 MaxCellsPerParticle, bool bHasPersistentIDBuffers)
{
	if (NumSlots == 0)
	{
		return;
	}

	TShaderMapRef<FNiagaraNeighborQueryScatterCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp(NumSlots, FNiagaraNeighborQueryScatterCS::ThreadGroupSize);
	const FIntVector DispatchSize = GetDispatchSize(NumThreadGroups);

	FNiagaraNeighborQueryScatterCS::FParameters* Parameters = GraphBuilder.AllocParameters<FNiagaraNeighborQueryScatterCS::FParameters>();
	Parameters->CellIdBuffer = CellIdSRV;
	Parameters->ParticleIdIndexBuffer = ParticleIdIndexSRV;
	Parameters->CellOffsetBuffer = CellOffsetUAV;
	Parameters->ParticleListBuffer = ParticleListUAV;
	Parameters->AcquireTagBuffer = AcquireTagSRV;
	Parameters->AcquireTagListBuffer = AcquireTagListUAV;
	Parameters->NumSlots = NumSlots;
	Parameters->MaxCellsPerParticle = MaxCellsPerParticle;
	Parameters->GroupsX = DispatchSize.X;
	Parameters->bHasPersistentIDBuffers = bHasPersistentIDBuffers ? 1u : 0u;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NiagaraNeighborQuerySort::Scatter NumSlots(%u) DispatchCount(%ux%ux1) NumThreads(%ux1x1)", NumSlots, DispatchSize.X, DispatchSize.Y, FNiagaraNeighborQueryScatterCS::ThreadGroupSize),
		ERDGPassFlags::Compute,
		ComputeShader,
		Parameters,
		DispatchSize
	);
}
