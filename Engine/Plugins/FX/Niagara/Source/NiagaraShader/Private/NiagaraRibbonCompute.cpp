// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraRibbonCompute.h"
#include "ShaderCompilerCore.h"

constexpr uint32 FNiagaraRibbonComputeCommon::VertexGenReductionInitializationThreadSize;
constexpr uint32 FNiagaraRibbonComputeCommon::VertexGenReductionPropagationThreadSize;
constexpr uint32 FNiagaraRibbonComputeCommon::VertexGenReductionFinalizationThreadSize;
constexpr uint32 FNiagaraRibbonComputeCommon::VertexGenFinalizationThreadSize;
constexpr uint32 FNiagaraRibbonComputeCommon::IndexGenThreadSize;


void FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment, int32 ThreadGroupSize)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	//OutEnvironment.SetDefine(TEXT("NiagaraVFLooseParameters"), TEXT("NiagaraRibbonVFLooseParameters"));
	OutEnvironment.SetDefine(TEXT("GPU_SIMULATION"), true);
	if (ThreadGroupSize > 0)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	// Precompile all NiagaraRibbon shaders with DXC, because FXC does not support the enum type in NiagaraRibbonShared.h
	OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
}

void FNiagaraRibbonSortPhase1CS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, BubbleSortGroupWidth);
}

void FNiagaraRibbonSortPhase2CS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, ThreadGroupSize);
}


void FNiagaraRibbonVertexReductionInitializationCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, FNiagaraRibbonComputeCommon::VertexGenReductionInitializationThreadSize);
}

void FNiagaraRibbonVertexReductionFinalizeCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, FNiagaraRibbonComputeCommon::VertexGenReductionFinalizationThreadSize);
}

void FNiagaraRibbonUVParamCalculationCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, FNiagaraRibbonComputeCommon::VertexGenFinalizationThreadSize);
}

void FNiagaraRibbonCreateIndexBufferParamsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, 1);
}

void FNiagaraRibbonCreateIndexBufferCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, FNiagaraRibbonComputeCommon::IndexGenThreadSize);


	// The optimal size is split in half for half above/half below geometry center using two
	// separate loops, and then in half again as the loops handle 2 vertices per slice
	
	const uint32 LoopUnrollSize = FNiagaraRibbonComputeCommon::IndexGenOptimalLoopVertexLimit / 2 / 2;
	
	OutEnvironment.SetDefine(TEXT("LOOP_UNROLL_SIZE"), LoopUnrollSize);
	
}

void FNiagaraRibbonAggregationStepCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, FNiagaraRibbonComputeCommon::VertexGenReductionInitializationThreadSize);
}

void FNiagaraRibbonAggregationApplyCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	FNiagaraRibbonComputeCommon::ModifyCompilationEnvironment(Parameters, OutEnvironment, FNiagaraRibbonComputeCommon::VertexGenReductionInitializationThreadSize);
}

IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonSortPhase1CS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonSortParticles.usf", "InitialSortList", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonSortPhase2CS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonSortParticles.usf", "MergeSort", SF_Compute);

IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonVertexReductionInitializationCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonVertexReductionInitialization.usf", "VertexGenInitialize", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonVertexReductionFinalizeCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonVertexReductionFinalization.usf", "OutputRibbonStats", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonUVParamCalculationCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonRibbonUVParamCalculation.usf", "GenerateRibbonUVParams", SF_Compute);

IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonCreateIndexBufferParamsCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonInitializeIndices.usf", "InitializeIndices", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonCreateIndexBufferCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonGenerateIndices.usf", "GenerateIndices", SF_Compute);

IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonAggregationStepCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonAggregationStep.usf", "RibbonAggregationStep", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraRibbonAggregationApplyCS, "/Plugin/FX/Niagara/Private/Ribbons/NiagaraRibbonAggregationApply.usf", "RibbonAggregationApply", SF_Compute);
