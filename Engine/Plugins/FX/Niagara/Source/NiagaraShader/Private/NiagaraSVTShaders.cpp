// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSVTShaders.h"

void FNiagaraCopySVTToDenseBufferCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

IMPLEMENT_GLOBAL_SHADER(FNiagaraCopySVTToDenseBufferCS, "/Plugin/FX/Niagara/Private/NiagaraSVTToDenseBuffer.usf", "PerformCopyCS", SF_Compute);

void FNiagaraBlendSVTsToDenseBufferCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

bool FNiagaraBlendSVTsToDenseBufferCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
}

IMPLEMENT_GLOBAL_SHADER(FNiagaraBlendSVTsToDenseBufferCS, "/Plugin/FX/Niagara/Private/NiagaraBlendSVTsToDenseBuffer.usf", "PerformBlendCS", SF_Compute);
