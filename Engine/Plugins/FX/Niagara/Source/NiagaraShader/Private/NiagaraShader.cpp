// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraShader.h"
#include "NiagaraSettings.h"
#include "NiagaraShared.h"
#include "NiagaraScriptBase.h"
#include "RenderingThread.h"
#include "Stats/StatsMisc.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "ShaderSerialization.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "UObject/DevObjectVersion.h"
#include "NiagaraShaderCompilationManager.h"
#if WITH_EDITOR
	#include "Interfaces/ITargetPlatformManagerModule.h"
	#include "DerivedDataCacheInterface.h"
	#include "DerivedDataCacheKey.h"
	#include "Interfaces/ITargetPlatformManagerModule.h"
#endif
#include "ProfilingDebugging/CookStats.h"

#include "NiagaraDataInterfaceBase.h"
#include "UObject/UObjectGlobals.h"
#include "NiagaraShaderModule.h"
#include "NiagaraCustomVersion.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "UObject/UObjectThreadContext.h"

IMPLEMENT_SHADER_TYPE(, FNiagaraShader, TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf"),TEXT("SimulateMain"), SF_Compute)

int32 GCreateNiagaraShadersOnLoad = 0;
static FAutoConsoleVariableRef CVarCreateNiagaraShadersOnLoad(
	TEXT("niagara.CreateShadersOnLoad"),
	GCreateNiagaraShadersOnLoad,
	TEXT("Whether to create Niagara's simulation shaders on load, which can reduce hitching, but use more memory.  Otherwise they will be created as needed.")
);

int32 GNiagaraSkipVectorVMBackendOptimizations = 1;
static FAutoConsoleVariableRef CVarNiagaraSkipVectorVMBackendOptimizations(
	TEXT("fx.SkipVectorVMBackendOptimizations"),
	GNiagaraSkipVectorVMBackendOptimizations,
	TEXT("If 1, skip HLSLCC's backend optimization passes during VectorVM compilation. \n"),
	ECVF_Default
);

int32 GNiagaraShaderForceBindEverything = 0;
static FAutoConsoleVariableRef CVarNiagaraShaderForceBindEverything(
	TEXT("fx.Niagara.Shader.ForceBindEverything"),
	GNiagaraShaderForceBindEverything,
	TEXT("Forces Niagara to display errors about missing shader bindings.")
);

bool GNiagaraEnableHLSL2021 = false;
static FAutoConsoleVariableRef CVarNiagaraEnableHLSL2021(
	TEXT("fx.Niagara.Shader.EnableHLSL2021"),
	GNiagaraEnableHLSL2021,
	TEXT("Enable HLSL 2021 compilation flag.\n")
	TEXT("Note: Only GPU simulations on some platforms will work with HLSL 2021.")
);

#if ENABLE_COOK_STATS
namespace NiagaraShaderCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static int32 ShadersCompiled = 0;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("NiagaraShader.Usage"), TEXT(""));
		AddStat(TEXT("NiagaraShader.Misc"), FCookStatsManager::CreateKeyValueArray(
			TEXT("ShadersCompiled"), ShadersCompiled
			));
	});
}
#endif


//
// Globals
//
FCriticalSection GIdToNiagaraShaderMapCS;
TMap<FNiagaraShaderMapId, FNiagaraShaderMap*> FNiagaraShaderMap::GIdToNiagaraShaderMap[SP_NumPlatforms];

#if WITH_EDITOR
TMap<FNiagaraShaderMapRef, TArray<FNiagaraShaderScript*>> FNiagaraShaderMap::NiagaraShaderMapsBeingCompiled;

static inline bool ShouldCacheNiagaraShader(const FNiagaraShaderType* ShaderType, EShaderPlatform Platform, const FNiagaraShaderScript* Script)
{
	return ShaderType->ShouldCache(Platform, Script) && Script->ShouldCache(Platform, ShaderType);
}
#endif //WITH_EDITOR


/** Called for every script shader to update the appropriate stats. */
void UpdateNiagaraShaderCompilingStats(const FNiagaraShaderScript* Script)
{
	INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumTotalNiagaraShaders,1);
}

int32 FNiagaraShaderMapPointerTable::AddIndexedPointer(const FTypeLayoutDesc& TypeDesc, void* Ptr)
{
	int32 Index = INDEX_NONE;
	if (DITypes.TryAddIndexedPtr(TypeDesc, Ptr, Index)) return Index;
	return Super::AddIndexedPointer(TypeDesc, Ptr);
}

void* FNiagaraShaderMapPointerTable::GetIndexedPointer(const FTypeLayoutDesc& TypeDesc, uint32 i) const
{
	void* Ptr = nullptr;
	if (DITypes.TryGetIndexedPtr(TypeDesc, i, Ptr)) return Ptr;
	return Super::GetIndexedPointer(TypeDesc, i);
}

void FNiagaraShaderMapPointerTable::SaveToArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, const void* FrozenObject) const
{
	Super::SaveToArchive(Ar, LayoutParams, FrozenObject);

	int32 NumDITypes = DITypes.Num();

	Ar << NumDITypes;

	for (int32 TypeIndex = 0; TypeIndex < NumDITypes; ++TypeIndex)
	{
		const UNiagaraDataInterfaceBase* DIType = DITypes.GetIndexedPointer(TypeIndex);
		FString DIClassName = DIType->GetClass()->GetFullName();
		Ar << DIClassName;
	}
}

bool FNiagaraShaderMapPointerTable::LoadFromArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, void* FrozenObject)
{
	const bool bResult = Super::LoadFromArchive(Ar, LayoutParams, FrozenObject);
	INiagaraShaderModule * Module = INiagaraShaderModule::Get();

	int32 NumDITypes = 0;

	Ar << NumDITypes;

	DITypes.Empty(NumDITypes);
	for (int32 TypeIndex = 0; TypeIndex < NumDITypes; ++TypeIndex)
	{
		FString DIClassName;
		Ar << DIClassName;
		UNiagaraDataInterfaceBase* DIType = Module->RequestDefaultDataInterface(*DIClassName);
		DITypes.LoadIndexedPointer(DIType);
	}

	return bResult;
}


/** Hashes the script-specific part of this shader map Id. */
uint64 FNiagaraShaderMapId::GetScriptHash() const
{
	FXxHash64Builder HashState;
	HashState.Update((const uint8*)&CompilerVersionID, sizeof(CompilerVersionID));
	HashState.Update(BaseCompileHash.Hash, FNiagaraCompileHash::HashSize);
	HashState.Update((const uint8*)&FeatureLevel, sizeof(FeatureLevel));

	for (int32 Index = 0; Index < AdditionalDefines.Num(); Index++)
	{
		HashState.Update(*AdditionalDefines[Index], AdditionalDefines[Index].Len() * sizeof(FMemoryImageString::ElementType));
	}

	for (int32 Index = 0; Index < AdditionalVariables.Num(); Index++)
	{
		HashState.Update(*AdditionalVariables[Index], AdditionalVariables[Index].Len() * sizeof(FMemoryImageString::ElementType));
	}

	for (int32 Index = 0; Index < ReferencedCompileHashes.Num(); Index++)
	{
		HashState.Update(ReferencedCompileHashes[Index].Hash, FNiagaraCompileHash::HashSize);
	}

	for (const FShaderTypeDependency& Dependency : ShaderTypeDependencies)
	{
		HashState.Update(Dependency.SourceHash.Hash, sizeof(Dependency.SourceHash));
	}
	
	return HashState.Finalize().Hash;
}

/** 
* Tests this set against another for equality, disregarding override settings.
* 
* @param ReferenceSet	The set to compare against
* @return				true if the sets are equal
*/
bool FNiagaraShaderMapId::operator==(const FNiagaraShaderMapId& ReferenceSet) const
{
	if (   BaseCompileHash != ReferenceSet.BaseCompileHash
		|| FeatureLevel != ReferenceSet.FeatureLevel
		|| CompilerVersionID != ReferenceSet.CompilerVersionID 
		|| bUsesRapidIterationParams != ReferenceSet.bUsesRapidIterationParams
		|| LayoutParams != ReferenceSet.LayoutParams)
	{
		return false;
	}

	if (AdditionalDefines.Num() != ReferenceSet.AdditionalDefines.Num())
	{
		return false;
	}

	if (AdditionalVariables.Num() != ReferenceSet.AdditionalVariables.Num())
	{
		return false;
	}

	if (ReferencedCompileHashes.Num() != ReferenceSet.ReferencedCompileHashes.Num())
	{
		return false;
	}

	for (int32 Idx = 0; Idx < ReferenceSet.AdditionalDefines.Num(); Idx++)
	{
		const FString& ReferenceStr = ReferenceSet.AdditionalDefines[Idx];

		if (AdditionalDefines[Idx] != ReferenceStr)
		{
			return false;
		}
	}

	for (int32 Idx = 0; Idx < ReferenceSet.AdditionalVariables.Num(); Idx++)
	{
		const FString& ReferenceStr = ReferenceSet.AdditionalVariables[Idx];

		if (AdditionalVariables[Idx] != ReferenceStr)
		{
			return false;
		}
	}

	for (int32 i = 0; i < ReferencedCompileHashes.Num(); i++)
	{
		if (ReferencedCompileHashes[i] != ReferenceSet.ReferencedCompileHashes[i])
		{
			return false;
		}
	}

	if (ShaderTypeDependencies != ReferenceSet.ShaderTypeDependencies)
	{
		return false;
	}

	return true;
}

#if WITH_EDITOR
void FNiagaraShaderMapId::AppendKeyString(FString& KeyString) const
{
	BaseCompileHash.AppendString(KeyString);
	KeyString.AppendChar('_');

	{
		const FSHAHash LayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FNiagaraShaderMapContent>(), LayoutParams);
		KeyString.AppendChar('_');
		LayoutHash.AppendString(KeyString);
		KeyString.AppendChar('_');
	}

	{
		const FSHAHash LayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FNiagaraShader>(), LayoutParams);
		KeyString.AppendChar('_');
		LayoutHash.AppendString(KeyString);
		KeyString.AppendChar('_');
	}

	// include the LayoutParams to differentiate between the frozen memory layout differences between platforms
	LayoutParams.AppendKeyString(KeyString);

	FName FeatureLevelName;
	GetFeatureLevelName(FeatureLevel, FeatureLevelName);

	FeatureLevelName.AppendString(KeyString);
	KeyString.AppendChar('_');
	CompilerVersionID.AppendString(KeyString);
	KeyString.AppendChar('_');

	if (bUsesRapidIterationParams)
	{
		KeyString += TEXT("USESRI_");
	}
	else
	{
		KeyString += TEXT("NORI_");
	}

	// Add base parameters structure
	KeyString.Appendf(TEXT("%08x"), TShaderParameterStructTypeInfo<FNiagaraShader::FParameters>::GetStructMetadata()->GetLayoutHash());

	// Add additional defines
	for (const FMemoryImageString& Define : AdditionalDefines)
	{
		KeyString.AppendChar('_');
		KeyString.Append(Define);
	}

	// Add additional variables
	for (const FMemoryImageString& Var : AdditionalVariables)
	{
		KeyString.AppendChar('_');
		KeyString.Append(Var);
	}

	// Add any referenced top level compile hashes to the key so that we will recompile when they are changed
	for (const FSHAHash& Hash : ReferencedCompileHashes)
	{
		KeyString.AppendChar('_');
		Hash.AppendString(KeyString);
	}

	// Add the inputs for any shaders that are stored inline in the shader map
	AppendKeyStringShaderDependencies(MakeArrayView(ShaderTypeDependencies), LayoutParams, KeyString);
}

/**
 * Enqueues a compilation for a new shader of this type.
 * @param script - The script to link the shader with.
 */
void FNiagaraShaderType::BeginCompileShader(
	uint32 ShaderMapId,
	int32 PermutationId,
	const FNiagaraShaderScript* Script,
	FSharedShaderCompilerEnvironment* CompilationEnvironment,
	EShaderPlatform Platform,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	FShaderTarget Target
	)
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(ShaderMapId, FShaderCompileJobKey(this, Platform, nullptr, PermutationId), EShaderCompileJobPriority::Normal);
	if (!NewJob)
	{
		return;
	}

	NewJob->ShaderParameters = MakeShared<const FParameters, ESPMode::ThreadSafe>(Script->GetScriptParametersMetadata());
	NewJob->Input.SharedEnvironment = CompilationEnvironment;
	NewJob->Input.Target = Target;
	NewJob->Input.ShaderFormat = LegacyShaderPlatformToShaderFormat(Platform);
	NewJob->Input.VirtualSourceFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf");
	NewJob->Input.EntryPointName = TEXT("SimulateMainComputeCS");
	NewJob->Input.Environment.SetDefine(TEXT("GPU_SIMULATION"), 1);
	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_MAX_GPU_SPAWN_INFOS"), NIAGARA_MAX_GPU_SPAWN_INFOS);
	Script->GetScriptHLSLSource(NewJob->Input.Environment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/NiagaraEmitterInstance.ush")));

	NewJob->Input.Environment.SetDefine(TEXT("SimulationStageIndex"), PermutationId);

	TConstArrayView<FSimulationStageMetaData> SimStageMetaDataArray = Script->GetBaseVMScript()->GetSimulationStageMetaData();
	const FSimulationStageMetaData& SimStageMetaData = SimStageMetaDataArray[PermutationId];
	if (SimStageMetaData.bWritesParticles && SimStageMetaData.bPartialParticleUpdate)
	{
		NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_PARTICLE_PARTIAL_ENABLED"), 1);
	}

	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_COMPRESSED_ATTRIBUTES_ENABLED"), Script->GetUsesCompressedAttributes() ? 1 : 0);

	const FIntVector ThreadGroupSize = SimStageMetaData.GpuDispatchNumThreads;
	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_DISPATCH_TYPE"), int(SimStageMetaData.GpuDispatchType));
	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_DISPATCH_INDIRECT"), SimStageMetaData.bGpuIndirectDispatch ? 1 : 0);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize.X * ThreadGroupSize.Y * ThreadGroupSize.Z);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSize.X);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSize.Y);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), ThreadGroupSize.Z);

	if (GNiagaraEnableHLSL2021)
	{
		NewJob->Input.Environment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	Script->ModifyCompilationEnvironment(Platform, NewJob->Input.Environment);

	AddUniformBufferIncludesToEnvironment(NewJob->Input.Environment, Platform);
	
	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	UE_LOGF(LogShaders, Verbose, "			%ls", GetName());
	COOK_STAT(NiagaraShaderCookStats::ShadersCompiled++);

	//update script shader stats
	UpdateNiagaraShaderCompilingStats(Script);

	// Allow the shader type to modify the compile environment.
	SetupCompileEnvironment(Platform, Script, ShaderEnvironment);

	::GlobalBeginCompileShader(
		Script->GetFriendlyName(),
		nullptr,
		this,
		nullptr,//ShaderPipeline,
		PermutationId,
		TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf"),
		TEXT("SimulateMainComputeCS"),
		FShaderTarget(GetFrequency(), Platform),
		NewJob->Input
	);

	NewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
}


/**
 * Enqueues a compilation for a new shader of this type.
 * @param script - The script to link the shader with.
 */
void FNiagaraShaderType::BeginCompileShaderFromSource(
	FStringView FriendlyName,
	uint32 ShaderMapId,
	int32 PermutationId,
	TSharedRef<FNiagaraShaderScriptParametersMetadata> ShaderParameters,
	FSharedShaderCompilerEnvironment* CompilationEnvironment,
	FStringView ScriptSource,
	EShaderPlatform Platform,
	const FSimulationStageMetaData& SimStageMetaData,
	FShaderTarget Target,
	TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs
) const
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(ShaderMapId, FShaderCompileJobKey(this, Platform, nullptr, PermutationId), EShaderCompileJobPriority::Normal);
	if (!NewJob)
	{
		return;
	}
	NewJob->ShaderParameters = MakeShared<const FParameters, ESPMode::ThreadSafe>(ShaderParameters);
	if (CompilationEnvironment)
	{
		// Initialize the Environment with what we've captured from the DI and the rest of the compilation process
		NewJob->Input.Environment = *CompilationEnvironment;
	}
	NewJob->Input.Target = Target;
	NewJob->Input.ShaderFormat = LegacyShaderPlatformToShaderFormat(Platform);
	NewJob->Input.VirtualSourceFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf");
	NewJob->Input.EntryPointName = TEXT("SimulateMainComputeCS");
	NewJob->Input.Environment.SetDefine(TEXT("GPU_SIMULATION"), 1);
	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_MAX_GPU_SPAWN_INFOS"), NIAGARA_MAX_GPU_SPAWN_INFOS);
	NewJob->Input.Environment.IncludeVirtualPathToContentsMap.Emplace(TEXT("/Engine/Generated/NiagaraEmitterInstance.ush"), ScriptSource);

	NewJob->Input.Environment.SetDefine(TEXT("SimulationStageIndex"), PermutationId);

	if (SimStageMetaData.bWritesParticles && SimStageMetaData.bPartialParticleUpdate)
	{
		NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_PARTICLE_PARTIAL_ENABLED"), 1);
	}

	const FIntVector ThreadGroupSize = SimStageMetaData.GpuDispatchNumThreads;
	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_DISPATCH_TYPE"), int(SimStageMetaData.GpuDispatchType));
	NewJob->Input.Environment.SetDefine(TEXT("NIAGARA_DISPATCH_INDIRECT"), SimStageMetaData.bGpuIndirectDispatch ? 1 : 0);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize.X * ThreadGroupSize.Y * ThreadGroupSize.Z);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSize.X);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSize.Y);
	NewJob->Input.Environment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), ThreadGroupSize.Z);

	if (GNiagaraEnableHLSL2021)
	{
		NewJob->Input.Environment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	AddUniformBufferIncludesToEnvironment(NewJob->Input.Environment, Platform);

	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	// Allow the shader type to modify the compile environment.
	ModifyCompilationEnvironment(FShaderPermutationParameters(Platform), ShaderEnvironment);

	::GlobalBeginCompileShader(
		FString(FriendlyName),
		nullptr,
		this,
		nullptr,//ShaderPipeline,
		PermutationId,
		TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf"),
		TEXT("SimulateMainComputeCS"),
		FShaderTarget(GetFrequency(), Platform),
		NewJob->Input
	);

	NewJobs.Emplace(NewJob);
}

void FNiagaraShaderType::AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, EShaderPlatform Platform) const
{
	UE::ShaderParameters::AddUniformBufferIncludesToEnvironment(OutEnvironment, ReferencedUniformBuffers);
}

FShader* FNiagaraShaderType::FinishCompileShader(
	uint64 ShaderMapHash,
	const FShaderCompileJob& CurrentJob
	) const
{
	check(CurrentJob.bSucceeded);

	FShader* Shader = ConstructCompiled(
		FNiagaraShaderType::CompiledShaderInitializerType(
			this,
			static_cast<const FParameters*>(CurrentJob.ShaderParameters.Get()),
			CurrentJob.Key.PermutationId,
			CurrentJob.Output,
			ShaderMapHash
		)
	);

	return Shader;
}
#endif // WITH_EDITOR

/**
* Finds the shader map for a script.
* @param ShaderMapId - The script id and static parameter set identifying the shader map
* @param Platform - The platform to lookup for
* @return NULL if no cached shader map was found.
*/
FNiagaraShaderMap* FNiagaraShaderMap::FindId(const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform InPlatform)
{
	FNiagaraShaderMap* Result = GIdToNiagaraShaderMap[InPlatform].FindRef(ShaderMapId);
	check(Result == nullptr || !Result->bDeletedThroughDeferredCleanup);
	return Result;
}

#if WITH_EDITOR

void NiagaraShaderMapAppendKeyString(EShaderPlatform Platform, FString& KeyString)
{
	// does nothing at the moment, but needs to append to keystring if runtime options impacting selection of sim shader permutations are added
	// for example static switches will need to go here
}

/** Creates a string key for the derived data cache given a shader map id. */
static FString GetNiagaraShaderMapKeyString(const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform Platform)
{
	static const FString NIAGARASHADERMAP_DERIVEDDATA_VER = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().NIAGARASHADERMAP_DERIVEDDATA_VER).ToString(EGuidFormats::DigitsWithHyphens);

	FName Format = LegacyShaderPlatformToShaderFormat(Platform);
	FString ShaderMapKeyString = Format.ToString() + TEXT("_") + FString(FString::FromInt(GetTargetPlatformManagerRef().ShaderFormatVersion(Format))) + TEXT("_");
	NiagaraShaderMapAppendKeyString(Platform, ShaderMapKeyString);
	ShaderMapAppendKeyString(Platform, ShaderMapKeyString);
	ShaderMapId.AppendKeyString(ShaderMapKeyString);
	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacyNIAGARASM"), TEXTVIEW("NiagaraShader"));
	return FDerivedDataCacheInterface::BuildCacheKey(TEXT("NIAGARASM"), *NIAGARASHADERMAP_DERIVEDDATA_VER, *ShaderMapKeyString);
}

void FNiagaraShaderMap::LoadFromDerivedDataCache(const FNiagaraShaderScript* Script, const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform Platform, FNiagaraShaderMapRef& InOutShaderMap)
{
	if (InOutShaderMap != NULL)
	{
		check(InOutShaderMap->GetShaderPlatform() == Platform);
	}
	else
	{
		// Shader map was not found in memory, try to load it from the DDC
		STAT(double NiagaraShaderDDCTime = 0);
		{
			SCOPE_SECONDS_COUNTER(NiagaraShaderDDCTime);
			COOK_STAT(auto Timer = NiagaraShaderCookStats::UsageStats.TimeSyncWork());

			TArray<uint8> CachedData;
			const FString DataKey = GetNiagaraShaderMapKeyString(ShaderMapId, Platform);

			if (GetDerivedDataCacheRef().GetSynchronous(*DataKey, CachedData, Script->GetFriendlyName()))
			{
				COOK_STAT(Timer.AddHit(CachedData.Num()));
				InOutShaderMap = new FNiagaraShaderMap();
				FMemoryReader Ar(CachedData, true);
				FShaderSerializeContext Ctx(Ar);
				// Deserialize from the cached data
				if (InOutShaderMap->Serialize(Ctx))
				{
					check(InOutShaderMap->GetShaderMapId() == ShaderMapId);

					// Register in the global map
					InOutShaderMap->Register(Platform);
				}
				else
				{
					// if the serialization failed it's likely because the resource is out of date now (i.e. shader parameters changed)
					COOK_STAT(Timer.TrackCyclesOnly());
					InOutShaderMap = nullptr;
				}
			}
			else
			{
				// We should be build the data later, and we can track that the resource was built there when we push it to the DDC.
				COOK_STAT(Timer.TrackCyclesOnly());
				InOutShaderMap = nullptr;
			}
		}
		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_DDCLoading,(float)NiagaraShaderDDCTime);
	}
}

void FNiagaraShaderMap::SaveToDerivedDataCache(const FNiagaraShaderScript* Script)
{
	COOK_STAT(auto Timer = NiagaraShaderCookStats::UsageStats.TimeSyncWork());
	TArray<uint8> SaveData;
	FMemoryWriter Ar(SaveData, true);
	FShaderSerializeContext Ctx(Ar);
	Serialize(Ctx);

	GetDerivedDataCacheRef().Put(*GetNiagaraShaderMapKeyString(GetContent()->ShaderMapId, GetShaderPlatform()), SaveData, Script ? Script->GetFriendlyName() : TEXT(""));
	COOK_STAT(Timer.AddMiss(SaveData.Num()));
}

/**
* Compiles the shaders for a script and caches them in this shader map.
* @param script - The script to compile shaders for.
* @param InShaderMapId - the script id and set of static parameters to compile for
* @param Platform - The platform to compile to
*/
void FNiagaraShaderMap::Compile(
	FNiagaraShaderScript* Script,
	const FNiagaraShaderMapId& InShaderMapId,
	TRefCountPtr<FSharedShaderCompilerEnvironment> CompilationEnvironment,
	EShaderPlatform InPlatform,
	bool bSynchronousCompile,
	bool bApplyCompletedShaderMapForRendering)
{
	if (FPlatformProperties::RequiresCookedData())
	{
		UE_LOGF(LogShaders, Fatal, "Trying to compile Niagara shader %ls at run-time, which is not supported on consoles!", *Script->GetFriendlyName() );
	}
	else
	{
		// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
		// Since it creates a temporary ref counted pointer.
		check(NumRefs > 0);
  
		//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
		check(IsInGameThread());
		// Add this shader map and to NiagaraShaderMapsBeingCompiled
		TArray<FNiagaraShaderScript*>* CorrespondingScripts = NiagaraShaderMapsBeingCompiled.Find(this);
  
		if (CorrespondingScripts)
		{
			check(!bSynchronousCompile);
			CorrespondingScripts->AddUnique(Script);
		}
		else
		{
			Script->RemoveOutstandingCompileId(CompilingId);
			// Assign a unique identifier so that shaders from this shader map can be associated with it after a deferred compile
			CompilingId = FShaderCommonCompileJob::GetNextJobId();
			Script->AddCompileId(CompilingId);
  
			TArray<FNiagaraShaderScript*> NewCorrespondingScripts;
			NewCorrespondingScripts.Add(Script);
			NiagaraShaderMapsBeingCompiled.Add(this, NewCorrespondingScripts);
			// Setup the compilation environment.
			Script->SetupShaderCompilationEnvironment(InPlatform, *CompilationEnvironment);
  
			// Store the script name for debugging purposes.
			FNiagaraShaderMapContent* NewContent = new FNiagaraShaderMapContent(InPlatform);
			NewContent->ShaderMapId = InShaderMapId;
			AssignContent(NewContent);

			uint32 NumShaders = 0;
			TArray<FShaderCommonCompileJobPtr> NewJobs;
	
			// Iterate over all shader types.
			for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
			{
				FNiagaraShaderType* ShaderType = ShaderTypeIt->GetNiagaraShaderType();
				if (ShaderType && ShouldCacheNiagaraShader(ShaderType, InPlatform, Script))
				{
					// Compile this niagara shader .
					TArray<FString> ShaderErrors;
  
					for (int32 PermutationId = 0; PermutationId < Script->GetNumPermutations(); ++PermutationId)
					{
						// Only compile the shader if we don't already have it
						if (!NewContent->HasShader(ShaderType, PermutationId))
						{
							ShaderType->BeginCompileShader(
								CompilingId,
								PermutationId, 
								Script,
								CompilationEnvironment,
								InPlatform,
								NewJobs,
								FShaderTarget(ShaderType->GetFrequency(), InPlatform)
							);
						}
						NumShaders++;
					}
				}
				else if (ShaderType)
				{
					UE_LOGF(LogShaders, Display, "Skipping compilation of %ls as it isn't supported on this target type.", *Script->SourceName);
					Script->RemoveOutstandingCompileId(CompilingId);
					// Can't call NotifyCompilationFinished() when post-loading. 
					// This normally happens when compiled in-sync for which the callback is not required.
					if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
					{
						Script->NotifyCompilationFinished();
					}
				}
			}
  
			if (!CorrespondingScripts)
			{
				UE_LOGF(LogShaders, Log, "		%u Shaders", NumShaders);
			}

			// Register this shader map in the global script->shadermap map
			Register(InPlatform);
  
			// Mark the shader map as not having been finalized with ProcessCompilationResults
			bCompilationFinalized = false;
  
			// Mark as not having been compiled
			bCompiledSuccessfully = false;
  
			GNiagaraShaderCompilationManager.AddJobs(NewJobs);
  
			// Compile the shaders for this shader map now if not deferring and deferred compiles are not enabled globally
			if (bSynchronousCompile)
			{
				TArray<int32> CurrentShaderMapId;
				CurrentShaderMapId.Add(CompilingId);
				GNiagaraShaderCompilationManager.FinishCompilation(CurrentShaderMapId);
			}
		}
	}
}


/**
* Compiles the shaders for a script and caches them in this shader map.
* @param script - The script to compile shaders for.
* @param InShaderMapId - the script id and set of static parameters to compile for
* @param Platform - The platform to compile to
*/
void FNiagaraShaderMap::CreateCompileJobs(
	const FNiagaraShaderType* ShaderType,
	FStringView FriendlyName,
	const FNiagaraShaderMapId& InShaderMapId,
	FStringView ScriptSource,
	TRefCountPtr<FSharedShaderCompilerEnvironment> CompilationEnvironment,
	TConstArrayView<FSimulationStageMetaData> StageMetaData,
	int32 PermutationCount,
	EShaderPlatform InPlatform,
	TSharedPtr<FNiagaraShaderScriptParametersMetadata> ShaderParameters,
	TArray<TRefCountPtr<FShaderCommonCompileJob>>& CompileJobs)
{
	if (FPlatformProperties::RequiresCookedData())
	{
		return;
	}

	// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
	// Since it creates a temporary ref counted pointer.
	check(NumRefs > 0);

	// Assign a unique identifier so that shaders from this shader map can be associated with it after a deferred compile
	CompilingId = FShaderCommonCompileJob::GetNextJobId();

	// Store the script name for debugging purposes.
	FNiagaraShaderMapContent* NewContent = new FNiagaraShaderMapContent(InPlatform);
	NewContent->ShaderMapId = InShaderMapId;
	AssignContent(NewContent);

	// Compile this niagara shader .
	for (int32 PermutationId = 0; PermutationId < PermutationCount; ++PermutationId)
	{
		ShaderType->BeginCompileShaderFromSource(
			FriendlyName,
			CompilingId,
			PermutationId,
			ShaderParameters.ToSharedRef(),
			CompilationEnvironment,
			ScriptSource,
			InPlatform,
			StageMetaData[PermutationId],
			FShaderTarget(ShaderType->GetFrequency(), InPlatform),
			CompileJobs
		);
	}

	// Register this shader map in the global script->shadermap map
	Register(InPlatform);

	// Mark the shader map as not having been finalized with ProcessCompilationResults
	bCompilationFinalized = false;

	// Mark as not having been compiled
	bCompiledSuccessfully = false;
}

FShader* FNiagaraShaderMap::ProcessCompilationResultsForSingleJob(const TRefCountPtr<class FShaderCommonCompileJob>& SingleJob, uint64 ShaderMapHash)
{
	auto CurrentJob = SingleJob->GetSingleShaderJob();
	check(CurrentJob->Id == CompilingId);

	GetResourceCode()->AddShaderCompilerOutput(CurrentJob->Output, CurrentJob->Key, SingleJob->GetSingleShaderJob()->Input.GenerateDebugInfo());

	FShader* Shader = nullptr;

	const FNiagaraShaderType* NiagaraShaderType = CurrentJob->Key.ShaderType->GetNiagaraShaderType();
	check(NiagaraShaderType);
	Shader = NiagaraShaderType->FinishCompileShader(ShaderMapHash, *CurrentJob);
	bCompiledSuccessfully = CurrentJob->bSucceeded;

	FNiagaraShader* NiagaraShader = static_cast<FNiagaraShader*>(Shader);

	// UE-67395 - we had a case where we polluted the DDC with a shader containing no bytecode.
	check(Shader && Shader->GetCodeSize() > 0);
	check(!GetContent()->HasShader(NiagaraShaderType, CurrentJob->Key.PermutationId));
	return GetMutableContent()->FindOrAddShader(NiagaraShaderType->GetHashedName(), CurrentJob->Key.PermutationId, Shader);
}

void FNiagaraShaderMap::ProcessAndFinalizeShaderCompileJob(const TRefCountPtr<FShaderCommonCompileJob>& SingleJob)
{
	uint64 ShaderMapHash = GetContent()->ShaderMapId.GetScriptHash();

	ProcessCompilationResultsForSingleJob(SingleJob, ShaderMapHash);

	FinalizeContent();

	// The shader map can now be used on the rendering thread
	bCompilationFinalized = true;
}

bool FNiagaraShaderMap::ProcessCompilationResults(const TArray<FNiagaraShaderScript*>& InScripts, const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutJobIndex, float& TimeBudget)
{
	check(InOutJobIndex < InCompilationResults.Num());

	double StartTime = FPlatformTime::Seconds();

	uint64 ShaderMapHash = GetContent()->ShaderMapId.GetScriptHash();

	do
	{
		{
			ProcessCompilationResultsForSingleJob(InCompilationResults[InOutJobIndex], ShaderMapHash);
		}

		InOutJobIndex++;
		
		double NewStartTime = FPlatformTime::Seconds();
		TimeBudget -= NewStartTime - StartTime;
		StartTime = NewStartTime;
	}
	while ((TimeBudget > 0.0f) && (InOutJobIndex < InCompilationResults.Num()));

	if (InOutJobIndex == InCompilationResults.Num())
	{
		FinalizeContent();

		SaveToDerivedDataCache(InScripts.IsEmpty() ? nullptr : InScripts.Top());
		// The shader map can now be used on the rendering thread
		bCompilationFinalized = true;
		return true;
	}

	return false;
}

bool FNiagaraShaderMap::TryToAddToExistingCompilationTask(FNiagaraShaderScript* Script)
{
	check(NumRefs > 0);
	//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
	check(IsInGameThread());
	TArray<FNiagaraShaderScript*>* CorrespondingScripts = FNiagaraShaderMap::NiagaraShaderMapsBeingCompiled.Find(this);

	if (CorrespondingScripts)
	{
		CorrespondingScripts->AddUnique(Script);

		UE_LOGF(LogShaders, Log, "TryToAddToExistingCompilationTask %p %d", Script, GetCompilingId());

		return true;
	}

	return false;
}

bool FNiagaraShaderMap::IsNiagaraShaderComplete(const FNiagaraShaderScript* Script, const FNiagaraShaderType* ShaderType, bool bSilent)
{
	// If we should cache this script, it's incomplete if the shader is missing
	if (ShouldCacheNiagaraShader(ShaderType, GetShaderPlatform(), Script))
	{
		for (int32 PermutationId = 0; PermutationId < Script->GetNumPermutations(); ++PermutationId)
		{
			if (!GetContent()->HasShader((FShaderType*)ShaderType, PermutationId))
			{
				if (!bSilent)
				{
					UE_LOGF(LogShaders, Warning, "Incomplete shader %ls, missing FNiagaraShader %ls.", *Script->GetFriendlyName(), ShaderType->GetName());
				}
				return false;
			}
		}
	}

	return true;
}

bool FNiagaraShaderMap::IsComplete(const FNiagaraShaderScript* Script, bool bSilent)
{
	check(!GIsThreadedRendering || !IsInActualRenderingThread());
	// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
	// Since it creates a temporary ref counted pointer.
	check(NumRefs > 0);
	//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
	check(IsInGameThread());
	const TArray<FNiagaraShaderScript*>* CorrespondingScripts = FNiagaraShaderMap::NiagaraShaderMapsBeingCompiled.Find(this);

	if (CorrespondingScripts)
	{
		check(!bCompilationFinalized);
		return false;
	}

	// Iterate over all shader types.
	for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
	{
		// Find this shader type in the script's shader map.
		const FNiagaraShaderType* ShaderType = ShaderTypeIt->GetNiagaraShaderType();
		if (ShaderType && !IsNiagaraShaderComplete(Script, ShaderType, bSilent))
		{
			return false;
		}
	}

	return true;
}

#endif // WITH_EDITOR

void FNiagaraShaderMap::GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, 0u, OutShaders);
}

void FNiagaraShaderMap::GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, OutShaders);
}

void FNiagaraShaderMap::GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const
{
	GetContent()->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
}

/**
 * Registers a Niagara shader map in the global map so it can be used by scripts.
 */
void FNiagaraShaderMap::Register(EShaderPlatform InShaderPlatform)
{
	extern int32 GCreateNiagaraShadersOnLoad;
	if (GCreateNiagaraShadersOnLoad && GetShaderPlatform() == InShaderPlatform)
	{
		// TODO
	}

	if (!bRegistered)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShaderMaps);
	}

	{
		FScopeLock ScopeLock(&GIdToNiagaraShaderMapCS);
		GIdToNiagaraShaderMap[GetShaderPlatform()].Add(GetContent()->ShaderMapId, this);
		bRegistered = true;
	}
}

void FNiagaraShaderMap::AddRef()
{
	FScopeLock ScopeLock(&GIdToNiagaraShaderMapCS);
	check(!bDeletedThroughDeferredCleanup);
	++NumRefs;
}

void FNiagaraShaderMap::Release()
{
	{
		FScopeLock ScopeLock(&GIdToNiagaraShaderMapCS);

		check(NumRefs > 0);
		if (--NumRefs == 0)
		{
			if (bRegistered)
			{
				DEC_DWORD_STAT(STAT_Shaders_NumShaderMaps);

				GIdToNiagaraShaderMap[GetShaderPlatform()].Remove(GetContent()->ShaderMapId);
				bRegistered = false;
			}

			check(!bDeletedThroughDeferredCleanup);
			bDeletedThroughDeferredCleanup = true;
		}
	}
	if (bDeletedThroughDeferredCleanup)
	{
		BeginCleanup(this);
	}
}

FNiagaraShaderMap::FNiagaraShaderMap() :
	CompilingId(1),
	NumRefs(0),
	bDeletedThroughDeferredCleanup(false),
	bRegistered(false),
	bCompilationFinalized(true),
	bCompiledSuccessfully(true),
	bIsPersistent(true) 
{
	checkSlow(IsInGameThread() || IsAsyncLoading());
}

#if WITH_EDITOR
FNiagaraShaderMap::FNiagaraShaderMap(EWorkerThread)
	: CompilingId(1)
	, NumRefs(0)
	, bDeletedThroughDeferredCleanup(false)
	, bRegistered(false)
	, bCompilationFinalized(true)
	, bCompiledSuccessfully(true)
	, bIsPersistent(true) 
{
}
#endif // WITH_EDITOR

FNiagaraShaderMap::~FNiagaraShaderMap()
{ 
	checkSlow(IsInGameThread() || IsAsyncLoading());
	check(bDeletedThroughDeferredCleanup);
	check(!bRegistered);
}


bool FNiagaraShaderMap::Serialize(FArchive& Ar, bool bInlineShaderResources, bool bLoadingCooked)
{
	FShaderSerializeContext Ctx(Ar);
	Ctx.bLoadingCooked = bLoadingCooked;
	return Serialize(Ctx);
}

bool FNiagaraShaderMap::Serialize(FShaderSerializeContext& Ctx)
{
	// Note: This is saved to the DDC, not into packages (except when cooked)
	// Backwards compatibility therefore will not work based on the version of Ar
	// Instead, just bump NIAGARASHADERMAP_DERIVEDDATA_VER
	return Super::Serialize(Ctx);
}

#if WITH_EDITOR
bool FNiagaraShaderMap::RemovePendingScript(FNiagaraShaderScript* Script)
{
	bool bRemoved = false;
	//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
	check(IsInGameThread());
	for (TMap<FNiagaraShaderMapRef, TArray<FNiagaraShaderScript*> >::TIterator It(NiagaraShaderMapsBeingCompiled); It; ++It)
	{
		TArray<FNiagaraShaderScript*>& Scripts = It.Value();
		int32 Result = Scripts.Remove(Script);
		if (Result)
		{
			Script->RemoveOutstandingCompileId(It.Key()->CompilingId);
			// Can't call NotifyCompilationFinished() when post-loading. 
			// This normally happens when compiled in-sync for which the callback is not required.
			if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
			{
				Script->NotifyCompilationFinished();
			}
			bRemoved = true;
		}
	}

	return bRemoved;
}


void FNiagaraShaderMap::RemovePendingMap(FNiagaraShaderMap* Map)
{
	//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
	check(IsInGameThread());
	TArray<FNiagaraShaderScript*>* Scripts = NiagaraShaderMapsBeingCompiled.Find(Map);
	if (Scripts)
	{
		for (FNiagaraShaderScript* Script : *Scripts)
		{
			Script->RemoveOutstandingCompileId(Map->CompilingId);
			// Can't call NotifyCompilationFinished() when post-loading. 
			// This normally happens when compiled in-sync for which the callback is not required.
			if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
			{
				Script->NotifyCompilationFinished();
			}
		}
	}

	NiagaraShaderMapsBeingCompiled.Remove(Map);
}

const FNiagaraShaderMap* FNiagaraShaderMap::GetShaderMapBeingCompiled(const FNiagaraShaderScript* Script)
{
	// Inefficient search, but only when compiling a lot of shaders
	//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
	check(IsInGameThread());
	for (TMap<FNiagaraShaderMapRef, TArray<FNiagaraShaderScript*> >::TIterator It(NiagaraShaderMapsBeingCompiled); It; ++It)
	{
		TArray<FNiagaraShaderScript*>& Scripts = It.Value();
		
		for (int32 ScriptIndex = 0; ScriptIndex < Scripts.Num(); ScriptIndex++)
		{
			if (Scripts[ScriptIndex] == Script)
			{
				return It.Key();
			}
		}
	}

	return NULL;
}
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////////

FNiagaraShader::FNiagaraShader(const FNiagaraShaderType::CompiledShaderInitializerType& Initializer)
	: FShader(Initializer)
{
	// Cache off requirements for later queries
	bNeedsViewUniformBuffer = Initializer.ParameterMap.ContainsParameterAllocation(TEXT("View"));

	// Legacy bindings
	ExternalConstantBufferParam[0].Bind(Initializer.ParameterMap, TEXT("FNiagaraExternalParameters"));
	ExternalConstantBufferParam[1].Bind(Initializer.ParameterMap, TEXT("PREV_FNiagaraExternalParameters"));

	// We can only use validation for missing parameter bindings if we don't use the external constant buffer and
	// all DataInterfaces use the new parameter bindings.  Otherwise we can't bind in a single pass.
	const FNiagaraShaderType::FParameters* InitializerParameters = static_cast<const FNiagaraShaderType::FParameters*>(Initializer.Parameters);
	INiagaraShaderModule* ShaderModule = INiagaraShaderModule::Get();
	bool bShouldBindEverything = true;
	
	if (GNiagaraShaderForceBindEverything == 0)
	{
		bShouldBindEverything &= ExternalConstantBufferParam[0].IsBound() == false;
		bShouldBindEverything &= ExternalConstantBufferParam[1].IsBound() == false;
	}

	// Bind parameters
	Bindings.BindForLegacyShaderParameters(
		this,
		Initializer.PermutationId,
		Initializer.ParameterMap,
		*InitializerParameters->ScriptParametersMetadata->ShaderParametersMetadata.Get(),
		bShouldBindEverything
	);

	// Gather data interface bindings
	MiscUsageBitMask = 0;
	DataInterfaceParameters.Empty(InitializerParameters->ScriptParametersMetadata->DataInterfaceParamInfo.Num());
	for (const FNiagaraDataInterfaceGPUParamInfo& DataInterfaceParamInfo : InitializerParameters->ScriptParametersMetadata->DataInterfaceParamInfo)
	{
		FNiagaraDataInterfaceParamRef& DataInterfaceParamRef = DataInterfaceParameters.AddDefaulted_GetRef();
		DataInterfaceParamRef.ShaderParametersOffset = DataInterfaceParamInfo.ShaderParametersOffset;

		UNiagaraDataInterfaceBase* CDODataInterface = ShaderModule->RequestDefaultDataInterface(*DataInterfaceParamInfo.DIClassName);
		if (CDODataInterface == nullptr)
		{
			continue;
		}

		for (auto& GeneratedFunction : DataInterfaceParamInfo.GeneratedFunctions)
		{
			MiscUsageBitMask |= GeneratedFunction.MiscUsageBitMask;
		}

		DataInterfaceParamRef.DIType = TIndexedPtr<UNiagaraDataInterfaceBase>(CDODataInterface);
		DataInterfaceParamRef.Parameters = CDODataInterface->CreateShaderStorage(DataInterfaceParamInfo, Initializer.ParameterMap);
		checkf(DataInterfaceParamRef.Parameters == nullptr || CDODataInterface->GetShaderStorageType() != nullptr, TEXT("DataInterface(%s) provides shader storage but did not implement GetShaderStorageType"), *GetNameSafe(CDODataInterface->GetClass()));
		checkf(DataInterfaceParamRef.Parameters == nullptr || CDODataInterface->GetShaderStorageType()->Interface == ETypeLayoutInterface::NonVirtual, TEXT("DataInterface(%s) shader storage is either abstract or virtual which is not allowed"), *GetNameSafe(CDODataInterface->GetClass()));
	}
}

#if WITH_EDITORONLY_DATA
void FNiagaraShader::BuildClassSchema(FAppendToClassSchemaContext& Context)
{
	// Used by iterative cooking.  This will provide additional context for if things have changed such that a cook will
	// be required.  This is focused on global settings rather than the usual dependencies between objects.

	FShaderKeyGenerator KeyGen([&Context](const void* Data, uint64 Size) { Context.Update(Data, Size); });

	FPlatformTypeLayoutParameters DefaultLayoutParams;
	DefaultLayoutParams.Append(KeyGen);

	const FSHAHash ContentLayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FNiagaraShaderMapContent>(), DefaultLayoutParams);
	Context.Update(&ContentLayoutHash, sizeof(ContentLayoutHash));

	const FSHAHash ShaderLayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FNiagaraShader>(), DefaultLayoutParams);
	Context.Update(&ShaderLayoutHash, sizeof(ShaderLayoutHash));

	const uint32 ShaderParamStructHash = TShaderParameterStructTypeInfo<FNiagaraShader::FParameters>::GetStructMetadata()->GetLayoutHash();
	Context.Update(&ShaderParamStructHash, sizeof(ShaderParamStructHash));

	// Add in any referenced HLSL files.
	if (AllowShaderCompiling())
	{
		const FSHAHash InstanceShaderHash = GetShaderFileHash((TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf")), EShaderPlatform::SP_PCD3D_SM5);
		Context.Update(&InstanceShaderHash, sizeof(InstanceShaderHash));

		const FSHAHash NiagaraShaderVersionHash = GetShaderFileHash((TEXT("/Plugin/FX/Niagara/Private/NiagaraShaderVersion.ush")), EShaderPlatform::SP_PCD3D_SM5);
		Context.Update(&NiagaraShaderVersionHash, sizeof(NiagaraShaderVersionHash));

		const FSHAHash ShaderVersionHash = GetShaderFileHash((TEXT("/Engine/Public/ShaderVersion.ush")), EShaderPlatform::SP_PCD3D_SM5);
		Context.Update(&ShaderVersionHash, sizeof(ShaderVersionHash));
	}

	Context.Update(&GNiagaraSkipVectorVMBackendOptimizations, sizeof(GNiagaraSkipVectorVMBackendOptimizations));
}
#endif


//////////////////////////////////////////////////////////////////////////
bool FNiagaraDataInterfaceGeneratedFunction::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	const int32 NiagaraVer = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	Ar << DefinitionName;
	Ar << InstanceName;
	Ar << Specifiers;

	if (NiagaraVer >= FNiagaraCustomVersion::AddVariadicParametersToGPUFunctionInfo)
	{
		Ar << VariadicInputs;
		Ar << VariadicOutputs;
	}

	if (NiagaraVer >= FNiagaraCustomVersion::SerializeUsageBitMaskToGPUFunctionInfo)
	{
		Ar << MiscUsageBitMask;
	}

	return true;
}

bool operator<<(FArchive& Ar, FNiagaraDataInterfaceGeneratedFunction& DIFunction)
{
	return DIFunction.Serialize(Ar);
}

void FNiagaraDataInterfaceParamRef::WriteFrozenParameters(FMemoryImageWriter& Writer, const TMemoryImagePtr<FNiagaraDataInterfaceParametersCS>& InParameters) const
{
	const UNiagaraDataInterfaceBase* Base = DIType.Get(Writer.TryGetPrevPointerTable());
	InParameters.WriteMemoryImageWithDerivedType(Writer, Base->GetShaderStorageType());
}

bool FNiagaraDataInterfaceGPUParamInfo::IsExternalParameter() const
{
	return DataInterfaceHLSLSymbol.StartsWith(TEXT("User_")) || DataInterfaceHLSLSymbol.StartsWith(TEXT("NPC_"));
}

bool FNiagaraDataInterfaceGPUParamInfo::IsUserParameter() const
{
	return DataInterfaceHLSLSymbol.StartsWith(TEXT("User_"));
}

bool FNiagaraDataInterfaceGPUParamInfo::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	const int32 NiagaraVer = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	Ar << DataInterfaceHLSLSymbol;
	Ar << DIClassName;

	// If FNiagaraDataInterfaceGPUParamInfo was only included in the DI parameters of FNiagaraShader, we wouldn't need to worry about
	// custom versions, because bumping FNiagaraCustomVersion::LatestScriptCompileVersion is enough to cause all shaders to be
	// rebuilt and things to be serialized correctly. However, there's a property of type FNiagaraVMExecutableData inside UNiagaraScript,
	// which in turn contains an array of FNiagaraDataInterfaceGPUParamInfo, so we must check the version in order to be able to load
	// UNiagaraScript objects saved before GeneratedFunctions was introduced.
	const bool SkipGeneratedFunctions = Ar.IsLoading() && (NiagaraVer < FNiagaraCustomVersion::AddGeneratedFunctionsToGPUParamInfo);
	if (!SkipGeneratedFunctions)
	{
		Ar << GeneratedFunctions;
	}

	return true;
}
