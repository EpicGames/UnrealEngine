// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraShaderCompilationManager.h"

#include "Misc/Paths.h"
#include "NiagaraShared.h"
#include "RenderingThread.h"
#include "ShaderCompiler.h"
#include "Tickable.h"
#include "UObject/UObjectThreadContext.h"
#include "DataDrivenShaderPlatformInfo.h"

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraShaderCompiler, Log, All);

static int32 GShowNiagaraShaderWarnings = 0;
static FAutoConsoleVariableRef CVarShowNiagaraShaderWarnings(
	TEXT("niagara.ShowShaderCompilerWarnings"),
	GShowNiagaraShaderWarnings,
	TEXT("When set to 1, will display all warnings from Niagara shader compiles.")
	);

#if WITH_EDITOR

NIAGARASHADER_API FNiagaraShaderCompilationManager GNiagaraShaderCompilationManager;

NIAGARASHADER_API void FNiagaraShaderCompilationManager::AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs)
{
	check(IsInGameThread());
	JobQueue.Append(InNewJobs);
	
	for (FShaderCommonCompileJobPtr& Job : InNewJobs)
	{
		FNiagaraShaderMapCompileResults& ShaderMapInfo = NiagaraShaderMapJobs.FindOrAdd(Job->Id);
//		ShaderMapInfo.bApplyCompletedShaderMapForRendering = bApplyCompletedShaderMapForRendering;
//		ShaderMapInfo.bRecreateComponentRenderStateOnCompletion = bRecreateComponentRenderStateOnCompletion;
		ShaderMapInfo.NumJobsQueued++;

		auto CurrentJob = Job->GetSingleShaderJob();

		// Fast math breaks The ExecGrid layout script because floor(x/y) returns a bad value if x == y. Yay.
		if (IsMetalPlatform((EShaderPlatform)CurrentJob->Input.Target.Platform))
		{
			CurrentJob->Input.Environment.CompilerFlags.Add(CFLAG_NoFastMath);
		}

		UE_LOGF(LogNiagaraShaderCompiler, Verbose, "Adding niagara gpu shader compile job... %ls", *CurrentJob->Input.DebugGroupName);

		CurrentJob->Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / CurrentJob->Input.ShaderPlatformName.ToString();
		FPaths::NormalizeDirectoryName(CurrentJob->Input.DumpDebugInfoRootPath);
		CurrentJob->Input.DebugExtension.Empty();
		CurrentJob->Input.DumpDebugInfoPath.Empty();
		if (GShaderCompilingManager->GetDumpShaderDebugInfo() == FShaderCompilingManager::EDumpShaderDebugInfo::Always)
		{
			CurrentJob->Input.DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(CurrentJob->Input);
		}
	}
	GShaderCompilingManager->SubmitJobs(InNewJobs, FString(), FString());
}

void FNiagaraShaderCompilationManager::ProcessAsyncResults()
{
	check(IsInGameThread());

	// Process the results from the shader compile worker
	for (int32 JobIndex = JobQueue.Num() - 1; JobIndex >= 0; JobIndex--)
	{
		auto CurrentJob = JobQueue[JobIndex]->GetSingleShaderJob();

		if (CurrentJob->GetStatus() != EShaderCompileJobStatus::Released)
		{
			continue;
		}

		CurrentJob->bSucceeded = CurrentJob->Output.bSucceeded;
		if (CurrentJob->Output.bSucceeded)
		{
			UE_LOGF(LogNiagaraShaderCompiler, Verbose, "GPU shader compile succeeded. Id %d", CurrentJob->Id);
		}
		else
		{
			UE_LOGF(LogNiagaraShaderCompiler, Warning, "GPU shader compile failed! Id: %d Name: %ls", CurrentJob->Id, *CurrentJob->Input.DebugGroupName);
		}

		FNiagaraShaderMapCompileResults& ShaderMapResults = NiagaraShaderMapJobs.FindChecked(CurrentJob->Id);
		ShaderMapResults.FinishedJobs.Add(JobQueue[JobIndex]);
		ShaderMapResults.bAllJobsSucceeded = ShaderMapResults.bAllJobsSucceeded && CurrentJob->bSucceeded;
		JobQueue.RemoveAt(JobIndex);
	}

	// Get all Niagara shader maps to finalize
	//
	for (TMap<int32, FNiagaraShaderMapCompileResults>::TIterator It(NiagaraShaderMapJobs); It; ++It)
	{
		const FNiagaraShaderMapCompileResults& Results = It.Value();

		if (Results.FinishedJobs.Num() == Results.NumJobsQueued)
		{
			PendingFinalizeNiagaraShaderMaps.Add(It.Key(), FNiagaraShaderMapFinalizeResults(Results));
			It.RemoveCurrent();
		}
	}

	if (PendingFinalizeNiagaraShaderMaps.Num() > 0)
	{
		ProcessCompiledNiagaraShaderMaps(PendingFinalizeNiagaraShaderMaps, 10);
	}
}


void FNiagaraShaderCompilationManager::ProcessCompiledNiagaraShaderMaps(TMap<int32, FNiagaraShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget)
{
	check(IsInGameThread());

	// Keeps shader maps alive as they are passed from the shader compiler and applied to the owning Script
	TArray<FNiagaraShaderMapRef> LocalShaderMapReferences;
	TMap<FNiagaraShaderScript*, FNiagaraShaderMap*> ScriptsToUpdate;

	// Process compiled shader maps in FIFO order, in case a shader map has been enqueued multiple times,
	// Which can happen if a script is edited while a background compile is going on
	for (TMap<int32, FNiagaraShaderMapFinalizeResults>::TIterator ProcessIt(CompiledShaderMaps); ProcessIt; ++ProcessIt)
	{
		FNiagaraShaderMapRef ShaderMap = NULL;
		TArray<FNiagaraShaderScript*>* Scripts = NULL;

		for (TMap<FNiagaraShaderMapRef, TArray<FNiagaraShaderScript*> >::TIterator ShaderMapIt(FNiagaraShaderMap::GetInFlightShaderMaps()); ShaderMapIt; ++ShaderMapIt)
		{
			if (ShaderMapIt.Key()->GetCompilingId() == ProcessIt.Key())
			{
				ShaderMap = ShaderMapIt.Key();
				Scripts = &ShaderMapIt.Value();
				break;
			}
		}

		if (ShaderMap && Scripts)
		{
			TArray<FString> Errors;
			FNiagaraShaderMapFinalizeResults& CompileResults = ProcessIt.Value();
			TArray<FShaderCommonCompileJobPtr>& ResultArray = CompileResults.FinishedJobs;

			// Make a copy of the array as this entry of FNiagaraShaderMap::ShaderMapsBeingCompiled will be removed below
			TArray<FNiagaraShaderScript*> ScriptArray = *Scripts;
			bool bSuccess = true;

			for (int32 JobIndex = 0; JobIndex < ResultArray.Num(); JobIndex++)
			{
				auto CurrentJob = ResultArray[JobIndex]->GetSingleShaderJob();
				bSuccess = bSuccess && CurrentJob->bSucceeded;

				if (bSuccess)
				{
					check(CurrentJob->Output.ShaderCode.GetShaderCodeSize() > 0);
				}

				if (GShowNiagaraShaderWarnings || !CurrentJob->bSucceeded)
				{
					for (int32 ErrorIndex = 0; ErrorIndex < CurrentJob->Output.Errors.Num(); ErrorIndex++)
					{
						Errors.AddUnique(CurrentJob->Output.Errors[ErrorIndex].GetErrorString().Replace(TEXT("Error"), TEXT("Err0r")));
					}

					if (CurrentJob->Output.Errors.Num())
					{
						UE_LOGF(LogNiagaraShaderCompiler, Display, "There were issues for job \"%ls\"", *CurrentJob->Input.DebugGroupName);
						for (const FShaderCompilerError& Error : CurrentJob->Output.Errors)
						{
							UE_LOGF(LogNiagaraShaderCompiler, Warning, "%ls", *Error.GetErrorString())
						}
					}
				}
				else
				{
					UE_LOGF(LogNiagaraShaderCompiler, Verbose, "Shader compile job \"%ls\" completed.", *CurrentJob->Input.DebugGroupName);
				}
			}

			bool bShaderMapComplete = true;
			if (bSuccess)
			{
				bShaderMapComplete = ShaderMap->ProcessCompilationResults(ScriptArray, ResultArray, CompileResults.FinalizeJobIndex, TimeBudget);
			}

			if (bShaderMapComplete)
			{
				ShaderMap->SetCompiledSuccessfully(bSuccess);

				// Pass off the reference of the shader map to LocalShaderMapReferences
				LocalShaderMapReferences.Add(ShaderMap);
				FNiagaraShaderMap::GetInFlightShaderMaps().Remove(ShaderMap);

				for (FNiagaraShaderScript* Script : ScriptArray)
				{
					FNiagaraShaderMap* CompletedShaderMap = ShaderMap;

					Script->RemoveOutstandingCompileId(ShaderMap->GetCompilingId());

					// Only process results that still match the ID which requested a compile
					// This avoids applying shadermaps which are out of date and a newer one is in the async compiling pipeline
					if (Script->IsSame(CompletedShaderMap->GetShaderMapId()))
					{
						if (Errors.Num() != 0)
						{
							FString SourceCode;
							Script->GetScriptHLSLSource(SourceCode);
							UE_LOGF(LogNiagaraShaderCompiler, Verbose, "Compile output as text:");
							UE_LOGF(LogNiagaraShaderCompiler, Verbose, "===================================================================================");
							TArray<FString> OutputByLines;
							SourceCode.ParseIntoArrayLines(OutputByLines, false);
							for (int32 i = 0; i < OutputByLines.Num(); i++)
							{
								UE_LOGF(LogNiagaraShaderCompiler, Verbose, "/*%04d*/\t\t%ls", i + 1, *OutputByLines[i]);
							}
							UE_LOGF(LogNiagaraShaderCompiler, Verbose, "===================================================================================");
						}

						if (!bSuccess)
						{
							// Propagate error messages
							Script->SetCompileErrors(Errors);
							ScriptsToUpdate.Add(Script, NULL);

							for (int32 ErrorIndex = 0; ErrorIndex < Errors.Num(); ErrorIndex++)
							{
								FString ErrorMessage = Errors[ErrorIndex];
								// Work around build machine string matching heuristics that will cause a cook to fail
								ErrorMessage.ReplaceInline(TEXT("error "), TEXT("err0r "), ESearchCase::CaseSensitive);
								UE_LOGF(LogNiagaraShaderCompiler, Warning, "	%ls", *ErrorMessage);
							}
						}
						else
						{
							Script->SetCompileErrors(Errors);

							// if we succeeded and our shader map is not complete this could be because the script was being edited quicker then the compile could be completed
							// Don't modify scripts for which the compiled shader map is no longer complete
							// This can happen if a script being compiled is edited
							if (CompletedShaderMap->IsComplete(Script, true))
							{
								ScriptsToUpdate.Add(Script, CompletedShaderMap);
							}

							if (GShowNiagaraShaderWarnings && Errors.Num() > 0)
							{
								UE_LOGF(LogNiagaraShaderCompiler, Warning, "Warnings while compiling Niagara Script %ls for platform %ls:",
									*Script->GetFriendlyName(),
									*LegacyShaderPlatformToShaderFormat(ShaderMap->GetShaderPlatform()).ToString());
								for (int32 ErrorIndex = 0; ErrorIndex < Errors.Num(); ErrorIndex++)
								{
									UE_LOGF(LogNiagaraShaderCompiler, Warning, "	%ls", *Errors[ErrorIndex]);
								}
							}
						}


						if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
						{
							Script->NotifyCompilationFinished();
						}
					}
					else
					{
						// Can't call NotifyCompilationFinished() when post-loading. 
						// This normally happens when compiled in-sync for which the callback is not required.
						if (CompletedShaderMap->IsComplete(Script, true) && !FUObjectThreadContext::Get().IsRoutingPostLoad)
						{
							Script->NotifyCompilationFinished();
						}
					}
				}

				// Cleanup shader jobs and compile tracking structures
				ResultArray.Empty();
				ProcessIt.RemoveCurrent();
			}

			if (TimeBudget < 0)
			{
				break;
			}
		}
	}

	if (ScriptsToUpdate.Num() > 0)
	{
		for (TMap<FNiagaraShaderScript*, FNiagaraShaderMap*>::TConstIterator It(ScriptsToUpdate); It; ++It)
		{
			FNiagaraShaderScript* Script = It.Key();
			FNiagaraShaderMap* ShaderMap = It.Value();
			//check(!ShaderMap || ShaderMap->IsValidForRendering());

			Script->SetGameThreadShaderMap(It.Value());

			ENQUEUE_RENDER_COMMAND(FSetShaderMapOnScriptResources)(
				[Script, ShaderMap](FRHICommandListImmediate& RHICmdList)
				{
					Script->SetRenderingThreadShaderMap(ShaderMap);
				});

			// Can't call NotifyCompilationFinished() when post-loading. 
			// This normally happens when compiled in-sync for which the callback is not required.
			if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
			{
				Script->NotifyCompilationFinished();
			}
		}
	}
}


void FNiagaraShaderCompilationManager::FinishCompilation(const TArray<int32>& ShaderMapIdsToFinishCompiling)
{
	check(!FPlatformProperties::RequiresCookedData());
	GShaderCompilingManager->FinishCompilation(NULL, ShaderMapIdsToFinishCompiling);
	ProcessAsyncResults();
}

// Handles prodding the GNiagaraShaderCompilationManager to finish shader compile jobs
// Only necessary for WITH_EDITOR builds, but can't be an FTickableEditorObject because -game requires this as well
class FNiagaraShaderProcessorTickable : FTickableGameObject
{
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual void Tick(float DeltaSeconds) override
	{
		GNiagaraShaderCompilationManager.ProcessAsyncResults();
	}

	virtual TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraShaderQueueTickable, STATGROUP_Tickables);
	}
};

static FNiagaraShaderProcessorTickable NiagaraShaderProcessor;

#endif // WITH_EDITOR
