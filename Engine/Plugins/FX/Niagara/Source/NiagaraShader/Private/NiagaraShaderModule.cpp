// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraShaderModule.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"

IMPLEMENT_MODULE(INiagaraShaderModule, NiagaraShader);

INiagaraShaderModule* INiagaraShaderModule::Singleton(nullptr);

void INiagaraShaderModule::StartupModule()
{
	Singleton = this;

	// Maps virtual shader source directory /Plugin/FX/Niagara to the plugin's actual Shaders directory.
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("Niagara"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/FX/Niagara"), PluginShaderDir);
	AddShaderSourceSharedVirtualDirectory(TEXT("/Plugin/FX/Niagara/Shared/"));
}

FDelegateHandle INiagaraShaderModule::SetOnProcessShaderCompilationQueue(FOnProcessQueue InOnProcessQueue)
{
	checkf(OnProcessQueue.IsBound() == false, TEXT("Shader processing queue delegate already set."));
	OnProcessQueue = InOnProcessQueue;
	return OnProcessQueue.GetHandle();
}

void INiagaraShaderModule::ResetOnProcessShaderCompilationQueue(FDelegateHandle DelegateHandle)
{
	checkf(OnProcessQueue.GetHandle() == DelegateHandle, TEXT("Can only reset the process compilation queue delegate with the handle it was created with."));
	OnProcessQueue.Unbind();
}

void INiagaraShaderModule::ProcessShaderCompilationQueue()
{
	checkf(OnProcessQueue.IsBound(), TEXT("Can not process shader queue.  Delegate was never set."));
	return OnProcessQueue.Execute();
}

FDelegateHandle INiagaraShaderModule::SetOnRequestDefaultDataInterfaceHandler(FOnRequestDefaultDataInterface InHandler)
{
	checkf(OnRequestDefaultDataInterface.IsBound() == false, TEXT("Shader OnRequestDefaultDataInterface delegate already set."));
	OnRequestDefaultDataInterface = InHandler;
	return OnRequestDefaultDataInterface.GetHandle();
}

void  INiagaraShaderModule::ResetOnRequestDefaultDataInterfaceHandler()
{
	OnRequestDefaultDataInterface.Unbind();
}

UNiagaraDataInterfaceBase* INiagaraShaderModule::RequestDefaultDataInterface(const FString& DIClassName)
{
	if (!OnRequestDefaultDataInterface.IsBound())
	{
		UE_LOGF(LogTemp, Log, "NiagaraShader requires data interface '%ls' and is serialized before Niagara module startup, attempting to load Niagara module.", *DIClassName);

		EModuleLoadResult LoadResult;
		FModuleManager::Get().LoadModuleWithFailureReason(TEXT("Niagara"), LoadResult);
		if (LoadResult != EModuleLoadResult::Success)
		{
			UE_LOGF(LogTemp, Warning, "Module manager failed to load Niagara module LoadResult(%d).", int(LoadResult));
		}

		if ( !OnRequestDefaultDataInterface.IsBound() )
		{
			UE_LOGF(LogTemp, Fatal, "Failed to start Niagara module for serialization of shader that requires data interface '%ls'.  This is a load order issue that can not be resolved, content dependencies will need fixing.", *DIClassName);
		}
	}

	return OnRequestDefaultDataInterface.Execute(DIClassName);
}
