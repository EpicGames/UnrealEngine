// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraShaderParticleID.h"
#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "PipelineStateCache.h"
#include "DataDrivenShaderPlatformInfo.h"

int32 GNiagaraWaveIntrinsics = 0; // TODO: Enable this
FAutoConsoleVariableRef CVarGNiagaraWaveIntrinsics(
	TEXT("Niagara.WaveIntrinsics"),
	GNiagaraWaveIntrinsics,
	TEXT("")
);

//////////////////////////////////////////////////////////////////////////

class FNiagaraInitFreeIDBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNiagaraInitFreeIDBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraInitFreeIDBufferCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<int>,	RWNewBuffer)
		SHADER_PARAMETER_SRV(Buffer<int>,	ExistingBuffer)
		SHADER_PARAMETER(uint32,			NumNewElements)
		SHADER_PARAMETER(uint32,			NumExistingElements)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FNiagaraInitFreeIDBufferCS, "/Plugin/FX/Niagara/Private/NiagaraInitFreeIDBuffer.usf", "InitIDBufferCS", SF_Compute);

void NiagaraInitGPUFreeIDList(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIUnorderedAccessView* NewBufferUAV, uint32 NewBufferNumElements, FRHIShaderResourceView* ExistingBufferSRV, uint32 ExistingBufferNumElements)
{
	// To simplify the shader code, the size of the ID table must be a multiple of the thread count.
	check(NewBufferNumElements % FNiagaraInitFreeIDBufferCS::ThreadGroupSize == 0);

	// Shrinking is not supported.
	check(NewBufferNumElements >= ExistingBufferNumElements);
	const uint32 NumNewElements = NewBufferNumElements - ExistingBufferNumElements;

	TShaderMapRef<FNiagaraInitFreeIDBufferCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp(NewBufferNumElements, FNiagaraInitFreeIDBufferCS::ThreadGroupSize);

	FNiagaraInitFreeIDBufferCS::FParameters Parameters;
	Parameters.RWNewBuffer			= NewBufferUAV;
	Parameters.ExistingBuffer		= ExistingBufferSRV;
	Parameters.NumNewElements		= NumNewElements;
	Parameters.NumExistingElements	= ExistingBufferNumElements;

	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumThreadGroups, 1, 1));
}

//////////////////////////////////////////////////////////////////////////

class NiagaraComputeFreeIDsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(NiagaraComputeFreeIDsCS);
	SHADER_USE_PARAMETER_STRUCT(NiagaraComputeFreeIDsCS, FGlobalShader);

	class FWaveIntrinsicsDim : SHADER_PERMUTATION_BOOL("USE_WAVE_INTRINSICS");
	using FPermutationDomain = TShaderPermutationDomain<FWaveIntrinsicsDim>;

	static constexpr uint32 ThreadGroupSize_WaveEnabled = 64;
	static constexpr uint32 ThreadGroupSize_WaveDisabled = 128;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveIntrinsicsDim>() && !FDataDrivenShaderPlatformInfo::GetSupportsIntrinsicWaveOnce(Parameters.Platform))
		{
			// Only some platforms support wave intrinsics.
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bWithWaveIntrinsics = PermutationVector.Get<FWaveIntrinsicsDim>();

		OutEnvironment.SetDefine(TEXT("USE_WAVE_INTRINSICS"), bWithWaveIntrinsics ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), bWithWaveIntrinsics ? ThreadGroupSize_WaveEnabled : ThreadGroupSize_WaveDisabled);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(Buffer<int>,	IDToIndexTable)
		SHADER_PARAMETER_UAV(RWBuffer<int>,	RWFreeIDList)
		SHADER_PARAMETER_UAV(RWBuffer<int>,	RWFreeIDListSizes)
		SHADER_PARAMETER(uint32,			FreeIDListIndex)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(NiagaraComputeFreeIDsCS, "/Plugin/FX/Niagara/Private/NiagaraComputeFreeIDs.usf", "ComputeFreeIDs", SF_Compute);

void NiagaraComputeGPUFreeIDs(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIShaderResourceView* IDToIndexTableSRV, uint32 NumIDs, FRHIUnorderedAccessView* FreeIDUAV, FRHIUnorderedAccessView* FreeIDListSizesUAV, uint32 FreeIDListIndex)
{
	const EShaderPlatform Platform = GShaderPlatformForFeatureLevel[FeatureLevel];
	const bool bUseWaveOps = FDataDrivenShaderPlatformInfo::GetSupportsIntrinsicWaveOnce(Platform) && GNiagaraWaveIntrinsics != 0;
	const uint32 ThreadGroupSize = bUseWaveOps ? NiagaraComputeFreeIDsCS::ThreadGroupSize_WaveEnabled : NiagaraComputeFreeIDsCS::ThreadGroupSize_WaveDisabled;

	check(NumIDs % ThreadGroupSize == 0);

	NiagaraComputeFreeIDsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<NiagaraComputeFreeIDsCS::FWaveIntrinsicsDim>(bUseWaveOps);

	TShaderMapRef<NiagaraComputeFreeIDsCS> ComputeShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp(NumIDs, ThreadGroupSize);

	NiagaraComputeFreeIDsCS::FParameters Parameters;
	Parameters.IDToIndexTable		= IDToIndexTableSRV;
	Parameters.RWFreeIDList			= FreeIDUAV;
	Parameters.RWFreeIDListSizes	= FreeIDListSizesUAV;
	Parameters.FreeIDListIndex		= FreeIDListIndex;

	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumThreadGroups, 1, 1));
}

//////////////////////////////////////////////////////////////////////////

class NiagaraFillIntBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(NiagaraFillIntBufferCS);
	SHADER_USE_PARAMETER_STRUCT(NiagaraFillIntBufferCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<int>,	BufferUAV)
		SHADER_PARAMETER(uint32,			NumElements)
		SHADER_PARAMETER(int,				FillValue)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(NiagaraFillIntBufferCS, "/Plugin/FX/Niagara/Private/NiagaraFillIntBuffer.usf", "FillIntBuffer", SF_Compute);

void NiagaraFillGPUIntBuffer(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIUnorderedAccessView* BufferUAV, uint32 NumElements, int32 FillValue)
{
	TShaderMapRef<NiagaraFillIntBufferCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));
	const uint32 NumThreadGroups = FMath::DivideAndRoundUp(NumElements, NiagaraFillIntBufferCS::ThreadGroupSize);

	NiagaraFillIntBufferCS::FParameters Parameters;
	Parameters.BufferUAV	= BufferUAV;
	Parameters.NumElements	= NumElements;
	Parameters.FillValue	= FillValue;

	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumThreadGroups, 1, 1));
}
