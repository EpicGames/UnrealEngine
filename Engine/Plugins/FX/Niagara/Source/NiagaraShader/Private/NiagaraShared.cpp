// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraShared.cpp: Shared Niagara compute shader implementation.
=============================================================================*/

#include "NiagaraShared.h"
#include "NiagaraShaderModule.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraShaderType.h"
#include "NiagaraShader.h"
#include "NiagaraScriptBase.h"
#include "NiagaraScript.h"		//-TODO: This should be fixed so we are not reading structures from modules we do not depend on
#include "PipelineStateCache.h"
#include "Stats/StatsMisc.h"
#include "Misc/App.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "PSOPrecache.h"
#include "ShaderCompiler.h"
#include "ShaderSerialization.h"
#include "NiagaraShaderCompilationManager.h"
#include "Modules/ModuleManager.h"
#include "NiagaraCustomVersion.h"
#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "ShaderCodeLibrary.h"
#endif
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraShared)

IMPLEMENT_TYPE_LAYOUT(FNiagaraDataInterfaceParamRef);
IMPLEMENT_TYPE_LAYOUT(FNiagaraShaderMapContent);
IMPLEMENT_TYPE_LAYOUT(FNiagaraShaderMapId);

//* CVars */
int32 GNiagaraTranslatorFailIfNotSetSeverity = 3;
static FAutoConsoleVariableRef CVarNiagaraTranslatorSilenceFailIfNotSet(
	TEXT("fx.Niagara.FailIfNotSetSeverity"),
	GNiagaraTranslatorFailIfNotSetSeverity,
	TEXT("The severity of messages emitted by Parameters with Default Mode \"Fail If Not Set\". 3 = Error, 2 = Warning, 1= Log, 0 = Disabled.\n"),
	ECVF_Default
);

int32 GbNiagaraEnableTraversalCache = 1;
static FAutoConsoleVariableRef CVarEnableTraversalCache(
	TEXT("fx.Niagara.EnableTraversalCache"),
	GbNiagaraEnableTraversalCache,
	TEXT("If > 0 the new traversal cache will be used to speed up utilities like GetStackFunctionInputs.\n"),
	ECVF_Default);

#if WITH_EDITOR
	FNiagaraCompilationQueue* FNiagaraCompilationQueue::Singleton = nullptr;
#endif

FNiagaraShaderScript::FNiagaraShaderScript()
	: BaseVMScript(nullptr)
	, GameThreadShaderMap(nullptr)
	, RenderingThreadShaderMap(nullptr)
	, ScriptParametersMetadata(MakeShared<FNiagaraShaderScriptParametersMetadata>())
	, ScriptParametersMetadata_RT(ScriptParametersMetadata)
	, FeatureLevel(GMaxRHIFeatureLevel)
	, ShaderPlatform(SP_NumPlatforms)
	, bLoadedCookedShaderMapId(false)
	, bLoadedFromCookedMaterial(false)
	, bQueuedForRelease(false)
{
}

FNiagaraShaderScript::~FNiagaraShaderScript()
{
#if WITH_EDITOR
	if (OutstandingCompileShaderMapIds.Num() > 0)
	{
		CancelCompilation();
	}
#endif
}

#if WITH_EDITOR
/** Populates OutEnvironment with defines needed to compile shaders for this script. */
void FNiagaraShaderScript::SetupShaderCompilationEnvironment(
	EShaderPlatform Platform,
	FShaderCompilerEnvironment& OutEnvironment
	) const
{
	OutEnvironment.SetDefine(TEXT("GPU_SIMULATION_SHADER"), TEXT("1"));
}
#endif // WITH_EDITOR

bool FNiagaraShaderScript::ShouldCache(EShaderPlatform Platform, const FShaderType* ShaderType) const
{
	check(ShaderType->GetNiagaraShaderType() != nullptr);
	if (BaseVMScript)
	{
		if (!BaseVMScript->ShouldCompile(Platform))
		{
			return false;
		}
	}
	return true;
}

void FNiagaraShaderScript::ModifyCompilationEnvironment(EShaderPlatform Platform, struct FShaderCompilerEnvironment& OutEnvironment) const
{
	if ( BaseVMScript )
	{
		BaseVMScript->ModifyCompilationEnvironment(Platform, OutEnvironment);
	}
}

bool FNiagaraShaderScript::GetUsesCompressedAttributes() const
{
	return AdditionalDefines.Contains(TEXT("CompressAttributes"));
}

#if WITH_EDITOR
void FNiagaraShaderScript::NotifyCompilationFinished()
{
	UpdateCachedData_PostCompile();

	OnCompilationCompleteDelegate.Broadcast();
}

void FNiagaraShaderScript::CancelCompilation()
{
	check(IsInGameThread());
	bool bWasPending = FNiagaraShaderMap::RemovePendingScript(this);
	FNiagaraCompilationQueue::Get()->RemovePending(this);

	// don't spam the log if no cancelling actually happened : 
	if (bWasPending)
	{
		UE_LOGF(LogShaders, Log, "CancelCompilation %p.", this);
	}
	OutstandingCompileShaderMapIds.Empty();
}

void FNiagaraShaderScript::RemoveOutstandingCompileId(const int32 OldOutstandingCompileShaderMapId)
{
	check(IsInGameThread());
	if (0 <= OutstandingCompileShaderMapIds.Remove(OldOutstandingCompileShaderMapId))
	{
		//UE_LOGF(LogShaders, Log, "RemoveOutstandingCompileId %p %d", this, OldOutstandingCompileShaderMapId);
	}
}
#endif // WITH_EDITOR

void FNiagaraShaderScript::Invalidate()
{
#if WITH_EDITOR
	CancelCompilation();
#endif
	ReleaseShaderMap();
#if WITH_EDITOR
	CompileErrors.Empty();
	HlslOutput.Empty();
#endif
}

bool FNiagaraShaderScript::IsSame(const FNiagaraShaderMapId& InId) const
{
	if (InId.ReferencedCompileHashes.Num() != ReferencedCompileHashes.Num() ||
		InId.AdditionalDefines.Num() != AdditionalDefines.Num() ||
		InId.AdditionalVariables.Num() != AdditionalVariables.Num())
	{
		return false;
	}

	for (int32 i = 0; i < ReferencedCompileHashes.Num(); ++i)
	{
		if (ReferencedCompileHashes[i] != InId.ReferencedCompileHashes[i])
		{
			return false;
		}
	}
	for (int32 i = 0; i < AdditionalDefines.Num(); ++i)
	{
		if (AdditionalDefines[i] != *InId.AdditionalDefines[i])
		{
			return false;
		}
	}
	for (int32 i = 0; i < AdditionalVariables.Num(); ++i)
	{
		if (AdditionalVariables[i] != *InId.AdditionalVariables[i])
		{
			return false;
		}
	}

	return
		InId.FeatureLevel == FeatureLevel &&/*
		InId.BaseScriptID == BaseScriptId &&*/
		InId.bUsesRapidIterationParams == bUsesRapidIterationParams &&
		InId.BaseCompileHash == BaseCompileHash &&
		InId.CompilerVersionID == CompilerVersionId;
}

bool FNiagaraShaderScript::IsShaderMapComplete() const
{
	if (GameThreadShaderMap == nullptr)
	{
		return false;
	}

#if WITH_EDITOR
	if (FNiagaraShaderMap::GetShaderMapBeingCompiled(this) != nullptr)
	{
		return false;
	}
#endif

	if (!GameThreadShaderMap->IsValid())
	{
		return false;
	}

	for (int i=0; i < GetNumPermutations(); ++i)
	{
		if (GameThreadShaderMap->GetShader<FNiagaraShader>(i).IsNull())
		{
			return false;
		}
	}
	return true;
}

#if WITH_EDITOR
void FNiagaraShaderScript::GetDependentShaderTypes(EShaderPlatform Platform, TArray<FShaderType*>& OutShaderTypes) const
{
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FNiagaraShaderType* ShaderType = ShaderTypeIt->GetNiagaraShaderType();

		if ( ShaderType && ShaderType->ShouldCache(Platform, this) && ShouldCache(Platform, ShaderType) )
		{
			OutShaderTypes.Add(ShaderType);
		}
	}
}



void FNiagaraShaderScript::GetShaderMapId(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, FNiagaraShaderMapId& OutId) const
{
	if (bLoadedCookedShaderMapId)
	{
		OutId = CookedShaderMapId;
	}
	else
	{
		INiagaraShaderModule* Module = INiagaraShaderModule::Get();

		OutId.FeatureLevel = GetFeatureLevel();
		OutId.bUsesRapidIterationParams = bUsesRapidIterationParams;		
		BaseCompileHash.ToSHAHash(OutId.BaseCompileHash);
		OutId.CompilerVersionID = FNiagaraCustomVersion::GetLatestScriptCompileVersion();

		OutId.ReferencedCompileHashes.Reserve(ReferencedCompileHashes.Num());
		for (const FNiagaraCompileHash& Hash : ReferencedCompileHashes)
		{
			Hash.ToSHAHash(OutId.ReferencedCompileHashes.AddDefaulted_GetRef());
		}

		OutId.AdditionalDefines.Empty(AdditionalDefines.Num());
		for(const FString& Define : AdditionalDefines)
		{
			OutId.AdditionalDefines.Emplace(Define);
		}

		OutId.AdditionalVariables.Empty(AdditionalVariables.Num());
		for(const FString& Variable : AdditionalVariables)
		{
			OutId.AdditionalVariables.Emplace(Variable);
		}

		TArray<FShaderType*> DependentShaderTypes;
		GetDependentShaderTypes(Platform, DependentShaderTypes);
		for (FShaderType* ShaderType : DependentShaderTypes)
		{
			OutId.ShaderTypeDependencies.Emplace(ShaderType, Platform);
		}

		if (TargetPlatform)
		{
			OutId.LayoutParams.InitializeForPlatform(TargetPlatform->IniPlatformName(), TargetPlatform->HasEditorOnlyData());
		}
		else
		{
			OutId.LayoutParams.InitializeForCurrent();
		}
	}
}
#endif // WITH_EDITOR


void FNiagaraShaderScript::AddReferencedObjects(FReferenceCollector& Collector)
{
}

void FNiagaraShaderScript::ReleaseShaderMap()
{
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap = nullptr;

		if (!bQueuedForRelease)
		{
			FNiagaraShaderScript* Script = this;
			ENQUEUE_RENDER_COMMAND(ReleaseShaderMap)(
				[Script](FRHICommandListImmediate& RHICmdList)
				{
					Script->SetRenderingThreadShaderMap(nullptr);
				});
		}

		UpdateCachedData_All();
	}
}

void FNiagaraShaderScript::SerializeShaderMap(FArchive& Ar)
{
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;
	Ar << NumPermutations;
	Ar << BaseCompileHash;

	if (Ar.IsLoading())
	{
		bLoadedFromCookedMaterial = bCooked;
	}

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogShaders, Fatal, "This platform requires cooked packages, and shaders were not cooked into this Niagara script %ls.", *GetFriendlyName());
	}

	if (bCooked)
	{
		if (Ar.IsCooking())
		{
#if WITH_EDITOR
			FinishCompilation();

			bool bValid = GameThreadShaderMap != nullptr && GameThreadShaderMap->CompiledSuccessfully();
			Ar << bValid;

			if (bValid)
			{
				// associate right here
				if (BaseVMScript)
				{
					GameThreadShaderMap->AssociateWithAsset(BaseVMScript->GetOutermost()->GetFName());
				}
				FShaderSerializeContext Ctx(Ar);
				GameThreadShaderMap->Serialize(Ctx);
			}
			//else if (GameThreadShaderMap != nullptr && !GameThreadShaderMap->CompiledSuccessfully())
			//{
			//	FString Name;
			//	UE_LOGF(LogShaders, Error, "Failed to compile Niagara shader %ls.", *GetFriendlyName());
			//}
#endif
		}
		else
		{
			bool bValid = false;
			Ar << bValid;

			if (bValid)
			{
				FNiagaraShaderMapRef LoadedShaderMap = new FNiagaraShaderMap();
				FShaderSerializeContext Ctx(Ar);
				Ctx.bLoadingCooked = true;
				bool bLoaded = LoadedShaderMap->Serialize(Ctx);

				// Toss the loaded shader data if this is a server only instance
				//@todo - don't cook it in the first place
				if (FApp::CanEverRender() && bLoaded)
				{
					GameThreadShaderMap = RenderingThreadShaderMap = LoadedShaderMap;
					GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());

					UpdateCachedData_PostCompile(true);
				}
			}
		}
	}
}

FName FNiagaraShaderScript::GetOwnerFName() const
{
	const UNiagaraScriptBase* Owner = GetBaseVMScript();
	return Owner ? Owner->GetOutermost()->GetFName() : NAME_None;
}

#if WITH_EDITOR
void FNiagaraShaderScript::SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, FStableShaderKeyAndValue& SaveKeyVal)
{
	if (GameThreadShaderMap && GameThreadShaderMap->IsValid())
	{
		FString FeatureLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);
		SaveKeyVal.FeatureLevel = FName(*FeatureLevelName);

		static FName FName_Num(TEXT("Num")); // Niagara resources aren't associated with a quality level, so we use Num which for the materials means "Default"
		SaveKeyVal.QualityLevel = FName_Num;

		GameThreadShaderMap->SaveShaderStableKeys(TargetShaderPlatform, SaveKeyVal);
	}
}
#endif // WITH_EDITOR

void FNiagaraShaderScript::SetScript(UNiagaraScriptBase* InScript, ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, const FGuid& InCompilerVersionID,  const TArray<FString>& InAdditionalDefines, const TArray<FString>& InAdditionalVariables,
		const FNiagaraCompileHash& InBaseCompileHash, const TArray<FNiagaraCompileHash>& InReferencedCompileHashes, 
		bool bInUsesRapidIterationParams, FString InFriendlyName)
{
	checkf(InBaseCompileHash.IsValid(), TEXT("Invalid base compile hash.  Script caching will fail."))
	BaseVMScript = InScript;
	CompilerVersionId = InCompilerVersionID;
	AdditionalDefines = InAdditionalDefines;
	AdditionalVariables = InAdditionalVariables;
	bUsesRapidIterationParams = bInUsesRapidIterationParams;
	BaseCompileHash = InBaseCompileHash;
	ReferencedCompileHashes = InReferencedCompileHashes;
	FriendlyName = InFriendlyName;
	SetFeatureLevel(InFeatureLevel);
	ShaderPlatform = InShaderPlatform;

	UpdateCachedData_All();
}

#if WITH_EDITOR
bool FNiagaraShaderScript::MatchesScript(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, const FNiagaraVMExecutableDataId& ScriptId) const
{
	return CompilerVersionId == ScriptId.CompilerVersionID
		&& AdditionalDefines == ScriptId.AdditionalDefines
		&& bUsesRapidIterationParams == ScriptId.bUsesRapidIterationParams
		&& BaseCompileHash == ScriptId.BaseScriptCompileHash
		&& ReferencedCompileHashes == ScriptId.ReferencedCompileHashes
		&& FeatureLevel == InFeatureLevel
		&& ShaderPlatform == InShaderPlatform;
}
#endif // WITH_EDITOR

void FNiagaraShaderScript::SetRenderingThreadShaderMap(FNiagaraShaderMap* InShaderMap)
{
	check(IsInRenderingThread());
	RenderingThreadShaderMap = InShaderMap;
}

bool FNiagaraShaderScript::IsCompilationFinished() const
{
#if WITH_EDITOR
	check(IsInGameThread());
	if (OutstandingCompileShaderMapIds.Num() == 0)
	{
		return true;
	}
	bool bRet = GameThreadShaderMap && GameThreadShaderMap.IsValid() && GameThreadShaderMap->IsCompilationFinalized();
	return bRet;
#else
	return true;
#endif
}

void FNiagaraShaderScript::SetRenderThreadCachedData(const FNiagaraShaderMapCachedData& CachedData)
{
	CachedData_RenderThread = CachedData;
}

bool FNiagaraShaderScript::QueueForRelease(FThreadSafeBool& Fence)
{
	check(!bQueuedForRelease);

	if (BaseVMScript)
	{
		bQueuedForRelease = true;
		Fence = false;
		FThreadSafeBool* Released = &Fence;

		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)(
			[Released](FRHICommandListImmediate& RHICmdList)
			{
				*Released = true;
			});
	}

	return bQueuedForRelease;
}

void FNiagaraShaderScript::UpdateCachedData_All()
{
	UpdateCachedData_PreCompile();
	UpdateCachedData_PostCompile();
}

void FNiagaraShaderScript::UpdateCachedData_PreCompile()
{
	if (BaseVMScript)
	{
		TConstArrayView<FSimulationStageMetaData> SimulationStages = BaseVMScript->GetSimulationStageMetaData();
		NumPermutations = SimulationStages.Num();
	}
	else
	{
		NumPermutations = 0;
	}
}

void FNiagaraShaderScript::UpdateCachedData_PostCompile(bool bCalledFromSerialize)
{
	check(IsInGameThread() || bCalledFromSerialize);

	FNiagaraShaderMapCachedData CachedData;
	CachedData.NumPermutations = GetNumPermutations();
	CachedData.bIsComplete = 1;
	CachedData.bExternalConstantBufferUsed = 0;
	CachedData.bViewUniformBufferUsed = 0;

	if (GameThreadShaderMap != nullptr && GameThreadShaderMap->IsValid())
	{		
		FPSOPrecacheDataArray PSOInitializers;
		PSOInitializers.Reserve(CachedData.NumPermutations);

		static const int32 PSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(TEXT("NiagaraComputePSOCollector"));

		for (int32 iPermutation = 0; iPermutation < CachedData.NumPermutations; ++iPermutation)
		{
			TNiagaraShaderRef<FShader> Shader = GameThreadShaderMap->GetShader(&FNiagaraShader::GetStaticType(), iPermutation);
			if (!Shader.IsValid())
			{
				CachedData.bIsComplete = 0;
				break;
			}
			FNiagaraShader* NiagaraShader = static_cast<FNiagaraShader*>(Shader.GetShader());

			for (int i = 0; i < 2; ++i)
			{
				const uint32 BitToSet = 1 << i;
				CachedData.bExternalConstantBufferUsed |= NiagaraShader->ExternalConstantBufferParam[i].IsBound() ? BitToSet : 0;
			}
			CachedData.bViewUniformBufferUsed |= NiagaraShader->bNeedsViewUniformBuffer;

			// request precache the compute shader
			// Note: this function can be called for different shader platforms so only precache if it's for the platform we are running
			if (GMaxRHIShaderPlatform == GameThreadShaderMap->GetShaderPlatform())
			{
				if (IsPSOShaderPreloadingEnabled())
				{
					FGraphEventArray PreloadEvent;
					GameThreadShaderMap->GetResource()->PreloadShader(Shader->GetResourceIndex(), PreloadEvent);
				}
				else if (IsResourcePSOPrecachingEnabled() || IsComponentPSOPrecachingEnabled())
				{
					check(NiagaraShader->GetFrequency() == SF_Compute);
					FRHIShader* RHIShader = GameThreadShaderMap->GetResource()->GetShader(Shader->GetResourceIndex());
					FRHIComputeShader* RHIComputeShader = static_cast<FRHIComputeShader*>(RHIShader);

					FPSOPrecacheData PSOPrecacheData;
					PSOPrecacheData.Type = FPSOPrecacheData::EType::Compute;
					PSOPrecacheData.ComputeInitializer = FComputePipelineStateInitializer(RHIComputeShader);
					PSOPrecacheData.InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
					PSOPrecacheData.Priority = EPSOPrecachePriority::High;
				#if PSO_PRECACHING_VALIDATE
					PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
					PSOPrecacheData.VertexFactoryType = nullptr;
				#endif // PSO_PRECACHING_VALIDATE
					PSOInitializers.Add(MoveTemp(PSOPrecacheData));
				}
			}
		}

		if (!PSOInitializers.IsEmpty())
		{
			RequestPrecachePSOs(EPSOPrecacheType::Global, PSOInitializers);
		}
	}
	else
	{
		CachedData.bIsComplete = 0;
	}

	if (bCalledFromSerialize)
	{
		CachedData_RenderThread = MoveTemp(CachedData);
	}
	else if (!bQueuedForRelease)
	{
		ENQUEUE_RENDER_COMMAND(UpdateCachedData)(
				[Script_RT = this, CachedData_RT = CachedData](FRHICommandListImmediate& RHICmdList)
				{
					Script_RT->SetRenderThreadCachedData(CachedData_RT);
				});
	}
}

/**
* Cache the script's shaders
*/
#if WITH_EDITOR

bool FNiagaraShaderScript::CacheShaders(bool bApplyCompletedShaderMapForRendering, bool bForceRecompile, bool bSynchronous, const ITargetPlatform* TargetPlatform)
{
	FNiagaraShaderMapId NoStaticParametersId;
	GetShaderMapId(ShaderPlatform, TargetPlatform, NoStaticParametersId);
	return CacheShaders(NoStaticParametersId, bApplyCompletedShaderMapForRendering, bForceRecompile, bSynchronous);
}

/**
* Caches the shaders for this script
*/
bool FNiagaraShaderScript::CacheShaders(const FNiagaraShaderMapId& ShaderMapId, bool bApplyCompletedShaderMapForRendering, bool bForceRecompile, bool bSynchronous)
{
	bool bSucceeded = false;

	check(IsInGameThread());

	{
		GameThreadShaderMap = nullptr;
		{
			extern FCriticalSection GIdToNiagaraShaderMapCS;
			FScopeLock ScopeLock(&GIdToNiagaraShaderMapCS);
			// Find the script's cached shader map.
			GameThreadShaderMap = FNiagaraShaderMap::FindId(ShaderMapId, ShaderPlatform);
		}

		// Attempt to load from the derived data cache if we are uncooked
		if (!bForceRecompile && !GameThreadShaderMap && !FPlatformProperties::RequiresCookedData())
		{
			FNiagaraShaderMap::LoadFromDerivedDataCache(this, ShaderMapId, ShaderPlatform, GameThreadShaderMap);
			if (GameThreadShaderMap && GameThreadShaderMap->IsValid())
			{
				UE_LOGF(LogShaders, Verbose, "Loaded shader for Niagara script %ls from DDC", *GetFriendlyName());
			}
			else
			{
				UE_LOGF(LogShaders, Display, "Loading shader for Niagara script %ls from DDC failed. Shader needs recompile.", *GetFriendlyName());
			}
		}
	}

	bool bAssumeShaderMapIsComplete = false;
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAssumeShaderMapIsComplete = FPlatformProperties::RequiresCookedData();
#endif

	UpdateCachedData_PreCompile();

	if (GameThreadShaderMap && GameThreadShaderMap->TryToAddToExistingCompilationTask(this))
	{
		//FNiagaraShaderMap::ShaderMapsBeingCompiled.Find(GameThreadShaderMap);
		OutstandingCompileShaderMapIds.AddUnique(GameThreadShaderMap->GetCompilingId());
		UE_LOGF(LogShaders, Log, "CacheShaders AddUniqueExisting %p %d", this, GameThreadShaderMap->GetCompilingId());

		// Reset the shader map so we fall back to CPU sim until the compile finishes.
		GameThreadShaderMap = nullptr;
		bSucceeded = true;
	}
	else if (bForceRecompile || !GameThreadShaderMap || !(bAssumeShaderMapIsComplete || GameThreadShaderMap->IsComplete(this, false)))
	{
		if (FPlatformProperties::RequiresCookedData())
		{
			UE_LOGF(LogShaders, Log, "Can't compile %ls with cooked content!", *GetFriendlyName());
			// Reset the shader map so we fall back to CPU sim
			GameThreadShaderMap = nullptr;
		}
		else
		{
			UE_LOGF(LogShaders, Log, "%ls cached shader map for script %ls, compiling.", GameThreadShaderMap? TEXT("Incomplete") : TEXT("Missing"), *GetFriendlyName());

			// If there's no cached shader map for this script compile a new one.
			// This is just kicking off the compile, GameThreadShaderMap will not be complete yet
			bSucceeded = BeginCompileShaderMap(ShaderMapId, GameThreadShaderMap, bApplyCompletedShaderMapForRendering, bSynchronous);

			if (!bSucceeded)
			{
				GameThreadShaderMap = nullptr;
			}
		}
	}
	else
	{
		bSucceeded = true;
	}

	UpdateCachedData_PostCompile();

	if (!bQueuedForRelease)
	{
		FNiagaraShaderScript* Script = this;
		FNiagaraShaderMap* LoadedShaderMap = GameThreadShaderMap;
		ENQUEUE_RENDER_COMMAND(FSetShaderMapOnScriptResources)(
			[Script, LoadedShaderMap](FRHICommandListImmediate& RHICmdList)
			{
				Script->SetRenderingThreadShaderMap(LoadedShaderMap);
			});
	}

	return bSucceeded;
}

void FNiagaraShaderScript::FinishCompilation()
{
	TArray<int32> ShaderMapIdsToFinish;
	GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);

	if (ShaderMapIdsToFinish.Num() > 0)
	{
		for (int32 i = 0; i < ShaderMapIdsToFinish.Num(); i++)
		{
			UE_LOGF(LogShaders, Log, "FinishCompilation()[%d] %ls id %d!", i, *GetFriendlyName(), ShaderMapIdsToFinish[i]);
		}
		// Block until the shader maps that we will save have finished being compiled
		// NIAGARATODO: implement when async compile works
		GNiagaraShaderCompilationManager.FinishCompilation(ShaderMapIdsToFinish);

		// Shouldn't have anything left to do...
		TArray<int32> ShaderMapIdsToFinish2;
		GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish2);
		if (ShaderMapIdsToFinish2.Num() != 0)
		{
			UE_LOGF(LogShaders, Warning, "Skipped multiple Niagara shader maps for compilation! May be indicative of no support for a given platform. Count: %d", ShaderMapIdsToFinish2.Num());
		}
	}
}

#endif // WITH_EDITOR

void FNiagaraShaderScript::BuildScriptParametersMetadata(const FNiagaraShaderScriptParametersMetadata& InScriptParametersMetadata)
{
	ScriptParametersMetadata = MakeShared<FNiagaraShaderScriptParametersMetadata>();
	ScriptParametersMetadata->DataInterfaceParamInfo = InScriptParametersMetadata.DataInterfaceParamInfo;

	FShaderParametersMetadataBuilder ShaderMetadataBuilder(TShaderParameterStructTypeInfo<FNiagaraShader::FParameters>::GetStructMetadata());

	// Build meta data for each data interface
	INiagaraShaderModule* ShaderModule = INiagaraShaderModule::Get();
	for (FNiagaraDataInterfaceGPUParamInfo& DataInterfaceParamInfo : ScriptParametersMetadata->DataInterfaceParamInfo)
	{
		UNiagaraDataInterfaceBase* CDODataInterface = ShaderModule->RequestDefaultDataInterface(*DataInterfaceParamInfo.DIClassName);
		if (CDODataInterface == nullptr)
		{
			Invalidate();
			continue;
		}

		const uint32 NextMemberOffset = ShaderMetadataBuilder.GetNextMemberOffset();
		FNiagaraShaderParametersBuilder ShaderParametersBuilder(DataInterfaceParamInfo, ScriptParametersMetadata->LooseMetadataNames, ScriptParametersMetadata->StructIncludeInfos, ShaderMetadataBuilder);
		CDODataInterface->BuildShaderParameters(ShaderParametersBuilder);
		DataInterfaceParamInfo.ShaderParametersOffset = NextMemberOffset;
	}

	ScriptParametersMetadata->ShaderParametersMetadata = MakeShareable<FShaderParametersMetadata>(ShaderMetadataBuilder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("FNiagaraShaderScript")));

	ENQUEUE_RENDER_COMMAND(SetScriptParametersMetadata)(
		[this, NewMetaData=ScriptParametersMetadata](FRHICommandListImmediate& RHICmdList)
		{
			ScriptParametersMetadata_RT=NewMetaData;
		}
	);
}

FNiagaraShaderRef FNiagaraShaderScript::GetShader(int32 PermutationId) const
{
	check(!GIsThreadedRendering || !IsInGameThread());
	if (!GIsEditor || RenderingThreadShaderMap /*&& RenderingThreadShaderMap->IsComplete(this, true)*/)
	{
		return RenderingThreadShaderMap->GetShader<FNiagaraShader>(PermutationId);
	}
	return FNiagaraShaderRef();
};

FNiagaraShaderRef FNiagaraShaderScript::GetShaderGameThread(int32 PermutationId) const
{
	if (GameThreadShaderMap && GameThreadShaderMap->IsValid())
	{
		return GameThreadShaderMap->GetShader<FNiagaraShader>(PermutationId);
	}

	return FNiagaraShaderRef();
};

#if WITH_EDITOR
void FNiagaraShaderScript::GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& ShaderMapIds)
{
	// Build an array of the shader map Id's are not finished compiling.
	if (GameThreadShaderMap && GameThreadShaderMap.IsValid() && !GameThreadShaderMap->IsCompilationFinalized())
	{
		ShaderMapIds.Add(GameThreadShaderMap->GetCompilingId());
	}
	else if (OutstandingCompileShaderMapIds.Num() != 0)
	{
		ShaderMapIds.Append(OutstandingCompileShaderMapIds);
	}
}

/**
* Compiles this script for Platform, storing the result in OutShaderMap
*
* @param ShaderMapId - the set of static parameters to compile
* @param Platform - the platform to compile for
* @param OutShaderMap - the shader map to compile
* @return - true if compile succeeded or was not necessary (shader map for ShaderMapId was found and was complete)
*/
bool FNiagaraShaderScript::BeginCompileShaderMap(
	const FNiagaraShaderMapId& ShaderMapId,
	FNiagaraShaderMapRef& OutShaderMap,
	bool bApplyCompletedShaderMapForRendering,
	bool bSynchronous)
{
	check(IsInGameThread());
	bool bSuccess = false;

	STAT(double NiagaraCompileTime = 0);


	SCOPE_SECONDS_COUNTER(NiagaraCompileTime);

	// Queue hlsl generation and shader compilation - Unlike materials, we queue this here, and compilation happens from the editor module
	FNiagaraShaderMapRef NewShaderMap = new FNiagaraShaderMap();
	OutstandingCompileShaderMapIds.AddUnique(NewShaderMap->GetCompilingId());		
	UE_LOGF(LogShaders, Log, "BeginCompileShaderMap AddUnique %p %d", this, NewShaderMap->GetCompilingId());

	FNiagaraCompilationQueue::Get()->Queue(this, NewShaderMap, ShaderMapId, ShaderPlatform, bApplyCompletedShaderMapForRendering);
	if (bSynchronous)
	{
		INiagaraShaderModule NiagaraShaderModule = FModuleManager::GetModuleChecked<INiagaraShaderModule>(TEXT("NiagaraShader"));
		NiagaraShaderModule.ProcessShaderCompilationQueue();
		OutShaderMap = NewShaderMap;
	}
	else
	{
		// For async compile, set to nullptr so that we fall back to CPU side simulation until shader compile is finished
		OutShaderMap = nullptr;
	}

	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_NiagaraShaders, (float)NiagaraCompileTime);

	return true;
}

FNiagaraCompileEventSeverity FNiagaraCVarUtilities::GetCompileEventSeverityForFailIfNotSet()
{
	switch (GNiagaraTranslatorFailIfNotSetSeverity) {
	case 3:
		return FNiagaraCompileEventSeverity::Error;
	case 2:
		return FNiagaraCompileEventSeverity::Warning;
	case 1:
		return FNiagaraCompileEventSeverity::Log;
	default:
		return FNiagaraCompileEventSeverity::Log;
	};
}

bool FNiagaraCVarUtilities::GetShouldEmitMessagesForFailIfNotSet()
{
	return GNiagaraTranslatorFailIfNotSetSeverity != 0;
}

#endif // WITH_EDITOR

