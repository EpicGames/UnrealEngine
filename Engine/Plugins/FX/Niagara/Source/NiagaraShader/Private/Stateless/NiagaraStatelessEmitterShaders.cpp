// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterShaders.h"

namespace NiagaraStateless
{
	IMPLEMENT_GLOBAL_SHADER(FSimulationShaderDefaultCS, "/Plugin/FX/Niagara/Private/Stateless/NiagaraStatelessSimulationDefault.usf", "StatelessMain", SF_Compute);
}
