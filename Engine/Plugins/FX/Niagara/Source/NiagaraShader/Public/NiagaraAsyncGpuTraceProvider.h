// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHIDefinitions.h"
#include "RHIUtilities.h"
#include "FXRenderingUtils.h"

class FPrimitiveComponentId;
class FSceneInterface;
namespace ENDICollisionQuery_AsyncGpuTraceProvider { enum Type : int; }

class FNiagaraGpuComputeDispatchInterface;
class FScene;
class FSceneView;

// mirrors structure in Engine\Plugins\FX\Niagara\Shaders\Private\NiagaraAsyncGpuTraceCommon.ush
struct FNiagaraAsyncGpuTrace
{
	float Origin[3];
	float TFar;
	float Direction[3];
	uint32 CollisionGroup;
};

// mirrors structure in Engine\Plugins\FX\Niagara\Shaders\Private\NiagaraAsyncGpuTraceCommon.ush
struct FNiagaraAsyncGpuTraceResult
{
	float WorldPosition[3];
	float HitT;
	float WorldNormal[3];
	float _Pad0; // padding to force 16 byte alignment to meet requirements for VK
};

class FNiagaraAsyncGpuTraceProvider
{
public:
	using EProviderType = ENDICollisionQuery_AsyncGpuTraceProvider::Type;
	using FProviderPriorityArray = TArray<TEnumAsByte<EProviderType>>;

	FNiagaraAsyncGpuTraceProvider(EShaderPlatform InShaderPlatform, FNiagaraGpuComputeDispatchInterface* InDispatcher)
		: ShaderPlatform(InShaderPlatform)
		, Dispatcher(InDispatcher)
	{
	}

	FNiagaraAsyncGpuTraceProvider() = delete;
	virtual ~FNiagaraAsyncGpuTraceProvider() = default;

	struct FDispatchRequest
	{
		FRWBufferStructured* TracesBuffer = nullptr;
		FRWBufferStructured* ResultsBuffer = nullptr;
		FRWBuffer* TraceCountsBuffer = nullptr;
		uint32 TracesOffset = 0;
		uint32 ResultsOffset = 0;
		uint32 TraceCountsOffset = 0;
		uint32 MaxTraceCount = 0;
		uint32 MaxRetraceCount = 0;
	};

	static NIAGARASHADER_API EProviderType ResolveSupportedType(EProviderType InType, const FProviderPriorityArray& Priorities);
	static NIAGARASHADER_API bool RequiresGlobalDistanceField(EProviderType InType, const FProviderPriorityArray& Priorities);
	static NIAGARASHADER_API bool RequiresRayTracingScene(EProviderType InType, const FProviderPriorityArray& Priorities);

	/** Hash table.
		PrimIdHashTable is the main hash table that maps GPUSceneInstanceIndex to and Index we can use to store Collision Groups inside HashToCollisionGroups.
	*/
	struct FCollisionGroupHashMap
	{
		FRWBufferStructured PrimIdHashTable;
		FRWBuffer HashToCollisionGroups;
		uint32 HashTableSize = 0;
	};

	static NIAGARASHADER_API void BuildCollisionGroupHashMap(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FSceneInterface* Scene, const TMap<FPrimitiveComponentId, uint32>& CollisionGroupMap, FCollisionGroupHashMap& Result);

	static NIAGARASHADER_API TArray<TUniquePtr<FNiagaraAsyncGpuTraceProvider>> CreateSupportedProviders(EShaderPlatform ShaderPlatform, FNiagaraGpuComputeDispatchInterface* Dispatcher, const FProviderPriorityArray& Priorities);
	static NIAGARASHADER_API void ClearResults(FRHICommandList& RHICmdList, EShaderPlatform ShaderPlatform, const FDispatchRequest& Request);

	virtual bool IsAvailable() const = 0;
	virtual EProviderType GetType() const = 0;

	NIAGARASHADER_API virtual void PostRenderOpaque(FRHICommandList& RHICmdList, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash);

	virtual void IssueTraces(FRHICommandList& RHICmdList, const FDispatchRequest& Request, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI, FCollisionGroupHashMap* CollisionGroupHash)
	{
	}

	virtual void Reset()
	{
	}

	const EShaderPlatform ShaderPlatform;
	FNiagaraGpuComputeDispatchInterface* Dispatcher;

private:
	UE_NONCOPYABLE(FNiagaraAsyncGpuTraceProvider);
};

