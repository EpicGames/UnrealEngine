// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BatchedElements.h"

class FTexture;

/**
 * Batched element parameters for gathering attributes from different slices into a single color
 */
class FBatchedElementNiagara2DArrayAttribute : public FBatchedElementParameters
{
public:
	typedef TFunction<void(FRHITexture*&, FRHISamplerState*&)> FGetTextureAndSamplerDelegate;

	FBatchedElementNiagara2DArrayAttribute(const FIntVector4& InAttributeSlices, FGetTextureAndSamplerDelegate InGetTextureAndSampler = nullptr)
		: AttributeSlices(InAttributeSlices)
		, GetTextureAndSampler(InGetTextureAndSampler)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	NIAGARASHADER_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:
	FIntVector4 AttributeSlices;
	FGetTextureAndSamplerDelegate GetTextureAndSampler;
};

/**
 * Batched element parameters for gathering attributes from a volume texture into a single color
 */
class FBatchedElementNiagaraVolumeAttribute : public FBatchedElementParameters
{
public:
	typedef TFunction<void(FRHITexture*&, FRHISamplerState*&)> FGetTextureAndSamplerDelegate;

	FBatchedElementNiagaraVolumeAttribute(const FVector2f& InTileUVs, TConstArrayView<FVector3f> InAttributeUVs, TConstArrayView<int32> InAttributeChannels, FGetTextureAndSamplerDelegate InGetTextureAndSampler = nullptr)
		: TileUVs(InTileUVs)
		, AttributeUVs(InAttributeUVs)
		, AttributeChannels(InAttributeChannels)
		, GetTextureAndSampler(InGetTextureAndSampler)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	NIAGARASHADER_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:
	FVector2f								TileUVs;
	TArray<FVector3f, TInlineAllocator<4>>	AttributeUVs;
	TArray<int32, TInlineAllocator<4>>		AttributeChannels;

	FGetTextureAndSamplerDelegate			GetTextureAndSampler;
};

/**
 * Batched element parameters for inverting a color channel
 */
class FBatchedElementNiagaraInvertColorChannel : public FBatchedElementParameters
{
public:
	FBatchedElementNiagaraInvertColorChannel(uint32 InChannelMask)
		: ChannelMask(InChannelMask)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	NIAGARASHADER_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:
	uint32	ChannelMask;
};

/**
 * Simple batched element using a 2d texture
 */
class FBatchedElementNiagaraSimple : public FBatchedElementParameters
{
public:
	FBatchedElementNiagaraSimple(FMatrix InColorTransform = FMatrix::Identity, bool InAlphaBlend = false)
		: ColorTransform(InColorTransform)
		, bAlphaBlend(InAlphaBlend)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	NIAGARASHADER_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:
	FMatrix	ColorTransform;
	bool	bAlphaBlend;
};
