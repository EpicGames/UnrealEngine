// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "RenderGraphFwd.h"

class FRDGBuilder;
class FRHICommandList;
class FRHIUnorderedAccessView;

namespace NiagaraClearCounts
{
	NIAGARASHADER_API void ClearCountsInt(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef UAV, TConstArrayView<TPair<uint32, int32>> IndexAndValueArray);
	NIAGARASHADER_API void ClearCountsUInt(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef UAV, TConstArrayView<TPair<uint32, uint32>> IndexAndValueArray);

	//-TODO:RDG: Deprecated
	NIAGARASHADER_API void ClearCountsInt(FRHICommandList& RHICmdList, FRHIUnorderedAccessView* UAV, TConstArrayView<TPair<uint32, int32>> IndexAndValueArray);
	NIAGARASHADER_API void ClearCountsUInt(FRHICommandList& RHICmdList, FRHIUnorderedAccessView* UAV, TConstArrayView<TPair<uint32, uint32>> IndexAndValueArray);
}

