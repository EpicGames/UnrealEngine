// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "RenderGraphFwd.h"

class FRDGBuilder;

class FSceneView;

struct FScreenPassRenderTarget;
namespace NiagaraDebugShaders
{
	NIAGARASHADER_API void ClearUAV(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef UAV, FUintVector4 ClearValues, uint32 UIntsToSet);

	NIAGARASHADER_API void DrawDebugLines(
		class FRDGBuilder& GraphBuilder, const class FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth,
		const uint32 LineInstanceCount, FRDGBufferRef LineBuffer
	);

	NIAGARASHADER_API void DrawDebugLines(
		class FRDGBuilder& GraphBuilder, const class FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth,
		FRDGBufferRef ArgsBuffer, FRDGBufferRef LineBuffer
	);

	NIAGARASHADER_API void VisualizeTexture(
		class FRDGBuilder& GraphBuilder, const FSceneView& View, const FScreenPassRenderTarget& Output,
		const FIntPoint& Location, const int32& DisplayHeight,
		const FIntVector4& AttributesToVisualize, FRDGTextureRef Texture, const FIntVector4& NumTextureAttributes, uint32 TickCounter,
		const FVector2D& PreviewDisplayRange = FVector2D::ZeroVector
	);
}

