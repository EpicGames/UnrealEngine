// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"


class FRDGBuilder;
class FGlobalDistanceFieldParameterData;
class FGlobalDistanceFieldParameters2;

namespace FNiagaraDistanceFieldHelper
{
	NIAGARASHADER_API void SetGlobalDistanceFieldParameters(const FGlobalDistanceFieldParameterData* OptionalParameterData, FGlobalDistanceFieldParameters2& ShaderParameters);
}
