// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GPUSceneWriter.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "NiagaraMeshVertexFactory.h"
#include "SceneView.h"

DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)

class FNiagaraGPUSceneUtils
{
public:	
	BEGIN_SHADER_PARAMETER_STRUCT(FUpdateMeshParticleInstancesParams, NIAGARASHADER_API)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneWriterParameters, GPUSceneWriterParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraMeshCommonParameters, Common)
	
		SHADER_PARAMETER(uint32, PrimitiveId)
		SHADER_PARAMETER(uint32, ParticleCount)
		SHADER_PARAMETER_SRV(Buffer<uint>, GPUParticleCountBuffer)
		SHADER_PARAMETER(uint32, GPUParticleCountOffset)
		
		SHADER_PARAMETER(int, MeshIndex)
		SHADER_PARAMETER(int, MeshIndexDataOffset)
		SHADER_PARAMETER(int, RendererVisibility)
		SHADER_PARAMETER(int, VisibilityTagDataOffset)

		SHADER_PARAMETER(FVector3f, LocalBoundingCenter)
		SHADER_PARAMETER(FVector2f, DistanceCullRangeSquared)
		SHADER_PARAMETER(FVector4f, LODScreenSize)

		SHADER_PARAMETER(int, bNeedsPrevTransform)
	END_SHADER_PARAMETER_STRUCT()

	static NIAGARASHADER_API void AddUpdateMeshParticleInstancesPass(
		FRDGBuilder& GraphBuilder,
		FUpdateMeshParticleInstancesParams& Params,
		ERHIFeatureLevel::Type FeatureLevel,
		bool bPreciseMotionVectors
	);
};

