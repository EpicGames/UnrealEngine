// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "NiagaraGenerateMips.generated.h"

class FRDGBuilder;

UENUM()
enum class ENiagaraMipMapGenerationType : uint8
{
	/** Result is taken from whatever texel the sample lies on, aka Point Sampling. */
	Unfiltered,
	/** Linear blending is performed between a 2x2 (or 2x2x2) region of texels, aka Bilinear / Trilinear. */
	Linear,
	/** A blur filter across a 3x3 (or 3x3x3) region of texels. */
	Blur1 UMETA(DisplayName="Gaussian 3 texel filter"),
	/** A blur filter across a 5x5 (or 5x5x5) region of texels. */
	Blur2 UMETA(DisplayName="Gaussian 5 texel filter"),
	/** A blur filter across a 7x7 (or 7x7x7) region of texels. */
	Blur3 UMETA(DisplayName="Gaussian 7 texel filter"),
	/** A blur filter across a 9x9 (or 9x9x9) region of texels. */
	Blur4 UMETA(DisplayName="Gaussian 9 texel filter"),
};

namespace NiagaraGenerateMips
{
	NIAGARASHADER_API void GenerateMips(FRDGBuilder& GraphBuilder, FRDGTextureRef RDGTexture, ENiagaraMipMapGenerationType GenType);
}

