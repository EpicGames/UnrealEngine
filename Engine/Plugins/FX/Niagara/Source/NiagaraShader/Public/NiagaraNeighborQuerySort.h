// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"

class FRDGBuilder;

// Counting sort dispatch functions for NiagaraDataInterfaceNeighborQuery.
// Called from PostStage after the output stage has written cell assignments.
namespace NiagaraNeighborQuerySort
{
	// Number of low bits in the combined sort key reserved for quantized distance-to-cell-center.
	// The remaining (32 - NQDistBits) high bits hold the cell ID.
	// Set to 0 to disable intra-cell distance sorting (full 32-bit cell ID, no distance packing).
	// Must agree with NQ_DIST_BITS in NiagaraDataInterfaceNeighborQuery.ush — both are fed from
	// this constant via ModifyCompilationEnvironment / GetParameterDefinitionHLSL.
	static constexpr uint32 NQDistBits = 0;
	// Max valid cell count. Stored as uint64 so 2^32 is representable without overflow.
	// When NQDistBits==0 the raw cell ID is the sort key, so one value is lost to the
	// NEIGHBORQUERY_INVALID_CELL sentinel (0xFFFFFFFF). When NQDistBits>0 the dist cap
	// (NQ_DIST_MAX = 2^N-2) ensures no valid combined key reaches 0xFFFFFFFF.
	static constexpr uint64 NQMaxSortCells = (uint64(1) << (32u - NQDistBits)) - uint64(NQDistBits == 0);

	// Pass 2: Build histogram — count particles per cell
	NIAGARASHADER_API void Histogram(FRDGBuilder& GraphBuilder, FRDGBufferSRVRef CellIdSRV, FRDGBufferUAVRef CellCountUAV, uint32 NumSlots);

	// Pass 3: Exclusive prefix sum over cell counts
	NIAGARASHADER_API void PrefixSum(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef CellCountUAV, FRDGBufferUAVRef CellOffsetUAV, uint32 NumCells);

	// Pass 4: Scatter sorted particle indices (and AcquireTag satellite data)
	// When bHasPersistentIDBuffers is false, particle index is derived from slot / MaxCellsPerParticle
	// and AcquireTag is not carried.
	NIAGARASHADER_API void Scatter(FRDGBuilder& GraphBuilder, FRDGBufferSRVRef CellIdSRV, FRDGBufferSRVRef ParticleIdIndexSRV, FRDGBufferUAVRef CellOffsetUAV, FRDGBufferUAVRef ParticleListUAV, FRDGBufferSRVRef AcquireTagSRV, FRDGBufferUAVRef AcquireTagListUAV, uint32 NumSlots, uint32 MaxCellsPerParticle, bool bHasPersistentIDBuffers);
}
