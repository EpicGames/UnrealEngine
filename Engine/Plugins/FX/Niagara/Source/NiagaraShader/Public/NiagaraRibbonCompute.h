// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraRibbonCompute.h: Niagara ribbon compute shaders for initialization of ribbons on the gpu
==============================================================================*/

#pragma once

#include "GlobalShader.h"
#include "ShaderPermutation.h"
#include "ShaderParameterStruct.h"

BEGIN_SHADER_PARAMETER_STRUCT(FRibbonComputeUniformParameters, NIAGARASHADER_API)

	// Total particle Count
	SHADER_PARAMETER(uint32, TotalNumParticlesDirect)
	SHADER_PARAMETER_SRV(Buffer<uint32>, EmitterParticleCountsBuffer)
	SHADER_PARAMETER(int32, EmitterParticleCountsBufferOffset)

	// Niagara sim data
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataFloat)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataHalf)
	SHADER_PARAMETER_SRV(Buffer<int>, NiagaraParticleDataInt)
	SHADER_PARAMETER(int, NiagaraFloatDataStride)
	SHADER_PARAMETER(int, NiagaraIntDataStride)

	// Int bindings
	SHADER_PARAMETER(int, RibbonIdDataOffset)	

	// Float bindings
	SHADER_PARAMETER(int, PositionDataOffset)
	SHADER_PARAMETER(int, PrevPositionDataOffset)
	SHADER_PARAMETER(int, VelocityDataOffset)
	SHADER_PARAMETER(int, WidthDataOffset)
	SHADER_PARAMETER(int, PrevWidthDataOffset)
	SHADER_PARAMETER(int, TwistDataOffset)
	SHADER_PARAMETER(int, PrevTwistDataOffset)
	SHADER_PARAMETER(int, ColorDataOffset)
	SHADER_PARAMETER(int, FacingDataOffset)
	SHADER_PARAMETER(int, PrevFacingDataOffset)
	SHADER_PARAMETER(int, NormalizedAgeDataOffset)
	SHADER_PARAMETER(int, MaterialRandomDataOffset)
	SHADER_PARAMETER(uint32, MaterialParamValidMask)
	SHADER_PARAMETER(int, MaterialParamDataOffset)
	SHADER_PARAMETER(int, MaterialParam1DataOffset)
	SHADER_PARAMETER(int, MaterialParam2DataOffset)
	SHADER_PARAMETER(int, MaterialParam3DataOffset)
	SHADER_PARAMETER(int, DistanceFromStartOffset)
	SHADER_PARAMETER(int, U0OverrideDataOffset)
	SHADER_PARAMETER(int, V0RangeOverrideDataOffset)
	SHADER_PARAMETER(int, U1OverrideDataOffset)
	SHADER_PARAMETER(int, V1RangeOverrideDataOffset)
	SHADER_PARAMETER(int, U0DistributionMode)
	SHADER_PARAMETER(int, U1DistributionMode)

	SHADER_PARAMETER(int, RibbonLinkOrderIsFloat)
	SHADER_PARAMETER(int, RibbonLinkOrderDataOffset)

	SHADER_PARAMETER(int, RibbonTessellationMode)	
END_SHADER_PARAMETER_STRUCT()


class FRibbonHasFullRibbonID : SHADER_PERMUTATION_BOOL("RIBBONID_IS_NIAGARAID");
class FRibbonHasRibbonID : SHADER_PERMUTATION_BOOL("RIBBONID_IS_INT");

struct FNiagaraRibbonComputeCommon
{	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment, int32 ThreadGroupSize);
	
	static constexpr uint32 VertexGenReductionInitializationThreadSize = 64;
	static constexpr uint32 VertexGenReductionPropagationThreadSize = 64;
	static constexpr uint32 VertexGenReductionFinalizationThreadSize = 64;
	static constexpr uint32 VertexGenFinalizationThreadSize = 64;
	
	static constexpr uint32 IndexGenThreadSize = 64;

	static constexpr uint32 IndexGenOptimalLoopVertexLimit = 32;
};

BEGIN_SHADER_PARAMETER_STRUCT(FRibbonOrderSortParameters, NIAGARASHADER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRibbonComputeUniformParameters, Common)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, DestinationSortedIndices)	
	SHADER_PARAMETER(uint32, MergeSortSourceBlockSize)
	SHADER_PARAMETER(uint32, MergeSortDestinationBlockSize)
END_SHADER_PARAMETER_STRUCT()

/**
* Compute shader used to generate particle sort keys.
*/
class FNiagaraRibbonSortPhase1CS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonSortPhase1CS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonSortPhase1CS, FGlobalShader);

	static constexpr uint32 BubbleSortGroupWidth = 32;

	using FParameters = FRibbonOrderSortParameters;
	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
* Compute shader used to generate particle sort keys.
*/
class FNiagaraRibbonSortPhase2CS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonSortPhase2CS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonSortPhase2CS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	using FParameters = FRibbonOrderSortParameters;
	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonVertexReductionParameters, NIAGARASHADER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRibbonComputeUniformParameters, Common)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<float>, InputTangentsAndDistances)
	SHADER_PARAMETER_UAV(RWBuffer<float>, OutputTangentsAndDistances)
	SHADER_PARAMETER_SRV(Buffer<uint32>, InputMultiRibbonIndices)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, OutputMultiRibbonIndices)
	SHADER_PARAMETER_SRV(Buffer<uint32>, InputSegments)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, OutputSegments)
	SHADER_PARAMETER_SRV(Buffer<float>, InputTessellationStats)
	SHADER_PARAMETER_UAV(RWBuffer<float>, OutputTessellationStats)
	SHADER_PARAMETER_UAV(RWBuffer<FRibbonAccumulationValues>, OutputAccumulationBuffer)
	SHADER_PARAMETER(float, CurveTension)
	SHADER_PARAMETER(int, PrefixScanStride)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonVertexReductionInitializationCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonVertexReductionInitializationCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonVertexReductionInitializationCS, FGlobalShader);
	
	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;
	using FParameters = FNiagaraRibbonVertexReductionParameters;
	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonVertexReductionFinalizationParameters, NIAGARASHADER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRibbonComputeUniformParameters, Common)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<float>, TangentsAndDistances)
	SHADER_PARAMETER_SRV(Buffer<uint32>, MultiRibbonIndices)
	SHADER_PARAMETER_SRV(Buffer<uint32>, Segments)
	SHADER_PARAMETER_SRV(Buffer<float>, TessellationStats)
	SHADER_PARAMETER_SRV(Buffer<FRibbonAccumulationValues>, AccumulationBuffer)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, PackedPerRibbonData)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, OutputCommandBuffer)
	SHADER_PARAMETER(int, OutputCommandBufferIndex)
	SHADER_PARAMETER(int, FinalizationThreadBlockSize)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonVertexReductionFinalizeCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonVertexReductionFinalizeCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonVertexReductionFinalizeCS, FGlobalShader);
	
	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;
	using FParameters = FNiagaraRibbonVertexReductionFinalizationParameters;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonUVSettingsParams, NIAGARASHADER_API)
	SHADER_PARAMETER(FVector2f, Offset)
	SHADER_PARAMETER(FVector2f, Scale)
	SHADER_PARAMETER(float, TilingLength)
	SHADER_PARAMETER(int, DistributionMode)
	SHADER_PARAMETER(int, LeadingEdgeMode)
	SHADER_PARAMETER(int, TrailingEdgeMode)
	SHADER_PARAMETER(int, bEnablePerParticleUOverride)
	SHADER_PARAMETER(int, bEnablePerParticleVRangeOverride)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonVertexFinalizationParameters, NIAGARASHADER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRibbonComputeUniformParameters, Common)
	SHADER_PARAMETER_STRUCT(FNiagaraRibbonUVSettingsParams, UV0Settings)
	SHADER_PARAMETER_STRUCT(FNiagaraRibbonUVSettingsParams, UV1Settings)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_UAV(RWBuffer<float>, TangentsAndDistances)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, PackedPerRibbonData)
	SHADER_PARAMETER_SRV(Buffer<uint32>, CommandBuffer)
	SHADER_PARAMETER(int32, CommandBufferOffset)
	SHADER_PARAMETER(int32, TotalNumRibbons)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonUVParamCalculationCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonUVParamCalculationCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonUVParamCalculationCS, FGlobalShader);
	
	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;
	using FParameters = FNiagaraRibbonVertexFinalizationParameters;
	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonInitializeIndices, NIAGARASHADER_API)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, IndirectDrawOutput)
	SHADER_PARAMETER_SRV(Buffer<uint32>, VertexGenerationResults)

	// Direct and Indirect particle counts
	SHADER_PARAMETER(uint32, TotalNumParticlesDirect)
	SHADER_PARAMETER_SRV(Buffer<uint32>, EmitterParticleCountsBuffer)
	SHADER_PARAMETER(int32, EmitterParticleCountsBufferOffset)

	SHADER_PARAMETER(uint32, IndirectDrawOutputIndex)
	SHADER_PARAMETER(int32, VertexGenerationResultsIndex)
	SHADER_PARAMETER(uint32, IndexGenThreadSize)
	SHADER_PARAMETER(uint32, TrianglesPerSegment)

	SHADER_PARAMETER(int, RibbonTessellationMode)
	SHADER_PARAMETER(float, ViewDistance)
	SHADER_PARAMETER(float, LODDistanceFactor)
	SHADER_PARAMETER(uint32, TessellationMode)
	SHADER_PARAMETER(uint32, bCustomUseConstantFactor)
	SHADER_PARAMETER(uint32, CustomTessellationFactor)
	SHADER_PARAMETER(float, CustomTessellationMinAngle)
	SHADER_PARAMETER(uint32, bCustomUseScreenSpace)
	SHADER_PARAMETER(uint32, GNiagaraRibbonMaxTessellation)
	SHADER_PARAMETER(float, GNiagaraRibbonTessellationAngle)
	SHADER_PARAMETER(float, GNiagaraRibbonTessellationScreenPercentage)
	SHADER_PARAMETER(uint32, GNiagaraRibbonTessellationEnabled)
	SHADER_PARAMETER(float, GNiagaraRibbonTessellationMinDisplacementError)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonCreateIndexBufferParamsCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonCreateIndexBufferParamsCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonCreateIndexBufferParamsCS, FGlobalShader);
	
	using FParameters = FNiagaraRibbonInitializeIndices;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};


BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonGenerateIndices, NIAGARASHADER_API)
	SHADER_PARAMETER_UAV(RWBuffer<uint32>, GeneratedIndicesBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<uint32>, MultiRibbonIndices)
	SHADER_PARAMETER_SRV(Buffer<uint32>, Segments)

	SHADER_PARAMETER_SRV(Buffer<uint32>, IndirectDrawInfo)
	SHADER_PARAMETER_SRV(Buffer<uint32>, TriangleToVertexIds)

	// Direct and Indirect particle counts
	SHADER_PARAMETER(uint32, TotalNumParticlesDirect)
	SHADER_PARAMETER_SRV(Buffer<uint32>, EmitterParticleCountsBuffer)
	SHADER_PARAMETER(int32, EmitterParticleCountsBufferOffset)

	SHADER_PARAMETER(uint32, IndexBufferOffset)
	SHADER_PARAMETER(uint32, IndirectDrawInfoIndex)
	SHADER_PARAMETER(uint32, TriangleToVertexIdsCount)

	SHADER_PARAMETER(uint32, TrianglesPerSegment)
	SHADER_PARAMETER(uint32, NumVerticesInSlice)
	SHADER_PARAMETER(uint32, BitsNeededForShape)
	SHADER_PARAMETER(uint32, BitMaskForShape)
	SHADER_PARAMETER(uint32, SegmentBitShift)
	SHADER_PARAMETER(uint32, SegmentBitMask)
	SHADER_PARAMETER(uint32, SubSegmentBitShift)
	SHADER_PARAMETER(uint32, SubSegmentBitMask)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonCreateIndexBufferCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonCreateIndexBufferCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonCreateIndexBufferCS, FGlobalShader);
	
	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;
	using FParameters = FNiagaraRibbonGenerateIndices;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonAggregationStepParameters, NIAGARASHADER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRibbonComputeUniformParameters, Common)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<FRibbonAccumulationValues>, InputAccumulation)
	SHADER_PARAMETER_UAV(RWBuffer<FRibbonAccumulationValues>, OutputAccumulation)
	SHADER_PARAMETER(uint32, PrefixScanStride)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonAggregationStepCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonAggregationStepCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonAggregationStepCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;
	using FParameters = FNiagaraRibbonAggregationStepParameters;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraRibbonAggregationApplyParameters, NIAGARASHADER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRibbonComputeUniformParameters, Common)
	SHADER_PARAMETER_SRV(Buffer<uint32>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<FRibbonAccumulationValues>, InputAccumulation)
	SHADER_PARAMETER_UAV(RWBuffer<float>, OutputTangentsAndDistances)
	SHADER_PARAMETER_UAV(RWBuffer<uint>, OutputMultiRibbonIndices)
	SHADER_PARAMETER_UAV(RWBuffer<float>, OutputTessellationStats)
	SHADER_PARAMETER_UAV(RWBuffer<uint>, OutputSegments)
END_SHADER_PARAMETER_STRUCT()

class FNiagaraRibbonAggregationApplyCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraRibbonAggregationApplyCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraRibbonAggregationApplyCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FRibbonHasFullRibbonID, FRibbonHasRibbonID>;
	using FParameters = FNiagaraRibbonAggregationApplyParameters;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

