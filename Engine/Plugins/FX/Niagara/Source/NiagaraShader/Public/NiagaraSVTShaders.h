// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

class FNiagaraCopySVTToDenseBufferCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraCopySVTToDenseBufferCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraCopySVTToDenseBufferCS, FGlobalShader);
	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARASHADER_API)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, DestinationBuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, TileDataTextureSampler)
		SHADER_PARAMETER_TEXTURE(Texture3D<uint>, SparseVolumeTexturePageTable)
		SHADER_PARAMETER_TEXTURE(Texture3D, SparseVolumeTextureA)	
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms0)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms1)
		SHADER_PARAMETER(FIntVector, TextureSize)
		SHADER_PARAMETER(int32, MipLevel)
	END_SHADER_PARAMETER_STRUCT()
};

class FNiagaraBlendSVTsToDenseBufferCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraBlendSVTsToDenseBufferCS, NIAGARASHADER_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraBlendSVTsToDenseBufferCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARASHADER_API)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, DestinationBuffer)

		SHADER_PARAMETER_SAMPLER(SamplerState, TileDataTextureSampler_A)
		SHADER_PARAMETER_TEXTURE(Texture3D<uint>, SparseVolumeTexturePageTable_A)
		SHADER_PARAMETER_TEXTURE(Texture3D, SparseVolumeTextureA_A)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms0_A)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms1_A)
		SHADER_PARAMETER(FIntVector, TextureSize_A)
		SHADER_PARAMETER(int32, MipLevels_A)

		SHADER_PARAMETER_SAMPLER(SamplerState, TileDataTextureSampler_B)
		SHADER_PARAMETER_TEXTURE(Texture3D<uint>, SparseVolumeTexturePageTable_B)
		SHADER_PARAMETER_TEXTURE(Texture3D, SparseVolumeTextureA_B)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms0_B)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms1_B)
		SHADER_PARAMETER(FIntVector, TextureSize_B)
		SHADER_PARAMETER(int32, MipLevels_B)

		SHADER_PARAMETER(float, LerpAmount)
						
	END_SHADER_PARAMETER_STRUCT()
};
