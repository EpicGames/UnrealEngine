// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCore.h"
#include "NiagaraScriptBase.generated.h"

enum EShaderPlatform : uint16;

UENUM()
enum class ENiagaraGpuDispatchType : uint8
{
	/* Instances will distribute along X using platforms specific thread counts. */
	OneD,
	/* Instances will distribute along X & Y using platforms specific thread counts. */
	TwoD,
	/* Instances will distribute along X, Y & Z using platforms specific thread counts. */
	ThreeD,
	/* NumThreads will be determined manually. */
	Custom			UMETA(Hidden),
};

UENUM()
enum class ENiagaraDirectDispatchElementType : uint8
{
	/**
	Number of elements is the number of threads launched in that dimension.
	Threads that are out of bounds due to shader thread group size will be automatically clipped (i.e. code will not run).
	For example, if GroupSize = 64,1,1 and NumElements = 32,1,1 only the first 32 threads will run the code.
	*/
	NumThreads,
	/**
	Number of elements is the number of threads launched in that dimension.
	Threads that are out of bounds due to shader thread group size will not be clipped and your code will execute.
	You are responsible for ensuring you do not make invalid access from these OOB threads.
	For example, if GroupSize = 64,1,1 and NumElements = 32,1,1 you code will execute 64 times.
	Use this path if you need to add group sync's within you graph code.
	*/
	NumThreadsNoClipping,
	/**
	Number of elements refers to the number of groups to launch.
	For example, if you defined NumElements as 3,1,1 and GroupSize was 64,1,1 you are effectively launching 192 threads.
	*/
	NumGroups,
};

UENUM()
enum class ENiagaraSimStageExecuteBehavior : uint8
{
	/** The stage will run every frame. */
	Always,
	/** The stage will only run on the frame when the simulation is reset. */
	OnSimulationReset,
	/** The stage will not run on the frame where the simulation is reset. */
	NotOnSimulationReset,
};

USTRUCT()
struct FSimulationStageMetaData
{
	GENERATED_USTRUCT_BODY()

	NIAGARASHADER_API FSimulationStageMetaData();

	/** User simulation stage name. */
	UPROPERTY()
	FName SimulationStageName;

	UPROPERTY()
	FName EnabledBinding;

	/** Optional binding to manually specify the element count. */
	UPROPERTY()
	FIntVector ElementCount = FIntVector::ZeroValue;

	/** Optional binding to manually specify the element count. */
	UPROPERTY()
	FName ElementCountXBinding;

	/** Optional binding to manually specify the element count. */
	UPROPERTY()
	FName ElementCountYBinding;

	/** Optional binding to manually specify the element count. */
	UPROPERTY()
	FName ElementCountZBinding;

	/** The source we are iteration over. */
	UPROPERTY()
	ENiagaraIterationSource IterationSourceType = ENiagaraIterationSource::Particles;

	/** When IterationSource is ENiagaraIterationSource::DataInterface this is the data interface name. */
	UPROPERTY()
	FName IterationDataInterface;

	/** When IterationSource is ENiagaraIterationSource::IterationDirectBinding this is the variable we are bound to. */
	UPROPERTY()
	FName IterationDirectBinding;

	/** Controls when the simulation stage will execute. */
	UPROPERTY()
	ENiagaraSimStageExecuteBehavior ExecuteBehavior = ENiagaraSimStageExecuteBehavior::Always;

	/** Do we write to particles this stage? */
	UPROPERTY()
	uint32 bWritesParticles : 1;

	/** When enabled the simulation stage does not write all variables out, so we are reading / writing to the same buffer. */
	UPROPERTY()
	uint32 bPartialParticleUpdate : 1;

	UPROPERTY()
	uint32 bParticleIterationStateEnabled : 1;

	UPROPERTY()
	uint32 bGpuIndirectDispatch : 1;

	/** When the value is not none this is the binding used for particle state iteration stages. */
	UPROPERTY()
	FName ParticleIterationStateBinding;

	/** Cached component index for particle iteration stage. */
	int32 ParticleIterationStateComponentIndex = INDEX_NONE;

	/** Inclusive range to compare the particle state value with. */
	UPROPERTY()
	FIntPoint ParticleIterationStateRange = FIntPoint(0, 0);

	/** DataInterfaces that we write to in this stage.*/
	UPROPERTY()
	TArray<FName> OutputDestinations;

	/** DataInterfaces that we read from in this stage.*/
	UPROPERTY()
	TArray<FName> InputDataInterfaces;

	/** The number of iterations for the stage. */
	UPROPERTY()
	int32 NumIterations = 1;

	/** Optional binding to gather num iterations from. */
	UPROPERTY()
	FName NumIterationsBinding;

	inline bool ShouldRunStage(bool bResetData) const
	{
		const bool bAlways = ExecuteBehavior == ENiagaraSimStageExecuteBehavior::Always;
		const bool bResetOnly = ExecuteBehavior == ENiagaraSimStageExecuteBehavior::OnSimulationReset;
		return bAlways || (bResetOnly == bResetData);
	}

	/** Dispatch type set for this stage. */
	UPROPERTY()
	ENiagaraGpuDispatchType GpuDispatchType = ENiagaraGpuDispatchType::OneD;

	UPROPERTY()
	ENiagaraDirectDispatchElementType GpuDirectDispatchElementType = ENiagaraDirectDispatchElementType::NumThreads;

	/** When in custom mode this is the num threads. */
	UPROPERTY()
	FIntVector GpuDispatchNumThreads = FIntVector(0, 0, 0);
};

UCLASS(MinimalAPI, abstract)
class UNiagaraScriptBase : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual void ModifyCompilationEnvironment(EShaderPlatform Platform, struct FShaderCompilerEnvironment& OutEnvironment) const PURE_VIRTUAL(UNiagaraScriptBase::ModifyCompilationEnvironment, );
	virtual bool ShouldCompile(EShaderPlatform Platform) const PURE_VIRTUAL(UNiagaraScriptBase::ShouldCompile, return true; );
	virtual TConstArrayView<FSimulationStageMetaData> GetSimulationStageMetaData() const PURE_VIRTUAL(UNiagaraScriptBase::GetSimulationStageMetaData(), return MakeArrayView<FSimulationStageMetaData>(nullptr, 0); )
};
