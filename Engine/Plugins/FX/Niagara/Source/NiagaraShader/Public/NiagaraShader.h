// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderParameters.h"
#include "SceneView.h"
#include "Shader.h"
#include "Math/Vector.h"
#include "NiagaraCommon.h"
#include "NiagaraScriptBase.h"
#include "NiagaraShared.h"
#include "NiagaraShaderType.h"
#include "SceneTexturesConfig.h"
#include "RendererUtils.h"


class UClass;
struct FAppendToClassSchemaContext;

template<typename TBufferStruct> class TUniformBufferRef;

/** Base class for all generated Niagara compute shaders. */
class FNiagaraShader : public FShader
{
public:
	DECLARE_SHADER_TYPE(FNiagaraShader, Niagara);

	// This structure is a replication of FNiagaraGlobalParameters with interpolated parameters includes
	BEGIN_SHADER_PARAMETER_STRUCT(FGlobalParameters, )
		SHADER_PARAMETER(float,		Engine_WorldDeltaTime)
		SHADER_PARAMETER(float,		Engine_DeltaTime)
		SHADER_PARAMETER(float,		Engine_InverseDeltaTime)
		SHADER_PARAMETER(float,		Engine_Time)
		SHADER_PARAMETER(float,		Engine_RealTime)
		SHADER_PARAMETER(int32,		Engine_QualityLevel)
		SHADER_PARAMETER(int32,		Engine_Pad0)
		SHADER_PARAMETER(int32,		Engine_Pad1)

		SHADER_PARAMETER(float,		PREV_Engine_WorldDeltaTime)
		SHADER_PARAMETER(float,		PREV_Engine_DeltaTime)
		SHADER_PARAMETER(float,		PREV_Engine_InverseDeltaTime)
		SHADER_PARAMETER(float,		PREV_Engine_Time)
		SHADER_PARAMETER(float,		PREV_Engine_RealTime)
		SHADER_PARAMETER(int32,		PREV_Engine_QualityLevel)
		SHADER_PARAMETER(int32,		PREV_Engine_Pad0)
		SHADER_PARAMETER(int32,		PREV_Engine_Pad1)
	END_SHADER_PARAMETER_STRUCT()

	// This structure is a replication of FNiagaraSystemParameters with interpolated parameters includes
	BEGIN_SHADER_PARAMETER_STRUCT(FSystemParameters, )
		SHADER_PARAMETER(float,		Engine_Owner_TimeSinceRendered)
		SHADER_PARAMETER(float,		Engine_Owner_LODDistance)
		SHADER_PARAMETER(float,		Engine_Owner_LODDistanceFraction)
		SHADER_PARAMETER(float,		Engine_System_Age)
		SHADER_PARAMETER(uint32,	Engine_Owner_ExecutionState)
		SHADER_PARAMETER(int32,		Engine_System_TickCount)
		SHADER_PARAMETER(int32,		Engine_System_NumEmitters)
		SHADER_PARAMETER(int32,		Engine_System_NumEmittersAlive)
		SHADER_PARAMETER(int32,		Engine_System_SignificanceIndex)
		SHADER_PARAMETER(int32,		Engine_System_RandomSeed)
		SHADER_PARAMETER(int32,		Engine_System_CurrentTimeStep)
		SHADER_PARAMETER(int32,		Engine_System_NumTimeSteps)
		SHADER_PARAMETER(float,		Engine_System_TimeStepFraction)
		SHADER_PARAMETER(int32,		Engine_System_NumParticles)
		SHADER_PARAMETER(int32,		System_Pad0)
		SHADER_PARAMETER(int32,		System_Pad1)

		SHADER_PARAMETER(float,		PREV_Engine_Owner_TimeSinceRendered)
		SHADER_PARAMETER(float,		PREV_Engine_Owner_LODDistance)
		SHADER_PARAMETER(float,		PREV_Engine_Owner_LODDistanceFraction)
		SHADER_PARAMETER(float,		PREV_Engine_System_Age)
		SHADER_PARAMETER(uint32,	PREV_Engine_Owner_ExecutionState)
		SHADER_PARAMETER(int32,		PREV_Engine_System_TickCount)
		SHADER_PARAMETER(int32,		PREV_Engine_System_NumEmitters)
		SHADER_PARAMETER(int32,		PREV_Engine_System_NumEmittersAlive)
		SHADER_PARAMETER(int32,		PREV_Engine_System_SignificanceIndex)
		SHADER_PARAMETER(int32,		PREV_Engine_System_RandomSeed)
		SHADER_PARAMETER(int32,		PREV_Engine_System_CurrentTimeStep)
		SHADER_PARAMETER(int32,		PREV_Engine_System_NumTimeSteps)
		SHADER_PARAMETER(float,		PREV_Engine_System_TimeStepFraction)
		SHADER_PARAMETER(int32,		PREV_Engine_System_NumParticles)
		SHADER_PARAMETER(int32,		PREV_System_Pad0)
		SHADER_PARAMETER(int32,		PREV_System_Pad1)
	END_SHADER_PARAMETER_STRUCT()

	// This structure is a replication of FNiagaraOwnerParameters with interpolated parameters includes
	BEGIN_SHADER_PARAMETER_STRUCT(FOwnerParameters, )
		SHADER_PARAMETER(FMatrix44f,	Engine_Owner_SystemLocalToWorld)
		SHADER_PARAMETER(FMatrix44f,	Engine_Owner_SystemWorldToLocal)
		SHADER_PARAMETER(FMatrix44f,	Engine_Owner_SystemLocalToWorldTransposed)
		SHADER_PARAMETER(FMatrix44f,	Engine_Owner_SystemWorldToLocalTransposed)
		SHADER_PARAMETER(FMatrix44f,	Engine_Owner_SystemLocalToWorldNoScale)
		SHADER_PARAMETER(FMatrix44f,	Engine_Owner_SystemWorldToLocalNoScale)
		SHADER_PARAMETER(FQuat4f,		Engine_Owner_Rotation)
		SHADER_PARAMETER(FVector3f,		Engine_Owner_Position)
		SHADER_PARAMETER(float,			Engine_Owner_Pad0)
		SHADER_PARAMETER(FVector3f,		Engine_Owner_Velocity)
		SHADER_PARAMETER(float,			Engine_Owner_Pad1)
		SHADER_PARAMETER(FVector3f,		Engine_Owner_SystemXAxis)
		SHADER_PARAMETER(float,			Engine_Owner_Pad2)
		SHADER_PARAMETER(FVector3f,		Engine_Owner_SystemYAxis)
		SHADER_PARAMETER(float,			Engine_Owner_Pad3)
		SHADER_PARAMETER(FVector3f,		Engine_Owner_SystemZAxis)
		SHADER_PARAMETER(float,			Engine_Owner_Pad4)
		SHADER_PARAMETER(FVector3f,		Engine_Owner_Scale)
		SHADER_PARAMETER(float,			Engine_Owner_Pad5)
		SHADER_PARAMETER(FVector4f,		Engine_Owner_LWCTile)

		SHADER_PARAMETER(FMatrix44f,	PREV_Engine_Owner_SystemLocalToWorld)
		SHADER_PARAMETER(FMatrix44f,	PREV_Engine_Owner_SystemWorldToLocal)
		SHADER_PARAMETER(FMatrix44f,	PREV_Engine_Owner_SystemLocalToWorldTransposed)
		SHADER_PARAMETER(FMatrix44f,	PREV_Engine_Owner_SystemWorldToLocalTransposed)
		SHADER_PARAMETER(FMatrix44f,	PREV_Engine_Owner_SystemLocalToWorldNoScale)
		SHADER_PARAMETER(FMatrix44f,	PREV_Engine_Owner_SystemWorldToLocalNoScale)
		SHADER_PARAMETER(FQuat4f,		PREV_Engine_Owner_Rotation)
		SHADER_PARAMETER(FVector3f,		PREV_Engine_Owner_Position)
		SHADER_PARAMETER(float,			PREV_Engine_Owner_Pad0)
		SHADER_PARAMETER(FVector3f,		PREV_Engine_Owner_Velocity)
		SHADER_PARAMETER(float,			PREV_Engine_Owner_Pad1)
		SHADER_PARAMETER(FVector3f,		PREV_Engine_Owner_SystemXAxis)
		SHADER_PARAMETER(float,			PREV_Engine_Owner_Pad2)
		SHADER_PARAMETER(FVector3f,		PREV_Engine_Owner_SystemYAxis)
		SHADER_PARAMETER(float,			PREV_Engine_Owner_Pad3)
		SHADER_PARAMETER(FVector3f,		PREV_Engine_Owner_SystemZAxis)
		SHADER_PARAMETER(float,			PREV_Engine_Owner_Pad4)
		SHADER_PARAMETER(FVector3f,		PREV_Engine_Owner_Scale)
		SHADER_PARAMETER(float,			PREV_Engine_Owner_Pad5)
		SHADER_PARAMETER(FVector4f,		PREV_Engine_Owner_LWCTile)
	END_SHADER_PARAMETER_STRUCT()

	// This structure is a replication of FNiagaraEmitterParameters with interpolated parameters includes
	BEGIN_SHADER_PARAMETER_STRUCT(FEmitterParameters, )
		SHADER_PARAMETER(int32,		Engine_Emitter_NumParticles)
		SHADER_PARAMETER(int32,		Engine_Emitter_TotalSpawnedParticles)
		SHADER_PARAMETER(float,		Engine_Emitter_SpawnCountScale)
		SHADER_PARAMETER(float,		Emitter_Age)
		SHADER_PARAMETER(int32,		Emitter_RandomSeed)
		SHADER_PARAMETER(int32,		Engine_Emitter_InstanceSeed)
		SHADER_PARAMETER(int32,		Emitter_Pad0)
		SHADER_PARAMETER(int32,		Emitter_Pad1)

		SHADER_PARAMETER(int32,		PREV_Engine_Emitter_NumParticles)
		SHADER_PARAMETER(int32,		PREV_Engine_Emitter_TotalSpawnedParticles)
		SHADER_PARAMETER(float,		PREV_Engine_Emitter_SpawnCountScale)
		SHADER_PARAMETER(float,		PREV_Emitter_Age)
		SHADER_PARAMETER(int32,		PREV_Emitter_RandomSeed)
		SHADER_PARAMETER(int32,		PREV_Engine_Emitter_InstanceSeed)
		SHADER_PARAMETER(int32,		PREV_Emitter_Pad0)
		SHADER_PARAMETER(int32,		PREV_Emitter_Pad1)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32,									ComponentBufferSizeRead)
		SHADER_PARAMETER(uint32,									ComponentBufferSizeWrite)
		SHADER_PARAMETER(uint32,									SimStart)

		SHADER_PARAMETER_SRV(Buffer<float>,							InputFloat)
		SHADER_PARAMETER_SRV(Buffer<half>,							InputHalf)
		SHADER_PARAMETER_SRV(Buffer<int>,							InputInt)
		SHADER_PARAMETER_SRV(Buffer<float>,							StaticInputFloat)
	
		SHADER_PARAMETER_UAV(RWBuffer<float>,						RWOutputFloat)
		SHADER_PARAMETER_UAV(RWBuffer<half>,						RWOutputHalf)
		SHADER_PARAMETER_UAV(RWBuffer<int>,							RWOutputInt)

		SHADER_PARAMETER_UAV(RWBuffer<uint>,						RWInstanceCounts)
		SHADER_PARAMETER(uint32,									ReadInstanceCountOffset)
		SHADER_PARAMETER(uint32,									WriteInstanceCountOffset)

		SHADER_PARAMETER_SRV(Buffer<int>,							FreeIDList)
		SHADER_PARAMETER_UAV(RWBuffer<int>,							RWIDToIndexTable)

		SHADER_PARAMETER(FUintVector4,								SimulationStageIterationInfo)

		SHADER_PARAMETER(FIntVector3,								ParticleIterationStateInfo)

		SHADER_PARAMETER(uint32,									EmitterTickCounter)
		SHADER_PARAMETER_ARRAY(FIntVector4,							EmitterSpawnInfoOffsets,	[(NIAGARA_MAX_GPU_SPAWN_INFOS + 3) / 4])
		SHADER_PARAMETER_ARRAY(FVector4f,							EmitterSpawnInfoParams,		[NIAGARA_MAX_GPU_SPAWN_INFOS])
		SHADER_PARAMETER(uint32,									NumSpawnedInstances)

		SHADER_PARAMETER(FUintVector3,								DispatchThreadIdBounds)
		SHADER_PARAMETER(FUintVector3,								DispatchThreadIdToLinear)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint4>,				IndirectDispatchArgs)
		SHADER_PARAMETER(uint32,									IndirectDispatchArgsOffset)
		RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer,				ERHIAccess::IndirectArgs)

		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalParameters,			GlobalParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSystemParameters,			SystemParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FOwnerParameters,			OwnerParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FEmitterParameters,			EmitterParameters)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters,	View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters,	SceneTextures)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstratePublicGlobalUniformParameters, SubstratePublic)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationParameters = FNiagaraShaderPermutationParameters;

	static NIAGARASHADER_API FName UniformBufferLayoutName;

	static FIntVector GetDefaultThreadGroupSize(ENiagaraGpuDispatchType DispatchType)
	{
		//-TODO: Grab this from FDataDrivenShaderPlatformInfo
		switch (DispatchType)
		{
			case ENiagaraGpuDispatchType::OneD:		return FIntVector(64, 1, 1);
			case ENiagaraGpuDispatchType::TwoD:		return FIntVector(8, 8, 1);
			case ENiagaraGpuDispatchType::ThreeD:	return FIntVector(4, 4, 4);
			default:								return FIntVector(64, 1, 1);
		}
	}

	static void ModifyCompilationEnvironment(const FNiagaraShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool bUseWaveIntrinsics = false; // TODO: Some content breaks with this - FDataDrivenShaderPlatformInfo::GetInfo(Platform).bSupportsIntrinsicWaveOnce;
		OutEnvironment.SetDefine(TEXT("USE_WAVE_INTRINSICS"), bUseWaveIntrinsics ? 1 : 0);
	}

	FNiagaraShader() {}
	NIAGARASHADER_API FNiagaraShader(const FNiagaraShaderType::CompiledShaderInitializerType& Initializer);

	TConstArrayView<FNiagaraDataInterfaceParamRef> GetDIParameters()
	{
		return MakeArrayView(DataInterfaceParameters);
	}

#if WITH_EDITORONLY_DATA
	/**
	 * Called from UNiagaraSystem::AppendToClassSchema() to collect the Script specific details for iterative cooking
	 */
	NIAGARASHADER_API static void BuildClassSchema(FAppendToClassSchemaContext& Context);
#endif

	LAYOUT_FIELD(bool, bNeedsViewUniformBuffer);
	LAYOUT_FIELD(uint16, MiscUsageBitMask);
	LAYOUT_ARRAY(FShaderUniformBufferParameter, ExternalConstantBufferParam, 2);

private:
	// Data about parameters used for each Data Interface.
	LAYOUT_FIELD(TMemoryImageArray<FNiagaraDataInterfaceParamRef>, DataInterfaceParameters);
};

extern NIAGARASHADER_API int32 GNiagaraSkipVectorVMBackendOptimizations;
