// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ShaderCompiler.h"

/** Results for a single compiled shader map. */
struct FNiagaraShaderMapCompileResults
{
	FNiagaraShaderMapCompileResults() :
		NumJobsQueued(0),
		bAllJobsSucceeded(true)
	{}

	int32 NumJobsQueued;
	bool bAllJobsSucceeded;
	TArray<FShaderCommonCompileJobPtr> FinishedJobs;
};


/** Results for a single compiled and finalized shader map. */
struct FNiagaraShaderMapFinalizeResults : public FNiagaraShaderMapCompileResults
{
	/** Tracks finalization progress on this shader map. */
	int32 FinalizeJobIndex;

	FNiagaraShaderMapFinalizeResults(const FNiagaraShaderMapCompileResults& InCompileResults) :
		FNiagaraShaderMapCompileResults(InCompileResults),
		FinalizeJobIndex(0)
	{}
};


// handles gpu compute shader compile jobs, applying of the shaders to their scripts, and some error handling
//
class FNiagaraShaderCompilationManager
{
public:
	FNiagaraShaderCompilationManager() = default;

	NIAGARASHADER_API void AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs);
	NIAGARASHADER_API void ProcessAsyncResults();

	void FinishCompilation(const TArray<int32>& ShaderMapIdsToFinishCompiling);
private:
	void ProcessCompiledNiagaraShaderMaps(TMap<int32, FNiagaraShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget);

	TArray<FShaderCommonCompileJobPtr> JobQueue;

	/** Map from shader map Id to the compile results for that map, used to gather compiled results. */
	TMap<int32, FNiagaraShaderMapCompileResults> NiagaraShaderMapJobs;

	/** Map from shader map id to results being finalized.  Used to track shader finalizations over multiple frames. */
	TMap<int32, FNiagaraShaderMapFinalizeResults> PendingFinalizeNiagaraShaderMaps;
};

extern NIAGARASHADER_API FNiagaraShaderCompilationManager GNiagaraShaderCompilationManager;


#endif //WITH_EDITOR
