// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

class FString;
class UNiagaraDataInterfaceBase;

/**
* Niagara shader module interface
*/
class INiagaraShaderModule : public IModuleInterface
{
public:
	DECLARE_DELEGATE_RetVal(void, FOnProcessQueue);
	DECLARE_DELEGATE_RetVal_OneParam(UNiagaraDataInterfaceBase*, FOnRequestDefaultDataInterface, const FString&); 
	
	FDelegateHandle NIAGARASHADER_API SetOnProcessShaderCompilationQueue(FOnProcessQueue InOnProcessQueue);
	void NIAGARASHADER_API ResetOnProcessShaderCompilationQueue(FDelegateHandle DelegateHandle);
	void ProcessShaderCompilationQueue();

	// Handles converting string name to data interface CDO pointer
	FDelegateHandle NIAGARASHADER_API SetOnRequestDefaultDataInterfaceHandler(FOnRequestDefaultDataInterface InHandler); 
	void NIAGARASHADER_API ResetOnRequestDefaultDataInterfaceHandler();
	UNiagaraDataInterfaceBase* RequestDefaultDataInterface(const FString& DIClassName);

	virtual void StartupModule() override;

	virtual void ShutdownModule() override
	{
		Singleton = nullptr;
	}

	// If you know that you want the shader module and have access to the header you don't have to find the module.
	static INiagaraShaderModule* Get()
	{
		return Singleton;
	}

private:
	FOnProcessQueue OnProcessQueue;
	FOnRequestDefaultDataInterface OnRequestDefaultDataInterface;
	static INiagaraShaderModule* Singleton;
};


