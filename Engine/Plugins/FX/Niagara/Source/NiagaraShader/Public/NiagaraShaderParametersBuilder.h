// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraShared.h"
#include "ShaderParameterMetadataBuilder.h"

class FNiagaraShaderParametersBuilder
{
public:
	explicit FNiagaraShaderParametersBuilder(const FNiagaraDataInterfaceGPUParamInfo& InGPUParamInfo, TArray<FString>& InLooseNames, TArray<FNiagaraDataInterfaceStructIncludeInfo>& InStructIncludeInfos, FShaderParametersMetadataBuilder& InMetadataBuilder)
		: GPUParamInfo(InGPUParamInfo)
		, StructIncludeInfos(InStructIncludeInfos)
		, LooseNames(InLooseNames)
		, MetadataBuilder(InMetadataBuilder)
	{
	}

	// Adds a loose parameter that is scoped to the data interface
	// i.e. if the parameter was called "MyFloat" the shader variable would be "UniqueDataInterfaceName_MyFloat"
	template<typename T> void AddLooseParam(const TCHAR* Name)
	{
		LooseNames.Emplace(FString::Printf(TEXT("%s_%s"), *GPUParamInfo.DataInterfaceHLSLSymbol, Name));
		MetadataBuilder.AddParam<T>(*LooseNames.Last());
	}

	// Adds a loose array parameter that is scoped to the data interface
	// i.e. if the parameter was called "MyFloat" the shader variable would be "UniqueDataInterfaceName_MyFloat"
	template<typename T> void AddLooseParamArray(const TCHAR* Name, int32 NumElements)
	{
		LooseNames.Emplace(FString::Printf(TEXT("%s_%s"), *GPUParamInfo.DataInterfaceHLSLSymbol, Name));
		MetadataBuilder.AddParamArray<T>(*LooseNames.Last(), NumElements);
	}

	// Adds a shader parameters structure that is scoped to the data interface
	// i.e. if the structured contained "MyFloat" the shader variable would be "UniqueDataInterfaceName_MyFloat"
	template<typename T> void AddNestedStruct()
	{
		MetadataBuilder.AddNestedStruct<T>(*GPUParamInfo.DataInterfaceHLSLSymbol);
	}

	// Adds a shader parameters structure that is global in scope
	// i.e. if the structure contained "MyFloat" the shader variable would be named "MyFloat"
	inline void AddIncludedStruct(const FShaderParametersMetadata* StructMetadata)
	{
		for (const FNiagaraDataInterfaceStructIncludeInfo& Existing : StructIncludeInfos)
		{
			if (Existing.StructMetadata == StructMetadata)
			{
				return;
			}
		}

		FNiagaraDataInterfaceStructIncludeInfo& NewInfo = StructIncludeInfos.AddDefaulted_GetRef();
		NewInfo.StructMetadata = StructMetadata;
		NewInfo.ParamterOffset = Align(MetadataBuilder.GetNextMemberOffset(), SHADER_PARAMETER_STRUCT_ALIGNMENT);

		MetadataBuilder.AddIncludedStruct(StructMetadata);
	}

	// Adds a shader parameters structure that is global in scope
	// i.e. if the structure contained "MyFloat" the shader variable would be named "MyFloat"
	template<typename T> void AddIncludedStruct()
	{
		AddIncludedStruct(TShaderParameterStructTypeInfo<T>::GetStructMetadata());
	}

	TConstArrayView<FNiagaraDataInterfaceGeneratedFunction> GetGeneratedFunctions() const { return GPUParamInfo.GeneratedFunctions; }

private:
	const FNiagaraDataInterfaceGPUParamInfo& GPUParamInfo;
	TArray<FNiagaraDataInterfaceStructIncludeInfo>& StructIncludeInfos;
	TArray<FString>& LooseNames;
	FShaderParametersMetadataBuilder& MetadataBuilder;
};
