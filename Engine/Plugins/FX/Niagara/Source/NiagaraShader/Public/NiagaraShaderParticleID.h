// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FRHICommandList;
class FRHIShaderResourceView;
class FRHIUnorderedAccessView;
namespace ERHIFeatureLevel { enum Type : int; }

extern NIAGARASHADER_API void NiagaraInitGPUFreeIDList(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIUnorderedAccessView* NewBufferUAV, uint32 NewBufferNumElements, FRHIShaderResourceView* ExistingBufferSRV, uint32 ExistingBufferNumElements);
extern NIAGARASHADER_API void NiagaraComputeGPUFreeIDs(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIShaderResourceView* IDToIndexTableSRV, uint32 NumIDs, FRHIUnorderedAccessView* FreeIDUAV, FRHIUnorderedAccessView* FreeIDListSizesUAV, uint32 FreeIDListIndex);
extern NIAGARASHADER_API void NiagaraFillGPUIntBuffer(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIUnorderedAccessView* BufferUAV, uint32 NumElements, int32 FillValue);

