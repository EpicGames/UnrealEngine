// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraShaderType.h: Niagara shader type definition.
=============================================================================*/

#pragma once

#include "Shader.h"

struct FNiagaraShaderScriptParametersMetadata;
struct FShaderTarget;
struct FSharedShaderCompilerEnvironment;

/** A macro to implement Niagara shaders. */
#define IMPLEMENT_NIAGARA_SHADER_TYPE(TemplatePrefix,ShaderClass,SourceFilename,FunctionName,Frequency) \
	IMPLEMENT_SHADER_TYPE( \
		TemplatePrefix, \
		ShaderClass, \
		SourceFilename, \
		FunctionName, \
		Frequency \
		);

class FNiagaraShaderScript;
class FShaderCommonCompileJob;
class FShaderCompileJob;
struct FSimulationStageMetaData;
class FUniformExpressionSet;


/** Called for every Niagara shader to update the appropriate stats. */
extern void UpdateNiagaraShaderCompilingStats(const FNiagaraShaderScript* Script);

/**
 * Dump shader stats for a given platform.
 * 
 * @param	Platform	Platform to dump stats for.
 */
extern ENGINE_API void DumpComputeShaderStats( EShaderPlatform Platform );

struct FNiagaraShaderPermutationParameters : public FShaderPermutationParameters
{
	const FNiagaraShaderScript* Script;

	FNiagaraShaderPermutationParameters(EShaderPlatform InPlatform, const FNiagaraShaderScript* InScript)
		: FShaderPermutationParameters(InPlatform)
		, Script(InScript)
	{}
};

/**
 * A shader meta type for niagara-linked shaders.
 */
class FNiagaraShaderType : public FShaderType
{
public:
	struct FParameters : public FShaderType::FParameters
	{
		TSharedRef<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata;

		FParameters(const TSharedRef<FNiagaraShaderScriptParametersMetadata>& InScriptParametersMetadata)
			: ScriptParametersMetadata(InScriptParametersMetadata)
		{
		}
	};

	struct CompiledShaderInitializerType : FShaderCompiledShaderInitializerType
	{
		CompiledShaderInitializerType(
			const FShaderType* InType,
			const FParameters* InParameters,
			int32 InPermutationId,
			const FShaderCompilerOutput& CompilerOutput,
			uint64 InNiagaraShaderMapHash
		)
			: FShaderCompiledShaderInitializerType(InType, InParameters, InPermutationId, CompilerOutput, InNiagaraShaderMapHash, nullptr, nullptr)
		{
		}
	};

	FNiagaraShaderType(
		FTypeLayoutDesc& InTypeLayout,
		const TCHAR* InName,
		const TCHAR* InSourceFilename,
		const TCHAR* InFunctionName,
		uint32 InFrequency,					// ugly - ignored for Niagara shaders but needed for IMPLEMENT_SHADER_TYPE macro magic
		int32 InTotalPermutationCount,
		int32 InTotalSpecializationCount,
		ConstructSerializedType InConstructSerializedRef,
		ConstructCompiledType InConstructCompiledRef,
		ShouldCompilePermutationType InShouldCompilePermutationRef,
		ShouldPrecachePermutationType InShouldPrecachePermutationRef,
		GetUnspecializedIdType InGetUnspecializedIdRef,
		GetSpecializationValuesType InGetSpecializationValuesRef,
		GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef,
		GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
		ModifyCompilationEnvironmentType InModifyCompilationEnvironmentRef,
		ValidateCompiledResultType InValidateCompiledResultRef,
		GetOverrideJobPriorityType InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
		uint32 InTypeSize,
		const FShaderParametersMetadata* InRootParametersMetadata = nullptr
#if WITH_EDITOR
		, GetPermutationIdStringType InGetPermutationIdStringRef = nullptr
#endif // WITH_EDITOR
	)
		: FShaderType(EShaderTypeForDynamicCast::Niagara, InTypeLayout, InName, InSourceFilename, InFunctionName, SF_Compute, InTotalPermutationCount, InTotalSpecializationCount,
			InConstructSerializedRef,
			InConstructCompiledRef,
			InShouldCompilePermutationRef,
			InShouldPrecachePermutationRef,
			InGetUnspecializedIdRef,
			InGetSpecializationValuesRef,
			InGetRayTracingPayloadTypeRef,
			InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
			InModifyCompilationEnvironmentRef,
			InValidateCompiledResultRef,
			InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
			InTypeSize,
			InRootParametersMetadata
#if WITH_EDITOR
			, InGetPermutationIdStringRef
#endif // WITH_EDITOR
		)
	{
		check(InTotalPermutationCount == 1);
	}

#if WITH_EDITOR
	/**
	 * Enqueues a compilation for a new shader of this type.
	 * @param Script - The script to link the shader with.
	 */
	void BeginCompileShader(
			uint32 ShaderMapId,
			int32 PermutationId,
			const FNiagaraShaderScript* Script,
			FSharedShaderCompilerEnvironment* CompilationEnvironment,
			EShaderPlatform Platform,
			TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs,
			FShaderTarget Target
		);

	void BeginCompileShaderFromSource(
		FStringView FriendlyName,
		uint32 ShaderMapId,
		int32 PermutationId,
		TSharedRef<FNiagaraShaderScriptParametersMetadata> ShaderParameters,
		FSharedShaderCompilerEnvironment* CompilationEnvironment,
		FStringView ScriptSource,
		EShaderPlatform Platform,
		const FSimulationStageMetaData& SimStageMetaData,
		FShaderTarget Target,
		TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs
	) const;

	/**
	 * Either creates a new instance of this type or returns an equivalent existing shader.
	 * @param NiagaraShaderMapHash - The hash computed for the owning shadermap
	 * @param CurrentJob - Compile job that was enqueued by BeginCompileShader.
	 */
	FShader* FinishCompileShader(
		uint64 NiagaraShaderMapHash,
		const FShaderCompileJob& CurrentJob
		) const;
#endif // WITH_EDITOR

	/**
	 * Checks if the shader type should be cached for a particular platform and script.
	 * @param Platform - The platform to check.
	 * @param script - The script to check.
	 * @return True if this shader type should be cached.
	 */
	bool ShouldCache(EShaderPlatform Platform,const FNiagaraShaderScript* Script) const
	{
		return ShouldCompilePermutation(FNiagaraShaderPermutationParameters(Platform, Script));
	}

#if WITH_EDITOR
	/** Adds include statements for uniform buffers that this shader type references, and builds a prefix for the shader file with the include statements. */
	void AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, EShaderPlatform Platform) const;
protected:

	/**
	 * Sets up the environment used to compile an instance of this shader type.
	 * @param Platform - Platform to compile for.
	 * @param Environment - The shader compile environment that the function modifies.
	 */
	void SetupCompileEnvironment(EShaderPlatform Platform, const FNiagaraShaderScript* Script, FShaderCompilerEnvironment& Environment) const
	{
		ModifyCompilationEnvironment(FNiagaraShaderPermutationParameters(Platform, Script), Environment);
	}
#endif // WITH_EDITOR
};

