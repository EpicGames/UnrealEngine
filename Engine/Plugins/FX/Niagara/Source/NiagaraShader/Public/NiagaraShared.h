// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraShared.h: Shared Niagara definitions.
=============================================================================*/

#pragma once

#include "Containers/IndirectArray.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "Templates/RefCounting.h"
#include "Misc/SecureHash.h"
#include "MaterialShared.h"
#include "RHI.h"
#include "RenderResource.h"
#include "RenderDeferredCleanup.h"
#include "Shader.h"
#include "VertexFactory.h"
#include "SceneTypes.h"
#include "StaticParameterSet.h"
#include "Misc/Optional.h"
#include "NiagaraCommon.h"
#include "NiagaraCompileHash.h"
#include "NiagaraDataInterfaceBase.h"
#include "NiagaraCore.h"
#include "NiagaraShared.generated.h"

class FNiagaraShaderScript;
class FNiagaraShaderMap;
class FNiagaraShader;
class FNiagaraShaderMapId;
class FThreadSafeBool;
class UNiagaraScriptBase;
struct FNiagaraVMExecutableDataId;
struct FSharedShaderCompilerEnvironment;
struct FSimulationStageMetaData;

/** Defines the compile event types for translation/compilation.*/
UENUM()
enum class FNiagaraCompileEventSeverity : uint8
{
	Log = 0,
	Display = 1,
	Warning = 2,
	Error = 3
};

UENUM()
enum class FNiagaraCompileEventSource : uint8
{
	Unset = 0, // No specific source of compile event.
	ScriptDependency = 1 // Compile event is from a dependency to other scripts.
};

/** Records necessary information to give UI cues for errors/logs/warnings during compile.*/
USTRUCT()
struct FNiagaraCompileEvent
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraCompileEvent()
	{
		Severity = FNiagaraCompileEventSeverity::Display;
		Message = FString();
		ShortDescription = FString();
		NodeGuid = FGuid();
		PinGuid = FGuid();
		StackGuids.Empty();
		Source = FNiagaraCompileEventSource::Unset;
	}
 
	FNiagaraCompileEvent(
		FNiagaraCompileEventSeverity InSeverity,
		const FString& InMessage,
		FString InShortDescription = FString(),
		FGuid InNodeGuid = FGuid(),
		FGuid InPinGuid = FGuid(),
		const TArray<FGuid>& InCallstackGuids = TArray<FGuid>(),
		FNiagaraCompileEventSource InSource = FNiagaraCompileEventSource::Unset
	)
		: Severity(InSeverity)
		, Message(InMessage)
		, ShortDescription(InShortDescription)
		, NodeGuid(InNodeGuid)
		, PinGuid(InPinGuid)
		, StackGuids(InCallstackGuids)
		, Source(InSource) 
	{}
 
	/** Whether or not this is an error, warning, or info*/
	UPROPERTY()
	FNiagaraCompileEventSeverity Severity;
	/** The message itself*/
	UPROPERTY()
	FString Message;
	/** A short, optional description of the event. */
	UPROPERTY()
	FString ShortDescription;
	/** The node guid that generated the compile event*/
	UPROPERTY()
	FGuid NodeGuid;
	/** The pin persistent id that generated the compile event*/
	UPROPERTY()
	FGuid PinGuid;
	/** The compile stack frame of node id's*/
	UPROPERTY()
	TArray<FGuid> StackGuids;
	/** The source of the compile event for partial invalidation purposes. */
	UPROPERTY()
	FNiagaraCompileEventSource Source;

	bool operator==(const FNiagaraCompileEvent& Other) const
	{
		return Other.Severity == Severity && Other.Message == Message && Other.ShortDescription == ShortDescription && Other.NodeGuid == NodeGuid && Other.PinGuid == PinGuid && Other.Source == Source;
	}
};

//
class FNiagaraShaderMapPointerTable : public FShaderMapPointerTable
{
public:
	using Super = FShaderMapPointerTable;

	virtual int32 AddIndexedPointer(const FTypeLayoutDesc& TypeDesc, void* Ptr) override;
	virtual void* GetIndexedPointer(const FTypeLayoutDesc& TypeDesc, uint32 i) const override;

	virtual void SaveToArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, const void* FrozenObject) const override;
	virtual bool LoadFromArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, void* FrozenObject) override;

	TPtrTable<UNiagaraDataInterfaceBase> DITypes;
};

template<typename ShaderType>
using TNiagaraShaderRef = TShaderRefBase<ShaderType, FNiagaraShaderMapPointerTable>;
using FNiagaraShaderRef = TNiagaraShaderRef<FNiagaraShader>;

/**
* Information about a data interface function generated by the translator.
*/
USTRUCT()
struct FNiagaraDataInterfaceGeneratedFunction
{
	GENERATED_USTRUCT_BODY()
	
	/** Name of the function as defined by the data interface. */
	FName DefinitionName;

	/** Name of the instance. Derived from the definition name but made unique for this DI instance and specifier values. */
	FString InstanceName;

	/** Specifier values for this instance. */
	using FunctionSpecifier = TTuple<FName, FName>;
	TArray<FunctionSpecifier> Specifiers;

	uint16 MiscUsageBitMask = 0;

	UPROPERTY()
	TArray<FNiagaraVariableCommonReference> VariadicInputs;
	UPROPERTY()
	TArray<FNiagaraVariableCommonReference> VariadicOutputs;

	const FName* FindSpecifierValue(const FName& Name) const
	{
		for (const TTuple<FName, FName>& Specifier : Specifiers)
		{
			if (Specifier.template Get<0>() == Name)
			{
				return &Specifier.template Get<1>();
			}
		}
		return nullptr;
	}

	NIAGARASHADER_API bool Serialize(FArchive& Ar);
	friend bool operator<<(FArchive& Ar, FNiagaraDataInterfaceGeneratedFunction& DIFunction);

	bool operator==(const FNiagaraDataInterfaceGeneratedFunction& Other) const
	{
		if (DefinitionName != Other.DefinitionName || InstanceName != Other.InstanceName || MiscUsageBitMask != Other.MiscUsageBitMask)
			return false;
		if (Specifiers.Num() != Other.Specifiers.Num())
			return false;
		for (int32 i = 0; i < Specifiers.Num(); i++)
		{
			if (Specifiers[i] != Other.Specifiers[i])
				return false;
		}

		if(VariadicInputs != Other.VariadicInputs)
			return false;
		if(VariadicOutputs != Other.VariadicOutputs)
			return false;

		return true;
	}

	bool operator!=(const FNiagaraDataInterfaceGeneratedFunction& Other) const
	{
		return !(*this == Other);
	}
};

template<> struct TStructOpsTypeTraits<FNiagaraDataInterfaceGeneratedFunction> : public TStructOpsTypeTraitsBase2<FNiagaraDataInterfaceGeneratedFunction>
{
	enum
	{
		WithSerializer = true,
	};
};

/**
* Data coming from that translator that describes parameters needed for each data interface.
*/
USTRUCT()
struct FNiagaraDataInterfaceGPUParamInfo
{
	GENERATED_USTRUCT_BODY()
		
	/** Symbol of this DI in the hlsl. Used for binding parameters. */
	UPROPERTY()
	FString DataInterfaceHLSLSymbol;

	/** Name of the class for this data interface. Used for constructing the correct parameters struct. */
	UPROPERTY()
	FString DIClassName;

	/** The offset for any shader parameters, when this member is INDEX_NONE we are in legacy mode for the data interface. */
	UPROPERTY()
	uint32 ShaderParametersOffset = INDEX_NONE;

	/** Information about all the functions generated by the translator for this data interface. */
	UPROPERTY()
	TArray<FNiagaraDataInterfaceGeneratedFunction> GeneratedFunctions;

	/**
	Check to see if this is an external parameter or not.
	An external parameter is defined as one that is not contained within the Niagara system, i.e. a user or parameter collection parameter.
	*/
	NIAGARASHADER_API bool IsExternalParameter() const;

	UE_DEPRECATED(5.7, "Please use IsExternalParameter as IsUserParameter does not cover NPCs and can result in incorrect behavior")
	NIAGARASHADER_API bool IsUserParameter() const;

	NIAGARASHADER_API bool Serialize(FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FNiagaraDataInterfaceGPUParamInfo> : public TStructOpsTypeTraitsBase2<FNiagaraDataInterfaceGPUParamInfo>
{
	enum
	{
		WithSerializer = true,
	};
};

USTRUCT()
struct FNiagaraShaderScriptExternalConstant
{
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY()
	FName Type;

	UPROPERTY()
	FString	Name;
};

/** Shader side information about global structures includes. */
struct FNiagaraDataInterfaceStructIncludeInfo
{
	const FShaderParametersMetadata* StructMetadata = nullptr;
	int32 ParamterOffset = 0;
};


/* Data provided from the translator to track information requires to bind data interface or parameters at runtime. */
USTRUCT()
struct FNiagaraShaderScriptParametersMetadata
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FNiagaraDataInterfaceGPUParamInfo> DataInterfaceParamInfo;

	UPROPERTY()
	TArray<FString> LooseMetadataNames;

	// Runtime generated bindings, not serialized
	TArray<FNiagaraDataInterfaceStructIncludeInfo> StructIncludeInfos;
	TSharedPtr<FShaderParametersMetadata> ShaderParametersMetadata;
};


/** 
* Shader side data needed for binding data interface parameters.
*/
struct FNiagaraDataInterfaceParamRef
{
	DECLARE_TYPE_LAYOUT(FNiagaraDataInterfaceParamRef, NonVirtual);
public:
	FNiagaraDataInterfaceParamRef() { }
	~FNiagaraDataInterfaceParamRef() { Parameters.SafeDelete(); }

	LAYOUT_FIELD(uint32, ShaderParametersOffset);

	/** Type of Parameters */
	LAYOUT_FIELD(TIndexedPtr<UNiagaraDataInterfaceBase>, DIType);

	/** Pointer to parameters struct for this data interface. */
	void WriteFrozenParameters(FMemoryImageWriter& Writer, const TMemoryImagePtr<FNiagaraDataInterfaceParametersCS>& InParameters) const;
	LAYOUT_FIELD_WITH_WRITER(TMemoryImagePtr<FNiagaraDataInterfaceParametersCS>, Parameters, WriteFrozenParameters);
};

/** Contains all the information needed to uniquely identify a FNiagaraShaderMapID. */
class FNiagaraShaderMapId
{
	DECLARE_TYPE_LAYOUT(FNiagaraShaderMapId, NonVirtual);
public:
	/** The version of the compiler that this needs to be built against.*/
	LAYOUT_FIELD(FGuid, CompilerVersionID);

	/** Feature level that the shader map is going to be compiled for.  */
	LAYOUT_FIELD(ERHIFeatureLevel::Type, FeatureLevel);

	/** Configuration options */
	LAYOUT_FIELD(TMemoryImageArray<FMemoryImageString>, AdditionalDefines);

	/** Configuration options */
	LAYOUT_FIELD(TMemoryImageArray<FMemoryImageString>, AdditionalVariables);

	/**
	* The hash of the subgraph this shader primarily represents.
	*/
	LAYOUT_FIELD(FSHAHash, BaseCompileHash);

	/** The compile hashes of the top level scripts the script is dependent on. */
	LAYOUT_FIELD(TMemoryImageArray<FSHAHash>, ReferencedCompileHashes);

	/*
	 * Type layout parameters of the memory image
	 */
	LAYOUT_FIELD(FPlatformTypeLayoutParameters, LayoutParams);

	/*
	 * Shader type dependencies
	 */
	LAYOUT_FIELD(TMemoryImageArray<FShaderTypeDependency>, ShaderTypeDependencies);

	/** Whether or not we need to bake Rapid Iteration params. True to keep params, false to bake.*/
	LAYOUT_FIELD_INITIALIZED(bool, bUsesRapidIterationParams, true);

	FNiagaraShaderMapId()
		: CompilerVersionID()
		, FeatureLevel(GMaxRHIFeatureLevel)
	{ }

	~FNiagaraShaderMapId() = default;

	//ENGINE_API void SetShaderDependencies(const TArray<FShaderType*>& ShaderTypes, const TArray<const FShaderPipelineType*>& ShaderPipelineTypes, const TArray<FVertexFactoryType*>& VFTypes);

	//void Serialize(FArchive& Ar);

	friend uint32 GetTypeHash(const FNiagaraShaderMapId& Ref)
	{
		return GetTypeHash(Ref.BaseCompileHash);
	}

	SIZE_T GetSizeBytes() const
	{
		return sizeof(*this);
	}

	/** Hashes the script-specific part of this shader map Id. */
	uint64 GetScriptHash() const;

	/**
	* Tests this set against another for equality, disregarding override settings.
	*
	* @param ReferenceSet	The set to compare against
	* @return				true if the sets are equal
	*/
	NIAGARASHADER_API bool operator==(const FNiagaraShaderMapId& ReferenceSet) const;

	bool operator!=(const FNiagaraShaderMapId& ReferenceSet) const
	{
		return !(*this == ReferenceSet);
	}

#if WITH_EDITOR
	/** Appends string representations of this Id to a key string. */
	NIAGARASHADER_API void AppendKeyString(FString& KeyString) const;
#endif

	/** Returns true if the requested shader type is a dependency of this shader map Id. */
	//bool ContainsShaderType(const FShaderType* ShaderType) const;
};






// Runtime code sticks scripts to compile along with their shader map here
// Niagara Editor ticks in FNiagaraShaderQueueTickable, kicking off compile jobs
#if WITH_EDITORONLY_DATA

class FNiagaraCompilationQueue
{
public:
	struct NiagaraCompilationQueueItem
	{
		FNiagaraShaderScript* Script;
		FNiagaraShaderMapRef ShaderMap;
		FNiagaraShaderMapId ShaderMapId;
		EShaderPlatform Platform;
		bool bApply;
	};

	static FNiagaraCompilationQueue *Get()
	{
		if (Singleton == nullptr)
		{
			Singleton = new FNiagaraCompilationQueue();
		}
		return Singleton;
	}

	TArray<NiagaraCompilationQueueItem> &GetQueue()
	{
		return CompilationQueue;
	}

	void Queue(FNiagaraShaderScript *InScript, FNiagaraShaderMapRef InShaderMap, const FNiagaraShaderMapId &MapId, EShaderPlatform InPlatform, bool InApply)
	{
		check(IsInGameThread());
		NiagaraCompilationQueueItem NewQueueItem;
		NewQueueItem.Script = InScript;
		NewQueueItem.ShaderMap = InShaderMap;
		NewQueueItem.ShaderMapId = MapId;
		NewQueueItem.Platform = InPlatform;
		NewQueueItem.bApply = InApply;
		CompilationQueue.Add(NewQueueItem);
	}

	void RemovePending(FNiagaraShaderScript* InScript)
	{
		check(IsInGameThread());
		for (NiagaraCompilationQueueItem& Item : CompilationQueue)
		{
			if (Item.Script == InScript)
			{
				Item.Script = nullptr;
			}
		}
	}

private:
	TArray<NiagaraCompilationQueueItem> CompilationQueue;
	NIAGARASHADER_API static FNiagaraCompilationQueue *Singleton;
};

#endif

class FNiagaraShaderMapContent : public FShaderMapContent
{
	using Super = FShaderMapContent;
	friend class FNiagaraShaderMap;
	DECLARE_TYPE_LAYOUT(FNiagaraShaderMapContent, NonVirtual);
private:
	explicit FNiagaraShaderMapContent(const EShaderPlatform InPlatform) : Super(InPlatform) {}

	/** The static parameter set that this shader map was compiled with */
	LAYOUT_FIELD(FNiagaraShaderMapId, ShaderMapId);
};

/**
* The set of shaders for a single script.
*/
class FNiagaraShaderMap : public TShaderMap<FNiagaraShaderMapContent, FNiagaraShaderMapPointerTable>, public FDeferredCleanupInterface
{
public:
	using Super = TShaderMap<FNiagaraShaderMapContent, FNiagaraShaderMapPointerTable>;

	enum EWorkerThread { WorkerThread };

	/**
	* Finds the shader map for a script.
	* @param Platform - The platform to lookup for
	* @return NULL if no cached shader map was found.
	*/
	static FNiagaraShaderMap* FindId(const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform Platform);

	// ShaderMap interface
	template<typename ShaderType> TNiagaraShaderRef<ShaderType> GetShader(int32 PermutationId) const { return TNiagaraShaderRef<ShaderType>(GetContent()->GetShader<ShaderType>(PermutationId), *this); }
	TNiagaraShaderRef<FShader> GetShader(FShaderType* ShaderType, int32 PermutationId) const { return TNiagaraShaderRef<FShader>(GetContent()->GetShader(ShaderType, PermutationId), *this); }

	//static void FixupShaderTypes(EShaderPlatform Platform, const TMap<FShaderType*, FString>& ShaderTypeNames);

#if WITH_EDITOR
	/**
	* Attempts to load the shader map for the given script from the Derived Data Cache.
	* If InOutShaderMap is valid, attempts to load the individual missing shaders instead.
	*/
	static void LoadFromDerivedDataCache(const FNiagaraShaderScript* Script, const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform Platform, FNiagaraShaderMapRef& InOutShaderMap);
#endif

	NIAGARASHADER_API FNiagaraShaderMap();
#if WITH_EDITOR
	NIAGARASHADER_API FNiagaraShaderMap(EWorkerThread);
#endif

	// Destructor.
	NIAGARASHADER_API ~FNiagaraShaderMap();

#if WITH_EDITOR
	/**
	* Compiles the shaders for a script and caches them in this shader map.
	* @param script - The script to compile shaders for.
	* @param ShaderMapId - the set of static parameters to compile for
	* @param Platform - The platform to compile to
	*/
	NIAGARASHADER_API void Compile(
		FNiagaraShaderScript* Script,
		const FNiagaraShaderMapId& ShaderMapId,
		TRefCountPtr<FSharedShaderCompilerEnvironment> CompilationEnvironment,
		EShaderPlatform Platform,
		bool bSynchronousCompile,
		bool bApplyCompletedShaderMapForRendering
		);

	/**
	* Compiles the shaders for a script and caches them in this shader map.
	* @param script - The script to compile shaders for.
	* @param ShaderMapId - the set of static parameters to compile for
	* @param Platform - The platform to compile to
	*/
	NIAGARASHADER_API void CreateCompileJobs(
		const FNiagaraShaderType* ShaderType,
		FStringView FriendlyName,
		const FNiagaraShaderMapId& ShaderMapId,
		FStringView SourceSource,
		TRefCountPtr<FSharedShaderCompilerEnvironment> CompilationEnvironment,
		TConstArrayView<FSimulationStageMetaData> StageMetaData,
		int32 PermutationCount,
		EShaderPlatform Platform,
		TSharedPtr<FNiagaraShaderScriptParametersMetadata> ShaderParameters,
		TArray<TRefCountPtr<FShaderCommonCompileJob>>& CompileJobs);

	NIAGARASHADER_API void ProcessAndFinalizeShaderCompileJob(const TRefCountPtr<FShaderCommonCompileJob>& SingleJob);

	/** Sorts the incoming compiled jobs into the appropriate mesh shader maps, and finalizes this shader map so that it can be used for rendering. */
	bool ProcessCompilationResults(const TArray<FNiagaraShaderScript*>& InScripts, const TArray<TRefCountPtr<class FShaderCommonCompileJob>>& InCompilationResults, int32& ResultIndex, float& TimeBudget);

	/**
	* Checks whether the shader map is missing any shader types necessary for the given script.
	* @param Script - The Niagara Script which is checked.
	* @return True if the shader map has all of the shader types necessary.
	*/
	bool IsComplete(const FNiagaraShaderScript* Script, bool bSilent);

	/**
	* Checks to see if the shader map is already being compiled for another script, and if so
	* adds the specified script to the list to be applied to once the compile finishes.
	* @param Script - The Niagara Script we also wish to apply the compiled shader map to.
	* @return True if the shader map was being compiled and we added Script to the list to be applied.
	*/
	bool TryToAddToExistingCompilationTask(FNiagaraShaderScript* Script);
#endif // WITH_EDITOR

	/** Builds a list of the shaders in a shader map. */
	NIAGARASHADER_API void GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const;
	NIAGARASHADER_API virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const override;
	NIAGARASHADER_API virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const override;

	/** Registers a niagara shader map in the global map so it can be used by Niagara scripts. */
	void Register(EShaderPlatform InShaderPlatform);

	// Reference counting.
	NIAGARASHADER_API  void AddRef();
	NIAGARASHADER_API  void Release();

#if WITH_EDITOR
	/** Removes a Script from NiagaraShaderMapsBeingCompiled. 
	* @return true if something was actually removed.
	*/
	NIAGARASHADER_API static bool RemovePendingScript(FNiagaraShaderScript* Script);
	NIAGARASHADER_API static void RemovePendingMap(FNiagaraShaderMap* Map);

	/** Finds a shader map currently being compiled that was enqueued for the given script. */
	static const FNiagaraShaderMap* GetShaderMapBeingCompiled(const FNiagaraShaderScript* Script);
#endif // WITH_EDITOR

	/** Serializes the shader map. */
	UE_DEPRECATED(5.5, "Use overload taking a FShaderSerializeContext")
	NIAGARASHADER_API bool Serialize(FArchive& Ar, bool bInlineShaderResources = true, bool bLoadingCooked = false);

	NIAGARASHADER_API bool Serialize(FShaderSerializeContext& Ctx);

#if WITH_EDITOR
	/** Saves this shader map to the derived data cache. */
	void SaveToDerivedDataCache(const FNiagaraShaderScript* Script);
#endif // WITH_EDITOR

	// Accessors.
	const FNiagaraShaderMapId& GetShaderMapId() const		{ return GetContent()->ShaderMapId; }
	EShaderPlatform GetShaderPlatform() const				{ return GetContent()->GetShaderPlatform(); }
	uint32 GetCompilingId() const							{ return CompilingId; }
	bool IsCompilationFinalized() const						{ return bCompilationFinalized; }
	bool CompiledSuccessfully() const						{ return bCompiledSuccessfully; }

	bool IsValid() const
	{
		return bCompilationFinalized && bCompiledSuccessfully && !bDeletedThroughDeferredCleanup;
	}

	//const FUniformExpressionSet& GetUniformExpressionSet() const { return NiagaraCompilationOutput.UniformExpressionSet; }

	int32 GetNumRefs() const { return NumRefs; }
	uint32 GetCompilingId()  { return CompilingId; }

#if WITH_EDITOR
	static TMap<FNiagaraShaderMapRef, TArray<FNiagaraShaderScript*>>& GetInFlightShaderMaps()
	{
		//All access to NiagaraShaderMapsBeingCompiled must be done on the game thread!
		check(IsInGameThread());
		return NiagaraShaderMapsBeingCompiled; 
	}

	void SetCompiledSuccessfully(bool bSuccess) { bCompiledSuccessfully = bSuccess; }
#endif // WITH_EDITOR

private:

	/**
	* A global map from a script's ID and static switch set to any shader map cached for that script.
	* Note: this does not necessarily contain all script shader maps in memory.  Shader maps with the same key can evict each other.
	* No ref counting needed as these are removed on destruction of the shader map.
	*/
	static TMap<FNiagaraShaderMapId, FNiagaraShaderMap*> GIdToNiagaraShaderMap[SP_NumPlatforms];

#if WITH_EDITOR
	/** Tracks resources and their shader maps that need to be compiled but whose compilation is being deferred. */
	static TMap<FNiagaraShaderMapRef, TArray<FNiagaraShaderScript*> > NiagaraShaderMapsBeingCompiled;
#endif

	/** Uniquely identifies this shader map during compilation, needed for deferred compilation where shaders from multiple shader maps are compiled together. */
	uint32 CompilingId;

	mutable int32 NumRefs;

	/** Used to catch errors where the shader map is deleted directly. */
	bool bDeletedThroughDeferredCleanup;

	/** Indicates whether this shader map has been registered in GIdToNiagaraShaderMap */
	uint32 bRegistered : 1;

	/**
	* Indicates whether this shader map has had ProcessCompilationResults called after Compile.
	* The shader map must not be used on the rendering thread unless bCompilationFinalized is true.
	*/
	uint32 bCompilationFinalized : 1;

	uint32 bCompiledSuccessfully : 1;

	/** Indicates whether the shader map should be stored in the shader cache. */
	uint32 bIsPersistent : 1;

#if WITH_EDITOR
	FShader* ProcessCompilationResultsForSingleJob(const TRefCountPtr<class FShaderCommonCompileJob>& SingleJob, uint64 ShaderMapHash);

	bool IsNiagaraShaderComplete(const FNiagaraShaderScript* Script, const FNiagaraShaderType* ShaderType, bool bSilent);
#endif

	friend class FShaderCompilingManager;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnNiagaraScriptCompilationComplete);

/**
 * FNiagaraShaderScript represents a Niagara script to the shader compilation process
 */
class FNiagaraShaderScript
{
	struct FNiagaraShaderMapCachedData
	{
		FNiagaraShaderMapCachedData()
		{
			NumPermutations = 0;
			bIsComplete = 0;
			bExternalConstantBufferUsed = 0;
			bViewUniformBufferUsed = 0;
		}

		int32 NumPermutations;
		uint32 bIsComplete : 1;
		uint32 bExternalConstantBufferUsed : 2;
		uint32 bViewUniformBufferUsed : 1;
	};

public:

	/**
	 * Minimal initialization constructor.
	 */
	NIAGARASHADER_API FNiagaraShaderScript();

	/**
	 * Destructor
	 */
	NIAGARASHADER_API  virtual ~FNiagaraShaderScript();

	/**
	 * Caches the shaders for this script with no static parameters on the given platform.
	 */
	NIAGARASHADER_API  bool CacheShaders(bool bApplyCompletedShaderMapForRendering, bool bForceRecompile, bool bSynchronous = false, const ITargetPlatform* TargetPlatform = nullptr);
	bool CacheShaders(const FNiagaraShaderMapId& ShaderMapId, bool bApplyCompletedShaderMapForRendering, bool bForceRecompile, bool bSynchronous = false);

	NIAGARASHADER_API bool GetUsesCompressedAttributes() const;

	/**
	 * Should the shader for this script with the given platform, shader type and vertex
	 * factory type combination be compiled
	 *
	 * @param Platform		The platform currently being compiled for
	 * @param ShaderType	Which shader is being compiled
	 * @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	 *
	 * @return true if the shader should be compiled
	 */
	NIAGARASHADER_API  virtual bool ShouldCache(EShaderPlatform Platform, const FShaderType* ShaderType) const;

	/**
	 * Allow Niagara script the opportunity to modify the compilation environment for GPU simulations.
	 * Generally used for data interfaces to inject project defines.
	*/
	NIAGARASHADER_API  virtual void ModifyCompilationEnvironment(EShaderPlatform Platform, struct FShaderCompilerEnvironment& OutEnvironment) const;

	/** Serializes the script. */
	NIAGARASHADER_API void SerializeShaderMap(FArchive& Ar);

	/** Releases this script's shader map.  Must only be called on scripts not exposed to the rendering thread! */
	NIAGARASHADER_API void ReleaseShaderMap();

#if WITH_EDITOR
	void GetDependentShaderTypes(EShaderPlatform Platform, TArray<FShaderType*>& OutShaderTypes) const;

	NIAGARASHADER_API  virtual void GetShaderMapId(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, FNiagaraShaderMapId& OutId) const;
#endif

	NIAGARASHADER_API void Invalidate();

	/**
	 * Should shaders compiled for this script be saved to disk?
	 */
	virtual bool IsPersistent() const { return true; }

#if WITH_EDITOR
	/**
	* Called when compilation finishes, after the GameThreadShaderMap is set and the render command to set the RenderThreadShaderMap is queued
	*/
	NIAGARASHADER_API virtual void NotifyCompilationFinished();

	/**
	* Cancels all outstanding compilation jobs
	*/
	NIAGARASHADER_API  void CancelCompilation();

	/**
	 * Blocks until compilation has completed. Returns immediately if a compilation is not outstanding.
	 */
	NIAGARASHADER_API  void FinishCompilation();
#endif // WITH_EDITOR

	/**
	 * Checks if the compilation for this shader is finished
	 *
	 * @return returns true if compilation is complete false otherwise
	 */
	NIAGARASHADER_API  bool IsCompilationFinished() const;

#if WITH_EDITOR
	// Accessors.
	const TArray<FString>& GetCompileErrors() const { return CompileErrors; }
	void SetCompileErrors(const TArray<FString>& InCompileErrors) { CompileErrors = InCompileErrors; }
#endif

	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }
	EShaderPlatform GetShaderPlatform() const { return ShaderPlatform; }

	class FNiagaraShaderMap* GetGameThreadShaderMap() const
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		return GameThreadShaderMap;
	}

	/** Note: SetRenderingThreadShaderMap must also be called with the same value, but from the rendering thread. */
	void SetGameThreadShaderMap(FNiagaraShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
	}

	/** Note: SetGameThreadShaderMap must also be called with the same value, but from the game thread. */
	NIAGARASHADER_API void SetRenderingThreadShaderMap(FNiagaraShaderMap* InShaderMap);
	void SetRenderThreadCachedData(const FNiagaraShaderMapCachedData& CachedData);

	NIAGARASHADER_API bool QueueForRelease(FThreadSafeBool& Fence);

#if WITH_EDITOR
	void AddCompileId(uint32 Id) 
	{
		check(IsInGameThread());
		OutstandingCompileShaderMapIds.Add(Id);
	}
#endif // WITH_EDITOR

	void SetShaderMap(FNiagaraShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
		bLoadedCookedShaderMapId = true;
		CookedShaderMapId = InShaderMap->GetShaderMapId();
	}

#if WITH_EDITOR
	NIAGARASHADER_API void RemoveOutstandingCompileId(const int32 OldOutstandingCompileShaderMapId);
#endif

	NIAGARASHADER_API  virtual void AddReferencedObjects(FReferenceCollector& Collector);

#if WITH_EDITOR
	/**
	* Get user source code for the shader
	* @param OutSource - generated source code
	* @param OutHighlightMap - source code highlight list
	* @return - true on Success
	*/
	bool GetScriptHLSLSource(FString& OutSource) const
	{
		OutSource = HlslOutput;
		return true;
	};

	void SetHlslOutput(const FString& InHlslOutput) { HlslOutput = InHlslOutput; }
	void SetSourceName(FString InSourceName) { SourceName = InSourceName; }

	/** Save "stable" shader for identifying the shader in the recorded PSO cache. */
	NIAGARASHADER_API void SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, struct FStableShaderKeyAndValue& SaveKeyVal); // arg is non-const, we modify it as we go
#endif

	const FString& GetFriendlyName() const { return FriendlyName; }

	NIAGARASHADER_API void SetScript(UNiagaraScriptBase* InScript, ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, const FGuid& InCompilerVersion, const TArray<FString>& InAdditionalDefines, const TArray<FString>& InAdditionalVariables,
		const FNiagaraCompileHash& InBaseCompileHash, const TArray<FNiagaraCompileHash>& InReferencedCompileHashes, 
		bool bInUsesRapidIterationParams, FString InFriendlyName);

#if WITH_EDITOR
	NIAGARASHADER_API bool MatchesScript(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, const FNiagaraVMExecutableDataId& ScriptId) const;
#endif

	UNiagaraScriptBase* GetBaseVMScript() { return BaseVMScript; }
	const UNiagaraScriptBase* GetBaseVMScript() const { return BaseVMScript; }
	/** Returns owner name for tracking */
	NIAGARASHADER_API FName GetOwnerFName() const;

	NIAGARASHADER_API  FNiagaraShaderRef GetShader(int32 PermutationId) const;
	NIAGARASHADER_API  FNiagaraShaderRef GetShaderGameThread(int32 PermutationId) const;
	
	NIAGARASHADER_API void BuildScriptParametersMetadata(const FNiagaraShaderScriptParametersMetadata& ScriptParametersMetadata);
	TSharedRef<FNiagaraShaderScriptParametersMetadata> GetScriptParametersMetadata() const { check(ScriptParametersMetadata.IsValid()); return ScriptParametersMetadata.ToSharedRef(); }
	const FNiagaraShaderScriptParametersMetadata& GetScriptParametersMetadata_RT() const { check(ScriptParametersMetadata_RT.IsValid()); return *ScriptParametersMetadata_RT.Get(); }

	FOnNiagaraScriptCompilationComplete& OnCompilationComplete()
	{
		return OnCompilationCompleteDelegate;
	}

	NIAGARASHADER_API bool IsSame(const FNiagaraShaderMapId& InId) const;

	int32 GetNumPermutations() const { return NumPermutations; }

	NIAGARASHADER_API bool IsShaderMapComplete() const;

	inline bool IsShaderMapComplete_RenderThread() const { check(IsInRenderingThread()); return CachedData_RenderThread.bIsComplete != 0; }
	inline int32 GetNumPermutations_RenderThread() const { check(IsInRenderingThread()); return CachedData_RenderThread.NumPermutations; }
	inline bool IsExternalConstantBufferUsed_RenderThread(int32 Index) const { return (CachedData_RenderThread.bExternalConstantBufferUsed & (1 << Index)) != 0; }
	inline bool IsViewUniformBufferUsed_RenderThread() const { return CachedData_RenderThread.bViewUniformBufferUsed != 0; }

	FNiagaraCompileHash GetBaseCompileHash()const { return BaseCompileHash; }

protected:
#if WITH_EDITOR
	/**
	* Fills the passed array with IDs of shader maps unfinished compilation jobs.
	*/
	void GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& ShaderMapIds);
#endif

	void SetFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel)
	{
		FeatureLevel = InFeatureLevel;
	}

	void UpdateCachedData_All();
	void UpdateCachedData_PreCompile();
	void UpdateCachedData_PostCompile(bool bCalledFromSerialize = false);

private:
	UNiagaraScriptBase* BaseVMScript;

#if WITH_EDITOR
	FString SourceName;
	FString HlslOutput;
	TArray<FString> CompileErrors;
#endif

	/** 
	 * Game thread tracked shader map, which is ref counted and manages shader map lifetime. 
	 * The shader map uses deferred deletion so that the rendering thread has a chance to process a release command when the shader map is no longer referenced.
	 * Code that sets this is responsible for updating RenderingThreadShaderMap in a thread safe way.
	 * During an async compile, this will be NULL and will not contain the actual shader map until compilation is complete.
	 */
	FNiagaraShaderMapRef GameThreadShaderMap;

	/** 
	 * Shader map for this FNiagaraShaderScript which is accessible by the rendering thread. 
	 * This must be updated along with GameThreadShaderMap, but on the rendering thread.
	 */
	FNiagaraShaderMap* RenderingThreadShaderMap;

	TSharedPtr<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata;
	TSharedPtr<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata_RT;

	/** Configuration options */
	TArray<FString> AdditionalDefines;

	/** Additional variables needed*/
	TArray<FString> AdditionalVariables;

	/** Whether or not we need to bake Rapid Iteration params. True to keep params, false to bake.*/
	bool bUsesRapidIterationParams = true;

	/** Compile hash for the base script. */
	FNiagaraCompileHash BaseCompileHash;

	/** The compiler version the script was generated with.*/
	FGuid CompilerVersionId;

	/** The compile hashes for the top level scripts referenced by the script. */
	TArray<FNiagaraCompileHash> ReferencedCompileHashes;

#if WITH_EDITOR
	/** 
	 * Contains the compiling id of this shader map when it is being compiled asynchronously. 
	 * This can be used to access the shader map during async compiling, since GameThreadShaderMap will not have been set yet.
	 */
	TArray<int32, TInlineAllocator<1> > OutstandingCompileShaderMapIds;
#endif // WITH_EDITOR

	/** Feature level and shader platform that this script is representing. */
	ERHIFeatureLevel::Type FeatureLevel;
	EShaderPlatform ShaderPlatform;

	uint32 bLoadedCookedShaderMapId : 1;
	uint32 bLoadedFromCookedMaterial : 1;
	uint32 bQueuedForRelease : 1;

	int32 NumPermutations = 0;

	FNiagaraShaderMapCachedData CachedData_RenderThread;

	FNiagaraShaderMapId CookedShaderMapId;

	FOnNiagaraScriptCompilationComplete OnCompilationCompleteDelegate;

#if WITH_EDITOR
	/**
	* Compiles this script for Platform, storing the result in OutShaderMap if the compile was synchronous
	*/
	bool BeginCompileShaderMap(
		const FNiagaraShaderMapId& ShaderMapId,
		TRefCountPtr<class FNiagaraShaderMap>& OutShaderMap, 
		bool bApplyCompletedShaderMapForRendering,
		bool bSynchronous = false);

	/** Populates OutEnvironment with defines needed to compile shaders for this script. */
	void SetupShaderCompilationEnvironment(
		EShaderPlatform Platform,
		FShaderCompilerEnvironment& OutEnvironment
		) const;
#endif // WITH_EDITOR

	FString FriendlyName;

	friend class FNiagaraShaderMap;
	friend class FShaderCompilingManager;
};

/** CVars */
extern NIAGARASHADER_API int32 GNiagaraTranslatorFailIfNotSetSeverity;
extern NIAGARASHADER_API int32 GbNiagaraEnableTraversalCache;

namespace FNiagaraCVarUtilities
{
	NIAGARASHADER_API FNiagaraCompileEventSeverity GetCompileEventSeverityForFailIfNotSet();
	NIAGARASHADER_API bool GetShouldEmitMessagesForFailIfNotSet();
};
