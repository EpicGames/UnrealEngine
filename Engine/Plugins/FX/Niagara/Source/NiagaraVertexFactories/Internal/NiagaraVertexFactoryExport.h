// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/MemoryLayout.h"
#include "MeshMaterialShader.h"
#include "RHIStaticStates.h"

class FVertexFactoryType;

class FNiagaraVertexFactoryExportCS : public FMeshMaterialShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FNiagaraVertexFactoryExportCS, MeshMaterial, NIAGARAVERTEXFACTORIES_API);

public:
	static constexpr uint32 ThreadGroupSize = 64;

	NIAGARAVERTEXFACTORIES_API FNiagaraVertexFactoryExportCS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer);
	FNiagaraVertexFactoryExportCS() = default;

	NIAGARAVERTEXFACTORIES_API static bool IsEnabled();
	NIAGARAVERTEXFACTORIES_API static bool SupportsVertexFactoryType(const FVertexFactoryType* VertexFactory);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
	}

	void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch,
		const FMeshBatchElement& BatchElement,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FMeshMaterialShader::GetElementShaderBindings(PointerTable, Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);
	}

	LAYOUT_FIELD(FShaderParameter,			IsIndirectDraw);
	LAYOUT_FIELD(FShaderParameter,			NumInstances);
	LAYOUT_FIELD(FShaderParameter,			NumVerticesPerInstance);
	LAYOUT_FIELD(FShaderParameter,			bApplyWPO);
	LAYOUT_FIELD(FShaderParameter,			bWorldSpace);
	LAYOUT_FIELD(FShaderParameter,			SectionInfoOutputOffset);

	LAYOUT_FIELD(FShaderParameter,			VertexStride);
	LAYOUT_FIELD(FShaderParameter,			VertexPositionOffset);
	LAYOUT_FIELD(FShaderParameter,			VertexColorOffset);
	LAYOUT_FIELD(FShaderParameter,			VertexTangentBasisOffset);
	LAYOUT_FIELD(FShaderParameter,			VertexTexCoordOffset);
	LAYOUT_FIELD(FShaderParameter,			VertexTexCoordNum);
	LAYOUT_FIELD(FShaderParameter,			VertexTexCoordBindings);
	LAYOUT_FIELD(FShaderParameter,			VertexOutputOffset);
	LAYOUT_FIELD(FShaderResourceParameter,	RWVertexData);
};
