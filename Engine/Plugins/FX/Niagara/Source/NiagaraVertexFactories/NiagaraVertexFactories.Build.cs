// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NiagaraVertexFactories : ModuleRules
{
    public NiagaraVertexFactories(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"Engine",		
				"RenderCore",
				"Renderer",
				"RHI",
            }
        );

		PublicIncludePathModuleNames.AddRange(
			new string[] {
                "NiagaraCore",
                "Niagara",
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[] {
				"VectorVM",
            }
        );


        PrivateIncludePaths.AddRange(
            new string[] {
            })
        ;
    }
}
