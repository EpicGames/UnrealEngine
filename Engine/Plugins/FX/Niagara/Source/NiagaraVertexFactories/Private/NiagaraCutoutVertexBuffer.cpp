// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraCutoutVertexBuffer.h: Niagara cutout uv buffer implementation.
=============================================================================*/

#include "NiagaraCutoutVertexBuffer.h"
#include "RHI.h"
#include "RHICommandList.h"

FNiagaraCutoutVertexBuffer::FNiagaraCutoutVertexBuffer(int32 ZeroInitCount)
{
	if (ZeroInitCount > 0)
	{
		Data.AddZeroed(ZeroInitCount);
	}
}

void FNiagaraCutoutVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (Data.Num())
	{
		// create a static vertex buffer
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector2f>(TEXT("FNiagaraCutoutVertexBuffer"), Data.Num())
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FVector2f> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		Initializer.WriteArray(MakeConstArrayView(Data));

		VertexBufferRHI = Initializer.Finalize();
		VertexBufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_G32R32F));

		Data.Empty();
	}
}

void FNiagaraCutoutVertexBuffer::ReleaseRHI()
{
	VertexBufferSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

TGlobalResource<FNiagaraCutoutVertexBuffer> GFNiagaraNullCutoutVertexBuffer(4);

