// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDispatchIndirect.h"

IMPLEMENT_GLOBAL_SHADER(FNiagaraDispatchIndirectArgsGenCS, "/Plugin/FX/Niagara/Private/NiagaraDispatchIndirectArgsGen.usf", "IndirectArgsGenCS", SF_Compute);

void FNiagaraDispatchIndirectArgsGenCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), ThreadCount);
}
