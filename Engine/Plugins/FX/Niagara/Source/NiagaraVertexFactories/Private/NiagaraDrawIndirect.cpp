// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraDrawIndirect.cpp : Niagara shader to generate the draw indirect args for Niagara renderers.
==============================================================================*/

#include "NiagaraDrawIndirect.h"
#include "NiagaraGPUSortInfo.h"
#include "ShaderParameterUtils.h"

IMPLEMENT_GLOBAL_SHADER(FNiagaraDrawIndirectArgsGenCS, "/Plugin/FX/Niagara/Private/NiagaraDrawIndirectArgsGen.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraDrawIndirectResetCountsCS, "/Plugin/FX/Niagara/Private/NiagaraDrawIndirectArgsGen.usf", "ResetCountsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FNiagaraInstanceCountsInitCS, "/Plugin/FX/Niagara/Private/NiagaraDrawIndirectArgsGen.usf", "InitInstanceCountsCS", SF_Compute);

void FNiagaraDrawIndirectArgsGenCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT);
	OutEnvironment.SetDefine(TEXT("NIAGARA_DRAW_INDIRECT_ARGS_SIZE"), NIAGARA_DRAW_INDIRECT_ARGS_SIZE);
	OutEnvironment.SetDefine(TEXT("NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE"), NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE);
	OutEnvironment.SetDefine(TEXT("NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE"), NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE);
}

void FNiagaraDrawIndirectResetCountsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT);
	OutEnvironment.SetDefine(TEXT("NIAGARA_DRAW_INDIRECT_ARGS_SIZE"), NIAGARA_DRAW_INDIRECT_ARGS_SIZE);
	OutEnvironment.SetDefine(TEXT("NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE"), NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE);
	OutEnvironment.SetDefine(TEXT("NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE"), NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE);
}

void FNiagaraInstanceCountsInitCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT);
	OutEnvironment.SetDefine(TEXT("NIAGARA_DRAW_INDIRECT_ARGS_SIZE"), NIAGARA_DRAW_INDIRECT_ARGS_SIZE);
	OutEnvironment.SetDefine(TEXT("NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE"), NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE);
	OutEnvironment.SetDefine(TEXT("NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE"), NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE);
}
