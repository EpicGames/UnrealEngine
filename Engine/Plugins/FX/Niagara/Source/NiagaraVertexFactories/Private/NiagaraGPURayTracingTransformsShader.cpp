// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraGPURayTracingTransformsShader.cpp : Niagara shader to generate the ray tracing instances information.
==============================================================================*/

#include "NiagaraGPURayTracingTransformsShader.h"
#include "NiagaraGPUSortInfo.h"
#include "ShaderParameterUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderUtils.h"

#if RHI_RAYTRACING
IMPLEMENT_GLOBAL_SHADER(FNiagaraGPURayTracingTransformsCS, "/Plugin/FX/Niagara/Private/NiagaraGPURayTracingTransforms.usf", "NiagaraGPURayTracingTransformsCS", SF_Compute);
#endif // RHI_RAYTRACING

bool FNiagaraGPURayTracingTransformsCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsRayTracingEnabledForProject(Parameters.Platform) && RHISupportsRayTracing(Parameters.Platform) && !IsMobilePlatform(Parameters.Platform);
}

void FNiagaraGPURayTracingTransformsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
}
