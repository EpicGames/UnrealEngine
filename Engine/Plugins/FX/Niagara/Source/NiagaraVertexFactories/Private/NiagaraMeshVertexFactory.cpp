// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.cpp: Particle vertex factory implementation.
=============================================================================*/

#include "NiagaraMeshVertexFactory.h"
#include "ParticleHelper.h"
#include "ParticleResources.h"
#include "ShaderParameterUtils.h"
#include "MaterialDomain.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "GlobalRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LocalVertexFactory.h"

// Main control for GPU scene on all platforms
#define NIAGARA_ENABLE_GPU_SCENE_MESHES 1

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraMeshUniformParameters, "NiagaraMeshVF");

class FNiagaraMeshVertexFactoryShaderParametersVS : public FNiagaraVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FNiagaraMeshVertexFactoryShaderParametersVS, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		FNiagaraVertexFactoryShaderParametersBase::Bind(ParameterMap);
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);

		const FNiagaraMeshVertexFactory* NiagaraMeshVF = static_cast<const FNiagaraMeshVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraMeshUniformParameters>(), NiagaraMeshVF->GetUniformBuffer());
	}
};

IMPLEMENT_TYPE_LAYOUT(FNiagaraMeshVertexFactoryShaderParametersVS);

class FNiagaraMeshVertexFactoryShaderParametersPS : public FNiagaraVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FNiagaraMeshVertexFactoryShaderParametersPS, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);

		const FNiagaraMeshVertexFactory* NiagaraMeshVF = static_cast<const FNiagaraMeshVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraMeshUniformParameters>(), NiagaraMeshVF->GetUniformBuffer());
	}

};

bool FNiagaraMeshVertexFactory::IsGPUSceneEnabled(const FStaticShaderPlatform Platform, const FStaticFeatureLevel FeatureLevel)
{
#if NIAGARA_ENABLE_GPU_SCENE_MESHES
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		return false;
	}
	return UseGPUScene(Platform);
#else
	return false;
#endif
}

IMPLEMENT_TYPE_LAYOUT(FNiagaraMeshVertexFactoryShaderParametersPS);

void FNiagaraMeshVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, bool bSupportsManualVertexFetch, FStaticMeshDataType& Data, FVertexDeclarationElementList& Elements, FVertexStreamList& InOutStreams)
{
	if (Data.PositionComponent.VertexBuffer != NULL)
	{
		Elements.Add(AccessStreamComponent(Data.PositionComponent, 0, InOutStreams));
	}

	if (!bSupportsManualVertexFetch)
	{
		// only tangent,normal are used by the stream. the binormal is derived in the shader
		uint8 TangentBasisAttributes[2] = { 1, 2 };
		for (int32 AxisIndex = 0; AxisIndex < 2; AxisIndex++)
		{
			if (Data.TangentBasisComponents[AxisIndex].VertexBuffer != NULL)
			{
				Elements.Add(AccessStreamComponent(Data.TangentBasisComponents[AxisIndex], TangentBasisAttributes[AxisIndex], InOutStreams));
			}
		}

		if (Data.ColorComponentsSRV == nullptr)
		{
			Data.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
			Data.ColorIndexMask = 0;
		}

		// Vertex color
		if (Data.ColorComponent.VertexBuffer != NULL)
		{
			Elements.Add(AccessStreamComponent(Data.ColorComponent, 3, InOutStreams));
		}
		else
		{
			//If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
			//This wastes 4 bytes of bandwidth per vertex, but prevents having to compile out twice the number of vertex factories.
			FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
			Elements.Add(AccessStreamComponent(NullColorComponent, 3, InOutStreams));
		}

		if (Data.TextureCoordinates.Num())
		{
			const int32 BaseTexCoordAttribute = 4;
			AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, Data.TextureCoordinates, Elements, InOutStreams);
		}
	}
}

void FNiagaraMeshVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	check(HasValidFeatureLevel());
	const bool bSupportsManualVertexFetch = SupportsManualVertexFetch(GetFeatureLevel());

	FVertexDeclarationElementList Elements;
	GetVertexElements(GetFeatureLevel(), bSupportsManualVertexFetch, Data, Elements, Streams);

#if NIAGARA_ENABLE_GPU_SCENE_MESHES
	if (bAddPrimitiveIDElement)
	{
		AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, 13, 13);
	}
#endif

	//if (Streams.Num() > 0)
	{
		InitDeclaration(Elements);
		check(IsValidRef(GetDeclaration()));
	}
}

bool FNiagaraMeshVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return	FNiagaraUtilities::SupportsNiagaraRendering(Parameters.Platform)
			&& (Parameters.MaterialParameters.bIsUsedWithNiagaraMeshParticles || Parameters.MaterialParameters.bIsSpecialEngineMaterial)
			&& (Parameters.MaterialParameters.MaterialDomain != MD_Volume);
}

void FNiagaraMeshVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FNiagaraVertexFactoryBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	const ERHIFeatureLevel::Type MaxSupportedFeatureLevel = GetMaxSupportedFeatureLevel(Parameters.Platform);
	const bool bUseGPUScene = IsGPUSceneEnabled(Parameters.Platform, MaxSupportedFeatureLevel);

	// Set a define so we can tell in MaterialTemplate.usf when we are compiling a mesh particle vertex factory
	OutEnvironment.SetDefine(TEXT("NIAGARA_MESH_FACTORY"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("NIAGARA_MESH_INSTANCED"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("NiagaraVFLooseParameters"), TEXT("NiagaraMeshVF"));
	OutEnvironment.SetDefine(TEXT("NIAGARA_MESH_WITH_GPU_SCENE"), bUseGPUScene);

#if NIAGARA_ENABLE_GPU_SCENE_MESHES
	const bool bSupportsPrimitiveIdStream = Parameters.VertexFactoryType->SupportsPrimitiveIdStream();
	
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bSupportsPrimitiveIdStream && bUseGPUScene);
	OutEnvironment.SetDefine(TEXT("VF_REQUIRES_PER_INSTANCE_CUSTOM_DATA"), bSupportsPrimitiveIdStream && bUseGPUScene);

	// Mobile GPUScene implementation relies on USE_INSTANCE_CULLING define for rendering instanced meshes
	if (bUseGPUScene && MaxSupportedFeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 1);
	}
#endif

	if (RHISupportsManualVertexFetch(Parameters.Platform))
	{
		OutEnvironment.SetDefineIfUnset(TEXT("MANUAL_VERTEX_FETCH"), TEXT("1"));
	}
}

void FNiagaraMeshVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	uint8 StreamIndex = 0;
	Elements.Add(FVertexElement(StreamIndex++, 0, VET_Float3, 0, 0, false));

	if (!RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		// Normals
		Elements.Add(FVertexElement(StreamIndex, 0, VET_PackedNormal, 1, 0, false));
		Elements.Add(FVertexElement(StreamIndex++, 4, VET_PackedNormal, 2, 0, false));

		// Color
		Elements.Add(FVertexElement(StreamIndex++, 0, VET_Color, 3, 0, false));

		// TexCoords
		int8 UVStreamIndex = StreamIndex;
		Elements.Add(FVertexElement(StreamIndex, 0, VET_Half4, 4, 0, false));
		Elements.Add(FVertexElement(StreamIndex, 8, VET_Half4, 5, 0, false));
		Elements.Add(FVertexElement(StreamIndex, 16, VET_Half4, 6, 0, false));
		Elements.Add(FVertexElement(StreamIndex++, 24, VET_Half4, 7, 0, false));
	}
	
#if NIAGARA_ENABLE_GPU_SCENE_MESHES
	// Primitive ID stream
	if (UseGPUScene(GMaxRHIShaderPlatform)
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		Elements.Add(FVertexElement(StreamIndex, 0, VET_UInt, 13, 0, true));
	}
#endif
}

void FNiagaraMeshVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, bool bSupportsManualVertexFetch, FStaticMeshDataType& Data, FVertexDeclarationElementList& Elements)
{
	FVertexStreamList InOutStreams;
	GetVertexElements(FeatureLevel, bSupportsManualVertexFetch, Data, Elements, InOutStreams);

	if (IsGPUSceneEnabled(GMaxRHIShaderPlatform, GMaxRHIFeatureLevel) && !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		Elements.Add(FVertexElement(InOutStreams.Num(), 0, VET_UInt, 13, sizeof(uint32), true));
	}
}

void FNiagaraMeshVertexFactory::SetData(FRHICommandListBase& RHICmdList, const FStaticMeshDataType& InData)
{
	Data = InData;
	UpdateRHI(RHICmdList);
}

#if NIAGARA_ENABLE_GPU_SCENE_MESHES
	#define NIAGARA_MESH_VF_FLAGS (EVertexFactoryFlags::UsedWithMaterials \
		| EVertexFactoryFlags::SupportsDynamicLighting \
		| EVertexFactoryFlags::SupportsRayTracing \
		| EVertexFactoryFlags::SupportsPrimitiveIdStream \
		| EVertexFactoryFlags::SupportsManualVertexFetch \
		| EVertexFactoryFlags::SupportsPSOPrecaching)
#else
	#define NIAGARA_MESH_VF_FLAGS (EVertexFactoryFlags::UsedWithMaterials \
		| EVertexFactoryFlags::SupportsDynamicLighting \
		| EVertexFactoryFlags::SupportsRayTracing \
		| EVertexFactoryFlags::SupportsManualVertexFetch \
		| EVertexFactoryFlags::SupportsPSOPrecaching)
#endif
#define NIAGARA_MESH_VF_FLAGS_EX (NIAGARA_MESH_VF_FLAGS | EVertexFactoryFlags::SupportsPrecisePrevWorldPos)

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraMeshVertexFactory, SF_Vertex, FNiagaraMeshVertexFactoryShaderParametersVS);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraMeshVertexFactory, SF_Compute, FNiagaraMeshVertexFactoryShaderParametersVS);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraMeshVertexFactory, SF_RayHitGroup, FNiagaraMeshVertexFactoryShaderParametersVS);
#endif
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraMeshVertexFactory, SF_Pixel, FNiagaraMeshVertexFactoryShaderParametersPS);

IMPLEMENT_VERTEX_FACTORY_TYPE(FNiagaraMeshVertexFactory, "/Plugin/FX/Niagara/Private/NiagaraMeshVertexFactory.ush",	NIAGARA_MESH_VF_FLAGS);
