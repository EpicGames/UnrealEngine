// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.cpp: Particle vertex factory implementation.
=============================================================================*/

#include "NiagaraRibbonVertexFactory.h"
#include "ParticleHelper.h"
#include "ParticleResources.h"
#include "ShaderParameterUtils.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "PipelineStateCache.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraRibbonUniformParameters, "NiagaraRibbonVF");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraRibbonVFLooseParameters, "NiagaraRibbonVFLooseParameters");

/**
* Shader parameters for the beam/trail vertex factory.
*/
class FNiagaraRibbonVertexFactoryShaderParametersVS : public FNiagaraVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FNiagaraRibbonVertexFactoryShaderParametersVS, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		FNiagaraVertexFactoryShaderParametersBase::Bind(ParameterMap);

	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);

		const FNiagaraRibbonVertexFactory* RibbonVF = static_cast<const FNiagaraRibbonVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraRibbonUniformParameters>(), RibbonVF->GetRibbonUniformBuffer());
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraRibbonVFLooseParameters>(), RibbonVF->LooseParameterUniformBuffer);
	}

private:
};

IMPLEMENT_TYPE_LAYOUT(FNiagaraRibbonVertexFactoryShaderParametersVS);


/**
* Shader parameters for the beam/trail vertex factory.
*/
class FNiagaraRibbonVertexFactoryShaderParametersPS : public FNiagaraVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FNiagaraRibbonVertexFactoryShaderParametersPS, NonVirtual);
public:
	 void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);

		const FNiagaraRibbonVertexFactory* RibbonVF = static_cast<const FNiagaraRibbonVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraRibbonUniformParameters>(), RibbonVF->GetRibbonUniformBuffer());
	}
};

IMPLEMENT_TYPE_LAYOUT(FNiagaraRibbonVertexFactoryShaderParametersPS);


///////////////////////////////////////////////////////////////////////////////
/**
* The Niagara ribbon vertex declaration resource type.
*/
class FNiagaraRibbonVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	// Destructor.
	virtual ~FNiagaraRibbonVertexDeclaration() {}

	virtual void FillDeclElements(FVertexDeclarationElementList& Elements, int32& Offset)
	{
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		FVertexDeclarationElementList Elements;
		int32	Offset = 0;
		FillDeclElements(Elements, Offset);

		// Create the vertex declaration for rendering the factory normally.
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI()
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

/** The simple element vertex declaration. */
static TGlobalResource<FNiagaraRibbonVertexDeclaration> GNiagaraRibbonVertexDeclaration;

///////////////////////////////////////////////////////////////////////////////

bool FNiagaraRibbonVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return (FNiagaraUtilities::SupportsNiagaraRendering(Parameters.Platform)) && (Parameters.MaterialParameters.bIsUsedWithNiagaraRibbons || Parameters.MaterialParameters.bIsSpecialEngineMaterial);
}

/**
* Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
*/
void FNiagaraRibbonVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FNiagaraVertexFactoryBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("NiagaraVFLooseParameters"), TEXT("NiagaraRibbonVFLooseParameters"));
	
	OutEnvironment.SetDefine(TEXT("NIAGARA_RIBBON_FACTORY"), TEXT("1"));

	// Ribbons are generated in world space and never have a matrix transform in raytracing, so it is safe to leave them in world space.
	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_WORLD_SPACE"), TEXT("1"));
}

/**
* Get vertex elements used when during PSO precaching materials using this vertex factory type
*/
void FNiagaraRibbonVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	GNiagaraRibbonVertexDeclaration.VertexDeclarationRHI->GetInitializer(Elements);
}

/**
*	Initialize the Render Hardware Interface for this vertex factory
*/
void FNiagaraRibbonVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	SetDeclaration(GNiagaraRibbonVertexDeclaration.VertexDeclarationRHI);

	FVertexStream* VertexStream = new(Streams) FVertexStream;
	FVertexStream* DynamicParameterStream = new(Streams) FVertexStream;
	FVertexStream* DynamicParameter1Stream = new(Streams) FVertexStream;
	FVertexStream* DynamicParameter2Stream = new(Streams) FVertexStream;
	FVertexStream* DynamicParameter3Stream = new(Streams) FVertexStream;
}

void FNiagaraRibbonVertexFactory::SetVertexBuffer(const FVertexBuffer* InBuffer, uint32 StreamOffset, uint32 Stride)
{
	check(Streams.Num() == 5);
	FVertexStream& VertexStream = Streams[0];
	VertexStream.VertexBuffer = InBuffer;
	VertexStream.Stride = Stride;
	VertexStream.Offset = StreamOffset;
}

void FNiagaraRibbonVertexFactory::SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, int32 ParameterIndex, uint32 StreamOffset, uint32 Stride)
{
	check(Streams.Num() == 5);
	FVertexStream& DynamicParameterStream = Streams[1 + ParameterIndex];
	if (InDynamicParameterBuffer)
	{
		DynamicParameterStream.VertexBuffer = InDynamicParameterBuffer;
		DynamicParameterStream.Stride = Stride;
		DynamicParameterStream.Offset = StreamOffset;
	}
	else
	{
		DynamicParameterStream.VertexBuffer = &GNullDynamicParameterVertexBuffer;
		DynamicParameterStream.Stride = 0;
		DynamicParameterStream.Offset = 0;
	}
}

///////////////////////////////////////////////////////////////////////////////

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraRibbonVertexFactory, SF_Vertex, FNiagaraRibbonVertexFactoryShaderParametersVS);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraRibbonVertexFactory, SF_Compute, FNiagaraRibbonVertexFactoryShaderParametersVS);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraRibbonVertexFactory, SF_RayHitGroup, FNiagaraRibbonVertexFactoryShaderParametersVS);
#endif
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraRibbonVertexFactory, SF_Pixel, FNiagaraRibbonVertexFactoryShaderParametersPS);
IMPLEMENT_VERTEX_FACTORY_TYPE(FNiagaraRibbonVertexFactory, "/Plugin/FX/Niagara/Private/NiagaraRibbonVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials 
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsPSOPrecaching
);
