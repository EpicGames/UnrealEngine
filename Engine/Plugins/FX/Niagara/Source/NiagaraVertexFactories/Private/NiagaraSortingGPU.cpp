// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraSortingGPU.cpp: Niagara sorting shaders
==============================================================================*/

#include "NiagaraSortingGPU.h"
#include "NiagaraGPUSortInfo.h"
#include "ShaderParameterUtils.h"
#include "ShaderCompilerCore.h"
#include "DataDrivenShaderPlatformInfo.h"

int32 GNiagaraGPUSortingCPUToGPUThreshold = -1;
static FAutoConsoleVariableRef CVarNiagaraGPUSortingCPUToGPUThreshold(
	TEXT("Niagara.GPUSorting.CPUToGPUThreshold"),
	GNiagaraGPUSortingCPUToGPUThreshold,
	TEXT("Particle count to move from a CPU sort to a GPU sort. -1 disables. (default=-1)"),
	ECVF_Default
);

int32 GNiagaraGPUCullingCPUToGPUThreshold = 0;
static FAutoConsoleVariableRef CVarNiagaraGPUCullingCPUToGPUThreshold(
	TEXT("Niagara.GPUCulling.CPUToGPUThreshold"),
	GNiagaraGPUCullingCPUToGPUThreshold,
	TEXT("Particle count to move from a CPU sort to a GPU cull. -1 disables. (default=0)"),
	ECVF_Default
);

IMPLEMENT_GLOBAL_SHADER(FNiagaraSortKeyGenCS, "/Plugin/FX/Niagara/Private/NiagaraSortKeyGen.usf", "GenerateParticleSortKeys", SF_Compute);

bool FNiagaraSortKeyGenCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	FPermutationDomain PermutationVector(Parameters.PermutationId);

	ERHIFeatureSupport WaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Parameters.Platform);
	if (PermutationVector.Get<FUseWaveOps>())
	{
		if ( WaveOpsSupport == ERHIFeatureSupport::Unsupported )
		{
			return false;
		}
	}
	else
	{
		if (WaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed)
		{
			return false;
		}
	}

	return true;
}

void FNiagaraSortKeyGenCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FPermutationDomain PermutationVector(Parameters.PermutationId);
	const bool bUseWaveIntrinsics = PermutationVector.Get<FUseWaveOps>();

	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_KEY_GEN_THREAD_COUNT);
	OutEnvironment.SetDefine(TEXT("SORT_NONE"), (uint8)ENiagaraSortMode::None);
	OutEnvironment.SetDefine(TEXT("SORT_VIEW_DEPTH"), (uint8)ENiagaraSortMode::ViewDepth);
	OutEnvironment.SetDefine(TEXT("SORT_VIEW_DISTANCE"), (uint8)ENiagaraSortMode::ViewDistance);
	OutEnvironment.SetDefine(TEXT("SORT_CUSTOM_ASCENDING"), (uint8)ENiagaraSortMode::CustomAscending);
	OutEnvironment.SetDefine(TEXT("SORT_CUSTOM_DESCENDING"), (uint8)ENiagaraSortMode::CustomDecending);
	OutEnvironment.SetDefine(TEXT("NIAGARA_KEY_GEN_MAX_CULL_PLANES"), NIAGARA_KEY_GEN_MAX_CULL_PLANES);
	OutEnvironment.SetDefine(TEXT("USE_WAVE_INTRINSICS"), bUseWaveIntrinsics);
	if (bUseWaveIntrinsics)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
	}
}

bool FNiagaraSortKeyGenCS::UseWaveOps(EShaderPlatform ShaderPlatform)
{
	const ERHIFeatureSupport UseWaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(ShaderPlatform);
	return (GRHISupportsWaveOperations && UseWaveOpsSupport == ERHIFeatureSupport::RuntimeDependent) || (UseWaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed);
}
