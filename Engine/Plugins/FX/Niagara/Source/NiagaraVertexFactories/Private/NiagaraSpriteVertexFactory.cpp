// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.cpp: Particle vertex factory implementation.
=============================================================================*/

#include "NiagaraSpriteVertexFactory.h"
#include "NiagaraCutoutVertexBuffer.h"
#include "ParticleHelper.h"
#include "ParticleResources.h"
#include "ShaderParameterUtils.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "PipelineStateCache.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraSpriteUniformParameters,"NiagaraSpriteVF");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraSpriteVFLooseParameters, "NiagaraSpriteVFLooseParameters");

class FNiagaraSpriteVertexFactoryShaderParametersVS : public FNiagaraVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FNiagaraSpriteVertexFactoryShaderParametersVS, NonVirtual);
public:

	void Bind(const FShaderParameterMap& ParameterMap)
	{
		FNiagaraVertexFactoryShaderParametersBase::Bind(ParameterMap);
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType VertexStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(Scene, View, Shader, VertexStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);

		const FNiagaraSpriteVertexFactory* SpriteVF = static_cast<const FNiagaraSpriteVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraSpriteUniformParameters>(), SpriteVF->GetSpriteUniformBuffer());
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraSpriteVFLooseParameters>(), SpriteVF->GetLooseParameterUniformBuffer());
	}
};

IMPLEMENT_TYPE_LAYOUT(FNiagaraSpriteVertexFactoryShaderParametersVS);

class FNiagaraSpriteVertexFactoryShaderParametersPS : public FNiagaraVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FNiagaraSpriteVertexFactoryShaderParametersPS, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);

		const FNiagaraSpriteVertexFactory* SpriteVF = static_cast<const FNiagaraSpriteVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FNiagaraSpriteUniformParameters>(), SpriteVF->GetSpriteUniformBuffer() );
	}
};

IMPLEMENT_TYPE_LAYOUT(FNiagaraSpriteVertexFactoryShaderParametersPS);

/**
 * The particle system vertex declaration resource type.
 */
class FNiagaraSpriteVertexDeclaration : public FRenderResource
{
public:

	FVertexDeclarationRHIRef VertexDeclarationRHI;

	// Constructor.
	FNiagaraSpriteVertexDeclaration() {}

	// Destructor.
	virtual ~FNiagaraSpriteVertexDeclaration() {}

	virtual void FillDeclElements(FVertexDeclarationElementList& Elements, int32& Offset)
	{
		uint32 InitialStride = sizeof(float) * 2;
		/** The stream to read the texture coordinates from. */
		check( Offset == 0 );
		Elements.Add(FVertexElement(0, Offset, VET_Float2, 0, InitialStride, false));
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		FVertexDeclarationElementList Elements;
		int32	Offset = 0;

		FillDeclElements(Elements, Offset);

		// Create the vertex declaration for rendering the factory normally
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI()
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

/** The simple element vertex declaration. */
static TGlobalResource<FNiagaraSpriteVertexDeclaration> GParticleSpriteVertexDeclaration;

bool FNiagaraSpriteVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return (FNiagaraUtilities::SupportsNiagaraRendering(Parameters.Platform)) && (Parameters.MaterialParameters.bIsUsedWithNiagaraSprites || Parameters.MaterialParameters.bIsSpecialEngineMaterial);
}

/**
 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
 */
void FNiagaraSpriteVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FNiagaraVertexFactoryBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("NiagaraVFLooseParameters"),TEXT("NiagaraSpriteVFLooseParameters"));

	// Set a define so we can tell in MaterialTemplate.usf when we are compiling a sprite vertex factory
	OutEnvironment.SetDefine(TEXT("PARTICLE_SPRITE_FACTORY"),TEXT("1"));

	// Sprites are generated in world space and never have a matrix transform in raytracing, so it is safe to leave them in world space.
	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_WORLD_SPACE"), TEXT("1"));
}

/**
* Get vertex elements used when during PSO precaching materials using this vertex factory type
*/
void FNiagaraSpriteVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	GParticleSpriteVertexDeclaration.VertexDeclarationRHI->GetInitializer(Elements);
}

/**
 *	Initialize the Render Hardware Interface for this vertex factory
 */
void FNiagaraSpriteVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	InitStreams();
	SetDeclaration(GParticleSpriteVertexDeclaration.VertexDeclarationRHI);
}

void FNiagaraSpriteVertexFactory::InitStreams()
{
	check(Streams.Num() == 0);
	FVertexStream* TexCoordStream = new(Streams) FVertexStream;
	TexCoordStream->VertexBuffer = VertexBufferOverride ? VertexBufferOverride : &GParticleTexCoordVertexBuffer;
	TexCoordStream->Stride = sizeof(FVector2f);
	TexCoordStream->Offset = 0;
}

void FNiagaraSpriteVertexFactory::SetTexCoordBuffer(const FVertexBuffer* InTexCoordBuffer)
{
	FVertexStream& TexCoordStream = Streams[0];
	TexCoordStream.VertexBuffer = InTexCoordBuffer;
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraSpriteVertexFactory, SF_Vertex, FNiagaraSpriteVertexFactoryShaderParametersVS);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraSpriteVertexFactory, SF_Pixel, FNiagaraSpriteVertexFactoryShaderParametersPS);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraSpriteVertexFactory, SF_Compute, FNiagaraSpriteVertexFactoryShaderParametersVS);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FNiagaraSpriteVertexFactory, SF_RayHitGroup, FNiagaraSpriteVertexFactoryShaderParametersVS);
#endif // RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_TYPE(FNiagaraSpriteVertexFactory,"/Plugin/FX/Niagara/Private/NiagaraSpriteVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsPSOPrecaching
);
