// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVertexFactoriesModule.h"
#include "Modules/ModuleManager.h"
#include "NiagaraVertexFactory.h"

IMPLEMENT_MODULE(INiagaraVertexFactoriesModule, NiagaraVertexFactories);


TGlobalResource<FNiagaraNullSortedIndicesVertexBuffer> GFNiagaraNullSortedIndicesVertexBuffer;