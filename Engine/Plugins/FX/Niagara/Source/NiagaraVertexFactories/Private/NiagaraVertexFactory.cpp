// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVertexFactory.h"
#include "ShaderParameterUtils.h"
#include "MeshMaterialShader.h"

IMPLEMENT_TYPE_LAYOUT(FNiagaraVertexFactoryShaderParametersBase);

void FNiagaraVertexFactoryShaderParametersBase::Bind(const FShaderParameterMap& ParameterMap)
{

}

void FNiagaraVertexFactoryShaderParametersBase::GetElementShaderBindings(
	const FSceneInterface* Scene,
	const FSceneView* View,
	const FMeshMaterialShader* Shader,
	const EVertexInputStreamType VertexStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams) const
{
	const FNiagaraVertexFactoryBase* NiagaraVF = static_cast<const FNiagaraVertexFactoryBase*>(VertexFactory);
}
