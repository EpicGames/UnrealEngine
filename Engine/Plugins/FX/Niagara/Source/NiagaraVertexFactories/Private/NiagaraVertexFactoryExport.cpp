// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVertexFactoryExport.h"
#include "SceneTexturesConfig.h"

static TAutoConsoleVariable<int32> CVarNiagarVertexFactoryExportEnabledMode(
	TEXT("fx.Niagara.VertexFactoryExport.EnabledMode"),
	1,
	TEXT("Determins compilation mode for vertex factory export shader permutations\n")
	TEXT("0 - Disabled on all platforms / targets\n")
	TEXT("1 - Enabled for editor only (default)\n")
	TEXT("2 - Enabled on all platforms / targets"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

FNiagaraVertexFactoryExportCS::FNiagaraVertexFactoryExportCS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
	: FMeshMaterialShader(Initializer)
{
	PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());

	IsIndirectDraw.Bind(Initializer.ParameterMap, TEXT("IsIndirectDraw"));
	NumInstances.Bind(Initializer.ParameterMap, TEXT("NumInstances"));
	NumVerticesPerInstance.Bind(Initializer.ParameterMap, TEXT("NumVerticesPerInstance"));
	bApplyWPO.Bind(Initializer.ParameterMap, TEXT("bApplyWPO"));
	bWorldSpace.Bind(Initializer.ParameterMap, TEXT("bWorldSpace"));
	SectionInfoOutputOffset.Bind(Initializer.ParameterMap, TEXT("SectionInfoOutputOffset"));

	VertexStride.Bind(Initializer.ParameterMap, TEXT("VertexStride"));
	VertexPositionOffset.Bind(Initializer.ParameterMap, TEXT("VertexPositionOffset"));
	VertexColorOffset.Bind(Initializer.ParameterMap, TEXT("VertexColorOffset"));
	VertexTangentBasisOffset.Bind(Initializer.ParameterMap, TEXT("VertexTangentBasisOffset"));
	VertexTexCoordOffset.Bind(Initializer.ParameterMap, TEXT("VertexTexCoordOffset"));
	VertexTexCoordNum.Bind(Initializer.ParameterMap, TEXT("VertexTexCoordNum"));
	VertexTexCoordBindings.Bind(Initializer.ParameterMap, TEXT("VertexTexCoordBindings"));
	VertexOutputOffset.Bind(Initializer.ParameterMap, TEXT("VertexOutputOffset"));
	RWVertexData.Bind(Initializer.ParameterMap, TEXT("RWVertexData"));
}

bool FNiagaraVertexFactoryExportCS::SupportsVertexFactoryType(const FVertexFactoryType* VertexFactoryType)
{
	return
		//VertexFactoryType == FindVertexFactoryType(TEXT("FNiagaraMeshVertexFactory")) ||
		VertexFactoryType == FindVertexFactoryType(TEXT("FNiagaraRibbonVertexFactory")) ||
		VertexFactoryType == FindVertexFactoryType(TEXT("FNiagaraSpriteVertexFactory"));
}

bool FNiagaraVertexFactoryExportCS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const int32 EnabledMode = CVarNiagarVertexFactoryExportEnabledMode.GetValueOnAnyThread();
	const bool bEnabled =
		(EnabledMode == 2) ||
		(EnabledMode == 1 && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData));

	return bEnabled && SupportsVertexFactoryType(Parameters.VertexFactoryType);
}

void FNiagaraVertexFactoryExportCS::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), ThreadGroupSize);
	OutEnvironment.SetDefine(TEXT("WITH_NIAGARA_VERTEX_FACTORY_EXPORT"), 1);
}

IMPLEMENT_MATERIAL_SHADER_TYPE(, FNiagaraVertexFactoryExportCS, TEXT("/Plugin/FX/Niagara/Private/NiagaraVertexFactoryExport.usf"), TEXT("VertexFactoryExportCS"), SF_Compute);
