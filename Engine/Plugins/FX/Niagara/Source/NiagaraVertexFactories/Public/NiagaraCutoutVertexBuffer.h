// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraCutoutVertexBuffer.h: Niagara cutout uv buffer.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"

/**
 * Vertex buffer to hold cutout UVs. 
 */
class FNiagaraCutoutVertexBuffer : public FVertexBuffer
{
public:

	NIAGARAVERTEXFACTORIES_API FNiagaraCutoutVertexBuffer(int32 ZeroInitCount = 0);

	/**
	 * Initialize the RHI for this rendering resource
	 */
	NIAGARAVERTEXFACTORIES_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	NIAGARAVERTEXFACTORIES_API virtual void ReleaseRHI() override;
	FShaderResourceViewRHIRef VertexBufferSRV;

	/** Data to initialize the buffer. */
	TArray<FVector2f> Data;
};

extern NIAGARAVERTEXFACTORIES_API TGlobalResource<FNiagaraCutoutVertexBuffer> GFNiagaraNullCutoutVertexBuffer;
