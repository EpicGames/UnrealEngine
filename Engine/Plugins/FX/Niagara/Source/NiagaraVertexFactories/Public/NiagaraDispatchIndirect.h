// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderPermutation.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"

struct FNiagaraDispatchIndirectInfoCS
{
	FNiagaraDispatchIndirectInfoCS() = default;
	explicit FNiagaraDispatchIndirectInfoCS(const FUintVector& InCounterOffsets, const FIntVector& InThreadGroupSize, uint32 InIndirectArgsOffset)
		: CounterOffsets(InCounterOffsets)
		, ThreadGroupSize(InThreadGroupSize)
		, IndirectArgsOffset(InIndirectArgsOffset)
	{
	}

	FUintVector		CounterOffsets = FUintVector(INDEX_NONE, INDEX_NONE, INDEX_NONE);
	FIntVector		ThreadGroupSize = FIntVector::ZeroValue;
	uint32			IndirectArgsOffset = 0;
};

struct FNiagaraDispatchIndirectParametersCS
{
	FUintVector4	GroupCount = FUintVector4(0, 0, 0, 0);
	FUintVector4	ThreadCount = FUintVector4(0, 0, 0, 0);
};

class FNiagaraDispatchIndirectArgsGenCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraDispatchIndirectArgsGenCS, NIAGARAVERTEXFACTORIES_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraDispatchIndirectArgsGenCS, FGlobalShader);

public:
	static constexpr int ThreadCount = 32;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARAVERTEXFACTORIES_API)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,		DispatchInfos)
		SHADER_PARAMETER(uint32,							NumDispatchInfos)
		SHADER_PARAMETER(FUintVector3,						MaxGroupsPerDimension)

		SHADER_PARAMETER_SRV(Buffer<uint>,					InstanceCounts)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint4>,	RWDispatchIndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static NIAGARAVERTEXFACTORIES_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};
