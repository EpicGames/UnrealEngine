// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraDrawIndirect.h: Niagara shader to generate the draw indirect args for Niagara renderers.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderPermutation.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"

#define NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT 64
#define NIAGARA_DRAW_INDIRECT_ARGS_SIZE 5
#define NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE 5
#define NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE 2

// #define NIAGARA_COPY_BUFFER_THREAD_COUNT 64
// #define NIAGARA_COPY_BUFFER_BUFFER_COUNT 3

enum class ENiagaraDrawIndirectArgGenTaskFlags : uint32
{
	None = 0,
	UseCulledCounts = 1 << 0,
	InstancedStereo = 1 << 1,
	PostOpaque = 1 << 2,
};
ENUM_CLASS_FLAGS(ENiagaraDrawIndirectArgGenTaskFlags);

/**
* Task info when generating draw indirect frame buffer.
* Task is either about generate Niagara renderers drawindirect buffer,
* or about resetting released instance counters.
*/
struct FNiagaraDrawIndirectArgGenTaskInfo
{
	explicit FNiagaraDrawIndirectArgGenTaskInfo(uint32 InInstanceCountBufferOffset, uint32 InNumIndicesPerInstance, uint32 InStartIndexLocation, ENiagaraDrawIndirectArgGenTaskFlags InFlags)
		: IndirectArgsBufferOffset(INDEX_NONE)
		, InstanceCountBufferOffset(InInstanceCountBufferOffset)
		, NumIndicesPerInstance(InNumIndicesPerInstance)
		, StartIndexLocation(InStartIndexLocation)
		, Flags((uint32)InFlags)
	{
	}

	uint32 IndirectArgsBufferOffset;	// Ignored in hash / comparison since we can de-duplicate the rest
	uint32 InstanceCountBufferOffset;
	uint32 NumIndicesPerInstance; // When -1 the counter needs to be reset to 0.
	uint32 StartIndexLocation;
	uint32 Flags;

	bool operator==(const FNiagaraDrawIndirectArgGenTaskInfo& Rhs) const
	{
		return InstanceCountBufferOffset == Rhs.InstanceCountBufferOffset
			&& NumIndicesPerInstance == Rhs.NumIndicesPerInstance
			&& StartIndexLocation == Rhs.StartIndexLocation
			&& Flags == Rhs.Flags;
	}
};

/**
 * Compute shader used to generate GPU emitter draw indirect args.
 * It also resets unused instance count entries.
 */
class FNiagaraDrawIndirectArgsGenCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraDrawIndirectArgsGenCS, NIAGARAVERTEXFACTORIES_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraDrawIndirectArgsGenCS, FGlobalShader);

public:
	class FSupportsTextureRW : SHADER_PERMUTATION_INT("SUPPORTS_TEXTURE_RW", 2);
	using FPermutationDomain = TShaderPermutationDomain<FSupportsTextureRW>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARAVERTEXFACTORIES_API)
		SHADER_PARAMETER_SRV(Buffer<uint>,		TaskInfos)
		SHADER_PARAMETER_SRV(Buffer<uint>,		CulledInstanceCounts)

		SHADER_PARAMETER_UAV(RWBuffer<uint>,	RWInstanceCounts)
		SHADER_PARAMETER_UAV(RWBuffer<uint>,	RWDrawIndirectArgs)

		SHADER_PARAMETER(FUintVector4,			TaskCount)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
 * Compute shader used to reset unused instance count entries. Used if the platform doesn't support RW texture buffers
 */
class FNiagaraDrawIndirectResetCountsCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraDrawIndirectResetCountsCS, NIAGARAVERTEXFACTORIES_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraDrawIndirectResetCountsCS, FGlobalShader);

public:
	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARAVERTEXFACTORIES_API)
		SHADER_PARAMETER_SRV(Buffer<uint>,		TaskInfos)
		SHADER_PARAMETER_UAV(RWBuffer<uint>,	RWInstanceCounts)
		SHADER_PARAMETER(FUintVector4,			TaskCount)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
 * Compute shader used to initialize instance count entries to specific values. Used if the platform doesn't support RW texture buffers
 */
class FNiagaraInstanceCountsInitCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraInstanceCountsInitCS, NIAGARAVERTEXFACTORIES_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraInstanceCountsInitCS, FGlobalShader);

public:
	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARAVERTEXFACTORIES_API)
		SHADER_PARAMETER_SRV(Buffer<uint>,		TaskInfos)
		SHADER_PARAMETER_UAV(RWBuffer<uint>,	RWInstanceCounts)
		SHADER_PARAMETER(FUintVector4,			TaskCount)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};
