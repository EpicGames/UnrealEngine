// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraGPURayTracingTransformsShader.h: Niagara shader to generate the ray tracing instances information.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderPermutation.h"
#include "ShaderParameterStruct.h"

/**
 * Compute shader used to pass GPU instances transforms to the ray tracing TLAS.
 */
class FNiagaraGPURayTracingTransformsCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraGPURayTracingTransformsCS, NIAGARAVERTEXFACTORIES_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraGPURayTracingTransformsCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 64;

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32,	ParticleDataFloatStride)
		//SHADER_PARAMETER(uint32,	ParticleDataHalfStride)
		SHADER_PARAMETER(uint32,	ParticleDataIntStride)
		SHADER_PARAMETER(uint32,	CPUNumInstances)
		SHADER_PARAMETER(uint32,	InstanceCountOffset)

		SHADER_PARAMETER(FVector3f,	SystemLWCTile)

		SHADER_PARAMETER(uint32,	PositionDataOffset)
		SHADER_PARAMETER(uint32,	RotationDataOffset)
		SHADER_PARAMETER(uint32,	ScaleDataOffset)
		SHADER_PARAMETER(uint32,	bLocalSpace)
		SHADER_PARAMETER(uint32,	RenderVisibilityOffset)
		SHADER_PARAMETER(uint32,	MeshIndexOffset)

		SHADER_PARAMETER(uint32,	RenderVisibilityValue)
		SHADER_PARAMETER(uint32,	MeshIndexValue)

		SHADER_PARAMETER(FVector3f,	DefaultPosition)
		SHADER_PARAMETER(FVector4f,	DefaultRotation)
		SHADER_PARAMETER(FVector3f,	DefaultScale)
		SHADER_PARAMETER(FVector3f,	MeshScale)
		SHADER_PARAMETER(FVector4f, MeshRotation)

		SHADER_PARAMETER(FMatrix44f,	LocalTransform)
		SHADER_PARAMETER(FVector3f,		LocalTransformTile)

		SHADER_PARAMETER_SRV(Buffer<float>,	ParticleDataFloatBuffer)
		//SHADER_PARAMETER_SRV(Buffer<half>,	ParticleDataHalfBuffer)
		SHADER_PARAMETER_SRV(Buffer<int>,	ParticleDataIntBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>, GPUInstanceCountBuffer)

		SHADER_PARAMETER(FVector3f,	ViewTilePosition)
		SHADER_PARAMETER(FVector3f, RelativePreViewTranslation)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float3x4>, TLASTransforms)
	END_SHADER_PARAMETER_STRUCT()

};
