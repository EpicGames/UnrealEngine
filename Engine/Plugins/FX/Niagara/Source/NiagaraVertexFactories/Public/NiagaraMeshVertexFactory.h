// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ParticleVertexFactory.h: Particle vertex factory definitions.
=============================================================================*/

#pragma once

#include "Components.h"
#include "CoreMinimal.h"
#include "NiagaraDataSet.h"
#include "NiagaraVertexFactory.h"
#include "RenderResource.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"

class FMaterial;
class FVertexBuffer;
struct FDynamicReadBuffer;
struct FShaderCompilerEnvironment;

/**
 * Common shader parameters for mesh particle renderers (used by multiple shaders)
 */
BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraMeshCommonParameters, NIAGARAVERTEXFACTORIES_API)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataFloat)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataHalf)
	SHADER_PARAMETER_SRV(Buffer<int>, NiagaraParticleDataInt)
	SHADER_PARAMETER(uint32, NiagaraFloatDataStride)
	SHADER_PARAMETER(uint32, NiagaraIntDataStride)

	SHADER_PARAMETER_SRV(Buffer<uint>, SortedIndices)
	SHADER_PARAMETER(int, SortedIndicesOffset)
	
	SHADER_PARAMETER(FVector3f, SystemLWCTile)
	SHADER_PARAMETER(int, bLocalSpace)
	SHADER_PARAMETER(int, AccurateMotionVectors)
	SHADER_PARAMETER(float, DeltaSeconds)
	SHADER_PARAMETER(uint32, FacingMode)

	SHADER_PARAMETER(FVector3f, MeshScale)
	SHADER_PARAMETER(FVector3f, MeshOffset)
	SHADER_PARAMETER(FQuat4f, MeshRotation)
	SHADER_PARAMETER(int, bMeshOffsetIsWorldSpace)

	SHADER_PARAMETER(uint32, bLockedAxisEnable)
	SHADER_PARAMETER(FVector3f, LockedAxis)
	SHADER_PARAMETER(uint32, LockedAxisSpace)
	
	SHADER_PARAMETER(int, ScaleDataOffset)
	SHADER_PARAMETER(int, RotationDataOffset)
	SHADER_PARAMETER(int, PositionDataOffset)
	SHADER_PARAMETER(int, VelocityDataOffset)
	SHADER_PARAMETER(int, CameraOffsetDataOffset)
	SHADER_PARAMETER(int, PrevScaleDataOffset)
	SHADER_PARAMETER(int, PrevRotationDataOffset)
	SHADER_PARAMETER(int, PrevPositionDataOffset)
	SHADER_PARAMETER(int, PrevVelocityDataOffset)
	SHADER_PARAMETER(int, PrevCameraOffsetDataOffset)

	SHADER_PARAMETER(FVector3f, DefaultScale)
	SHADER_PARAMETER(FQuat4f, DefaultRotation)
	SHADER_PARAMETER(FVector3f, DefaultPosition)
	SHADER_PARAMETER(FVector3f, DefaultVelocity)
	SHADER_PARAMETER(float, DefaultCameraOffset)
	SHADER_PARAMETER(FVector3f, DefaultPrevScale)
	SHADER_PARAMETER(FQuat4f, DefaultPrevRotation)
	SHADER_PARAMETER(FVector3f, DefaultPrevPosition)
	SHADER_PARAMETER(FVector3f, DefaultPrevVelocity)
	SHADER_PARAMETER(float, DefaultPrevCameraOffset)
END_SHADER_PARAMETER_STRUCT()

/**
* Uniform buffer for mesh particle vertex factories.
*/
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraMeshUniformParameters, NIAGARAVERTEXFACTORIES_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraMeshCommonParameters, Common)

	SHADER_PARAMETER_SRV(Buffer<float2>, VertexFetch_TexCoordBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_PackedTangentsBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_ColorComponentsBuffer)
	SHADER_PARAMETER(FIntVector4, VertexFetch_Parameters)

	SHADER_PARAMETER(FVector4f, SubImageSize)
	SHADER_PARAMETER(uint32, TexCoordWeightA)
	SHADER_PARAMETER(uint32, TexCoordWeightB)
	SHADER_PARAMETER(uint32, MaterialParamValidMask)

	SHADER_PARAMETER(int, NormalizedAgeDataOffset)
	SHADER_PARAMETER(int, SubImageDataOffset)
	SHADER_PARAMETER(int, MaterialRandomDataOffset)
	SHADER_PARAMETER(int, ColorDataOffset)
	SHADER_PARAMETER(int, MaterialParamDataOffset)
	SHADER_PARAMETER(int, MaterialParam1DataOffset)
	SHADER_PARAMETER(int, MaterialParam2DataOffset)
	SHADER_PARAMETER(int, MaterialParam3DataOffset)

	SHADER_PARAMETER(float, DefaultNormAge)
	SHADER_PARAMETER(float, DefaultSubImage)
	SHADER_PARAMETER(float, DefaultMatRandom)
	SHADER_PARAMETER(FVector4f, DefaultColor)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter0)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter1)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter2)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter3)

	SHADER_PARAMETER(int, SubImageBlendMode)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FNiagaraMeshUniformParameters> FNiagaraMeshUniformBufferRef;

class FNiagaraMeshInstanceVertices;


/**
* Vertex factory for rendering instanced mesh particles with out dynamic parameter support.
*/
class FNiagaraMeshVertexFactory : public FNiagaraVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FNiagaraMeshVertexFactory, NIAGARAVERTEXFACTORIES_API);
public:
	
	/** Default constructor. */
	FNiagaraMeshVertexFactory(ENiagaraVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel)
		: FNiagaraVertexFactoryBase(InType, InFeatureLevel)
		, bAddPrimitiveIDElement(true)
	{}

	FNiagaraMeshVertexFactory()
		: FNiagaraVertexFactoryBase(NVFT_MAX, ERHIFeatureLevel::Num)
		, bAddPrimitiveIDElement(true)
	{}

	/**
	* Should we cache the material's shadertype on this platform with this vertex factory?
	*/
	static NIAGARAVERTEXFACTORIES_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);


	/**
	* Modify compile environment to enable instancing
	* @param OutEnvironment - shader compile environment to modify
	*/
	static NIAGARAVERTEXFACTORIES_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	* Get vertex elements used when during PSO precaching materials using this vertex factory type
	*/
	static NIAGARAVERTEXFACTORIES_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	static NIAGARAVERTEXFACTORIES_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, bool bSupportsManualVertexFetch, FStaticMeshDataType& Data, FVertexDeclarationElementList& Elements);

	/**
	* An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	*/
	NIAGARAVERTEXFACTORIES_API void SetData(FRHICommandListBase& RHICmdList, const FStaticMeshDataType& InData);

	/**
	* Set the uniform buffer for this vertex factory.
	*/
	inline void SetUniformBuffer(const FNiagaraMeshUniformBufferRef& InMeshParticleUniformBuffer)
	{
		MeshParticleUniformBuffer = InMeshParticleUniformBuffer;
	}

	/**
	* Retrieve the uniform buffer for this vertex factory.
	*/
	inline FRHIUniformBuffer* GetUniformBuffer() const
	{
		return MeshParticleUniformBuffer;
	}

	inline FRHIShaderResourceView* GetTangentsSRV() const
	{
		return Data.TangentsSRV;
	}

	inline FRHIShaderResourceView* GetTextureCoordinatesSRV() const
	{
		return Data.TextureCoordinatesSRV;
	}

	inline FRHIShaderResourceView* GetColorComponentsSRV() const
	{
		return Data.ColorComponentsSRV;
	}

	inline uint32 GetColorIndexMask() const
	{
		return Data.ColorIndexMask;
	}

	inline int GetNumTexcoords() const
	{
		return Data.NumTexCoords;
	}

	// FRenderResource interface.
	NIAGARAVERTEXFACTORIES_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	bool IsPrimitiveIDElementEnabled() const { return bAddPrimitiveIDElement; }
	void EnablePrimitiveIDElement(bool bEnable) { bAddPrimitiveIDElement = bEnable; }
	
	inline void SetNiagaraMeshFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel)
	{
		FNiagaraVertexFactoryBase::SetFeatureLevel(InFeatureLevel);
	}

	NIAGARAVERTEXFACTORIES_API static bool IsGPUSceneEnabled(const FStaticShaderPlatform Platform, const FStaticFeatureLevel FeatureLevel);

protected:
	static NIAGARAVERTEXFACTORIES_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, bool bSupportsManualVertexFetch, FStaticMeshDataType& Data, FVertexDeclarationElementList& Elements, FVertexStreamList& InOutStreams);

protected:
	FStaticMeshDataType Data;
	bool bAddPrimitiveIDElement;

	/** Uniform buffer with mesh particle parameters. */
	FRHIUniformBuffer* MeshParticleUniformBuffer;
};
