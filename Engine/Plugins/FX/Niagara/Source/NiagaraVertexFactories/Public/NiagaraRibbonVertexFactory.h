// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.h: Particle vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "NiagaraVertexFactory.h"
#include "SceneView.h"
#include "NiagaraDataSet.h"
#
class FMaterial;

//	FNiagaraRibbonVertexDynamicParameter
struct FNiagaraRibbonVertexDynamicParameter
{
	/** The dynamic parameter of the particle			*/
	float DynamicValue[4];
};

/**
* Uniform buffer for particle beam/trail vertex factories.
*/
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraRibbonUniformParameters, NIAGARAVERTEXFACTORIES_API)
	SHADER_PARAMETER(FVector4f, CameraRight)
	SHADER_PARAMETER(FVector4f, CameraUp)
	SHADER_PARAMETER(FVector4f, ScreenAlignment)
	SHADER_PARAMETER(int, PositionDataOffset)
	SHADER_PARAMETER(int, PrevPositionDataOffset)
	SHADER_PARAMETER(int, VelocityDataOffset)
	SHADER_PARAMETER(int, WidthDataOffset)
	SHADER_PARAMETER(int, PrevWidthDataOffset)
	SHADER_PARAMETER(int, TwistDataOffset)
	SHADER_PARAMETER(int, PrevTwistDataOffset)
	SHADER_PARAMETER(int, ColorDataOffset)
	SHADER_PARAMETER(int, FacingDataOffset)
	SHADER_PARAMETER(int, PrevFacingDataOffset)
	SHADER_PARAMETER(int, NormalizedAgeDataOffset)
	SHADER_PARAMETER(int, MaterialRandomDataOffset)
	SHADER_PARAMETER(uint32, MaterialParamValidMask)
	SHADER_PARAMETER(int, MaterialParamDataOffset)
	SHADER_PARAMETER(int, MaterialParam1DataOffset)
	SHADER_PARAMETER(int, MaterialParam2DataOffset)
	SHADER_PARAMETER(int, MaterialParam3DataOffset)
	SHADER_PARAMETER(int, DistanceFromStartOffset)
	SHADER_PARAMETER(int, U0OverrideDataOffset)
	SHADER_PARAMETER(int, V0RangeOverrideDataOffset)
	SHADER_PARAMETER(int, U1OverrideDataOffset)
	SHADER_PARAMETER(int, V1RangeOverrideDataOffset)
	SHADER_PARAMETER(int, InterpCount)
	SHADER_PARAMETER(float, OneOverInterpCount)
	SHADER_PARAMETER(int, ParticleIdShift)
	SHADER_PARAMETER(int, ParticleIdMask)
	SHADER_PARAMETER(int, InterpIdShift)
	SHADER_PARAMETER(int, InterpIdMask)
	SHADER_PARAMETER(int, SliceVertexIdMask)
	SHADER_PARAMETER(int, ShouldFlipNormalToView)
	SHADER_PARAMETER(int, ShouldUseMultiRibbon)
	SHADER_PARAMETER(int, U0DistributionMode)
	SHADER_PARAMETER(int, U1DistributionMode)
	SHADER_PARAMETER(FVector3f, SystemLWCTile)
	SHADER_PARAMETER(FVector4f, PackedVData)
	SHADER_PARAMETER(uint32, bLocalSpace)
	SHADER_PARAMETER_EX(float, DeltaSeconds, EShaderPrecisionModifier::Half)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
typedef TUniformBufferRef<FNiagaraRibbonUniformParameters> FNiagaraRibbonUniformBufferRef;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraRibbonVFLooseParameters, NIAGARAVERTEXFACTORIES_API)
	SHADER_PARAMETER(uint32, NiagaraFloatDataStride)
	SHADER_PARAMETER(uint32, FacingMode)
	SHADER_PARAMETER(uint32, Shape)
	SHADER_PARAMETER(uint32, NeedsPreciseMotionVectors)
	SHADER_PARAMETER(uint32, UseGeometryNormals)
	SHADER_PARAMETER(uint32, UseIndexBufferForRayTracing)
	SHADER_PARAMETER_SRV(Buffer<uint>, IndexBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<float>, TangentsAndDistances)
	SHADER_PARAMETER_SRV(Buffer<uint>, MultiRibbonIndices)
	SHADER_PARAMETER_SRV(Buffer<uint>, PackedPerRibbonDataByIndex)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataFloat)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataHalf)
	SHADER_PARAMETER_SRV(Buffer<float>, SliceVertexData)
	SHADER_PARAMETER_SRV(Buffer<uint>, IndirectDrawOutput)
	SHADER_PARAMETER(int32, IndirectDrawOutputOffset)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
typedef TUniformBufferRef<FNiagaraRibbonVFLooseParameters> FNiagaraRibbonVFLooseParametersRef;

/**
* Beam/Trail particle vertex factory.
*/
class FNiagaraRibbonVertexFactory : public FNiagaraVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FNiagaraRibbonVertexFactory, NIAGARAVERTEXFACTORIES_API);

public:

	/** Default constructor. */
	FNiagaraRibbonVertexFactory(ENiagaraVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel)
		: FNiagaraVertexFactoryBase(InType, InFeatureLevel)
		, DataSet(nullptr)
	{}

	FNiagaraRibbonVertexFactory()
		: FNiagaraVertexFactoryBase(NVFT_MAX, ERHIFeatureLevel::Num)
		, DataSet(nullptr)
	{}

	/**
	* Should we cache the material's shadertype on this platform with this vertex factory?
	*/
	static NIAGARAVERTEXFACTORIES_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	* Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	*/
	static NIAGARAVERTEXFACTORIES_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	* Get vertex elements used when during PSO precaching materials using this vertex factory type
	*/
	static NIAGARAVERTEXFACTORIES_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	// FRenderResource interface.
	NIAGARAVERTEXFACTORIES_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	* Set the uniform buffer for this vertex factory.
	*/
	inline void SetRibbonUniformBuffer(FNiagaraRibbonUniformBufferRef InSpriteUniformBuffer)
	{
		NiagaraRibbonUniformBuffer = InSpriteUniformBuffer;
	}

	/**
	* Retrieve the uniform buffer for this vertex factory.
	*/
	inline FNiagaraRibbonUniformBufferRef GetRibbonUniformBuffer() const
	{
		return NiagaraRibbonUniformBuffer;
	}

	/**
	* Set the source vertex buffer.
	*/
	NIAGARAVERTEXFACTORIES_API void SetVertexBuffer(const FVertexBuffer* InBuffer, uint32 StreamOffset, uint32 Stride);

	/**
	* Set the source vertex buffer that contains particle dynamic parameter data.
	*/
	NIAGARAVERTEXFACTORIES_API void SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, int32 ParameterIndex, uint32 StreamOffset, uint32 Stride);


	FUniformBufferRHIRef LooseParameterUniformBuffer;

private:

	/** Uniform buffer with beam/trail parameters. */
	FNiagaraRibbonUniformBufferRef NiagaraRibbonUniformBuffer;


	const FNiagaraDataSet *DataSet;
};
