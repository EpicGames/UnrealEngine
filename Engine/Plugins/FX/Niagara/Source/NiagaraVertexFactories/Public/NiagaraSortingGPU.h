// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraSortingGPU.h: Niagara sorting shaders
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderPermutation.h"
#include "ShaderParameterStruct.h"

struct FNiagaraGPUSortInfo;

extern NIAGARAVERTEXFACTORIES_API int32 GNiagaraGPUSortingCPUToGPUThreshold;
extern NIAGARAVERTEXFACTORIES_API int32 GNiagaraGPUCullingCPUToGPUThreshold;

#define NIAGARA_KEY_GEN_THREAD_COUNT 64
#define NIAGARA_COPY_BUFFER_THREAD_COUNT 64
#define NIAGARA_COPY_BUFFER_BUFFER_COUNT 3
#define NIAGARA_KEY_GEN_MAX_CULL_PLANES 12

/**
 * Compute shader used to generate particle sort keys.
 */
class FNiagaraSortKeyGenCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNiagaraSortKeyGenCS, NIAGARAVERTEXFACTORIES_API);
	SHADER_USE_PARAMETER_STRUCT(FNiagaraSortKeyGenCS, FGlobalShader);

public:
	class FEnableCulling : SHADER_PERMUTATION_BOOL("ENABLE_CULLING");
	class FSortUsingMaxPrecision : SHADER_PERMUTATION_BOOL("SORT_MAX_PRECISION");
	class FUseWaveOps : SHADER_PERMUTATION_BOOL("DIM_USE_WAVE_OPS");
	using FPermutationDomain = TShaderPermutationDomain<FEnableCulling, FSortUsingMaxPrecision, FUseWaveOps>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, NIAGARAVERTEXFACTORIES_API)
		SHADER_PARAMETER_SRV(Buffer, NiagaraParticleDataFloat)
		SHADER_PARAMETER_SRV(Buffer, NiagaraParticleDataHalf)
		SHADER_PARAMETER_SRV(Buffer, NiagaraParticleDataInt)
		SHADER_PARAMETER_SRV(Buffer, GPUParticleCountBuffer)

		SHADER_PARAMETER(uint32, FloatDataStride)
		SHADER_PARAMETER(uint32, HalfDataStride)
		SHADER_PARAMETER(uint32, IntDataStride)
		SHADER_PARAMETER(uint32, ParticleCount)
		SHADER_PARAMETER(uint32, GPUParticleCountOffset)
		SHADER_PARAMETER(uint32, CulledGPUParticleCountOffset)
		SHADER_PARAMETER(uint32, EmitterKey)
		SHADER_PARAMETER(uint32, OutputOffset)
		SHADER_PARAMETER(FVector3f, CameraPosition)
		SHADER_PARAMETER(FVector3f, CameraDirection)
		SHADER_PARAMETER(uint32, SortMode)
		SHADER_PARAMETER(uint32, SortAttributeOffset)
		SHADER_PARAMETER(uint32, SortKeyMask)
		SHADER_PARAMETER(uint32, SortKeyShift)
		SHADER_PARAMETER(uint32, SortKeySignBit)
		SHADER_PARAMETER(uint32, CullPositionAttributeOffset)
		SHADER_PARAMETER(uint32, CullOrientationAttributeOffset)
		SHADER_PARAMETER(uint32, CullScaleAttributeOffset)
		SHADER_PARAMETER(int32, NumCullPlanes)
		SHADER_PARAMETER_ARRAY(FVector4f, CullPlanes, [NIAGARA_KEY_GEN_MAX_CULL_PLANES])
		SHADER_PARAMETER(int32, RendererVisibility)
		SHADER_PARAMETER(uint32, RendererVisTagAttributeOffset)
		SHADER_PARAMETER(int32, MeshIndex)
		SHADER_PARAMETER(uint32, MeshIndexAttributeOffset)
		SHADER_PARAMETER(FVector2f, CullDistanceRangeSquared)
		SHADER_PARAMETER(FVector4f, LODScreenSize)
		SHADER_PARAMETER(FVector4f, LocalBoundingSphere)
		SHADER_PARAMETER(FVector3f, CullingWorldSpaceOffset)
		SHADER_PARAMETER(FVector3f, SystemLWCTile)

		SHADER_PARAMETER_UAV(Buffer, OutKeys)
		SHADER_PARAMETER_UAV(Buffer, OutParticleIndices)
		SHADER_PARAMETER_UAV(Buffer, OutCulledParticleCounts)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	NIAGARAVERTEXFACTORIES_API static bool UseWaveOps(EShaderPlatform ShaderPlatform);
};
