// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.h: Particle vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "NiagaraVertexFactory.h"
#include "NiagaraDataSet.h"
#include "SceneView.h"

class FMaterial;

/**
 * Uniform buffer for particle sprite vertex factories.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FNiagaraSpriteUniformParameters, NIAGARAVERTEXFACTORIES_API)
	SHADER_PARAMETER( uint32, bLocalSpace)
	SHADER_PARAMETER_EX( FVector4f, TangentSelector, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, NormalsSphereCenter, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, NormalsCylinderUnitDirection, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, SubImageSize, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector3f, CameraFacingBlend, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, RemoveHMDRoll, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER( FVector4f, MacroUVParameters )
	SHADER_PARAMETER_EX( float, RotationScale, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, RotationBias, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, NormalsType, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, DeltaSeconds, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector2f, DefaultPivotOffset, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector2f, DefaultPrevPivotOffset, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER(int, PositionDataOffset)
	SHADER_PARAMETER(int, PrevPositionDataOffset)
	SHADER_PARAMETER(int, VelocityDataOffset)
	SHADER_PARAMETER(int, PrevVelocityDataOffset)
	SHADER_PARAMETER(int, RotationDataOffset)
	SHADER_PARAMETER(int, PrevRotationDataOffset)
	SHADER_PARAMETER(int, SizeDataOffset)
	SHADER_PARAMETER(int, PrevSizeDataOffset)
	SHADER_PARAMETER(int, SubimageDataOffset)
	SHADER_PARAMETER(int, ColorDataOffset)
	SHADER_PARAMETER(uint32, MaterialParamValidMask)
	SHADER_PARAMETER(int, MaterialParamDataOffset)
	SHADER_PARAMETER(int, MaterialParam1DataOffset)
	SHADER_PARAMETER(int, MaterialParam2DataOffset)
	SHADER_PARAMETER(int, MaterialParam3DataOffset)
	SHADER_PARAMETER(int, FacingDataOffset)
	SHADER_PARAMETER(int, PrevFacingDataOffset)
	SHADER_PARAMETER(int, AlignmentDataOffset)
	SHADER_PARAMETER(int, PrevAlignmentDataOffset)
	SHADER_PARAMETER(int, SubImageBlendMode)
	SHADER_PARAMETER(int, CameraOffsetDataOffset)
	SHADER_PARAMETER(int, PrevCameraOffsetDataOffset)
	SHADER_PARAMETER(int, UVScaleDataOffset)
	SHADER_PARAMETER(int, PivotOffsetDataOffset)
	SHADER_PARAMETER(int, PrevPivotOffsetDataOffset)
	SHADER_PARAMETER(int, NormalizedAgeDataOffset)
	SHADER_PARAMETER(int, MaterialRandomDataOffset)
	SHADER_PARAMETER(FVector4f, DefaultPos)
	SHADER_PARAMETER(FVector4f, DefaultPrevPos)
	SHADER_PARAMETER(FVector2f, DefaultSize)
	SHADER_PARAMETER(FVector2f, DefaultPrevSize)
	SHADER_PARAMETER(FVector2f, DefaultUVScale)
	SHADER_PARAMETER(FVector3f, DefaultVelocity)
	SHADER_PARAMETER(FVector3f, DefaultPrevVelocity)
	SHADER_PARAMETER(FVector3f, SystemLWCTile)
	SHADER_PARAMETER(float, DefaultRotation)
	SHADER_PARAMETER(float, DefaultPrevRotation)
	SHADER_PARAMETER(FVector4f, DefaultColor)
	SHADER_PARAMETER(float, DefaultMatRandom)
	SHADER_PARAMETER(float, DefaultCamOffset)
	SHADER_PARAMETER(float, DefaultPrevCamOffset)
	SHADER_PARAMETER(float, DefaultNormAge)
	SHADER_PARAMETER(float, DefaultSubImage)
	SHADER_PARAMETER(FVector4f, DefaultFacing)
	SHADER_PARAMETER(FVector4f, DefaultPrevFacing)
	SHADER_PARAMETER(FVector4f, DefaultAlignment)
	SHADER_PARAMETER(FVector4f, DefaultPrevAlignment)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter0)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter1)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter2)
	SHADER_PARAMETER(FVector4f, DefaultDynamicMaterialParameter3)
	SHADER_PARAMETER(int, PixelCoverageEnabled)
	SHADER_PARAMETER(FVector4f, PixelCoverageColorBlend)
	SHADER_PARAMETER(int, AccurateMotionVectors)
	END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FNiagaraSpriteUniformParameters> FNiagaraSpriteUniformBufferRef;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNiagaraSpriteVFLooseParameters, NIAGARAVERTEXFACTORIES_API)
	SHADER_PARAMETER(uint32, CutoutParameters)
	SHADER_PARAMETER(uint32, NiagaraFloatDataStride)
	SHADER_PARAMETER(uint32, ParticleAlignmentMode)
	SHADER_PARAMETER(uint32, ParticleFacingMode)
	SHADER_PARAMETER(uint32, SortedIndicesOffset)
	SHADER_PARAMETER(uint32, IndirectArgsOffset)
	SHADER_PARAMETER_SRV(Buffer<float2>, CutoutGeometry)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataFloat)
	SHADER_PARAMETER_SRV(Buffer<float>, NiagaraParticleDataHalf)
	SHADER_PARAMETER_SRV(Buffer<uint>, SortedIndices)
	SHADER_PARAMETER_SRV(Buffer<uint>, IndirectArgsBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FNiagaraSpriteVFLooseParameters> FNiagaraSpriteVFLooseParametersRef;

/**
 * Vertex factory for rendering particle sprites.
 */
class FNiagaraSpriteVertexFactory : public FNiagaraVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FNiagaraSpriteVertexFactory, NIAGARAVERTEXFACTORIES_API);

public:

	/** Default constructor. */
	FNiagaraSpriteVertexFactory(ENiagaraVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel ) : FNiagaraVertexFactoryBase(InType, InFeatureLevel) {}
	FNiagaraSpriteVertexFactory() : FNiagaraVertexFactoryBase(NVFT_MAX, ERHIFeatureLevel::Num) {}

	// FRenderResource interface.
	NIAGARAVERTEXFACTORIES_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual bool RendersPrimitivesAsCameraFacingSprites() const override { return true; }

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static NIAGARAVERTEXFACTORIES_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	 */
	static NIAGARAVERTEXFACTORIES_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	* Get vertex elements used when during PSO precaching materials using this vertex factory type
	*/
	static NIAGARAVERTEXFACTORIES_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	NIAGARAVERTEXFACTORIES_API void SetTexCoordBuffer(const FVertexBuffer* InTexCoordBuffer);
	
	void SetSpriteUniformBuffer(const FNiagaraSpriteUniformBufferRef& InSpriteUniformBuffer ) {SpriteUniformBuffer = InSpriteUniformBuffer; }
	FRHIUniformBuffer* GetSpriteUniformBuffer() const { return SpriteUniformBuffer; }

	void SetCutoutParameters(bool bUseSubImage, int32 InNumCutoutVertexPerSubImage)
	{
		check((InNumCutoutVertexPerSubImage & 1) == 0);
		CutoutParameters = InNumCutoutVertexPerSubImage | (bUseSubImage ? 1 : 0);
	}
	uint32 GetCutoutParameters() const { return CutoutParameters; }

	void SetCutoutGeometry(FRHIShaderResourceView* InCutoutGeometrySRV) { CutoutGeometrySRV = InCutoutGeometrySRV; }
	inline FRHIShaderResourceView* GetCutoutGeometrySRV() const { return CutoutGeometrySRV; }

	void SetSortedIndices(const FShaderResourceViewRHIRef& InSortedIndicesSRV, uint32 InSortedIndicesOffset)
	{
		SortedIndicesSRV = InSortedIndicesSRV;
		SortedIndicesOffset = InSortedIndicesOffset;
	}

	FRHIShaderResourceView* GetSortedIndicesSRV() const { return SortedIndicesSRV; }
	int32 GetSortedIndicesOffset() const { return SortedIndicesOffset; }

	void SetFacingMode(uint32 InMode) { FacingMode = InMode; }
	uint32 GetFacingMode() const { return FacingMode; }

	void SetAlignmentMode(uint32 InMode) { AlignmentMode = InMode; }
	uint32 GetAlignmentMode() const { return AlignmentMode; }

	void SetVertexBufferOverride(const FVertexBuffer* InVertexBufferOverride) { VertexBufferOverride = InVertexBufferOverride; }

	void SetLooseParameterUniformBuffer(FUniformBufferRHIRef InBuffer) { LooseParameterUniformBuffer = InBuffer; }
	FRHIUniformBuffer* GetLooseParameterUniformBuffer() const { return LooseParameterUniformBuffer; }

protected:
	/** Initialize streams for this vertex factory. */
	NIAGARAVERTEXFACTORIES_API void InitStreams();

private:

	const FVertexBuffer*		VertexBufferOverride = nullptr;
	FUniformBufferRHIRef		LooseParameterUniformBuffer;

	/** Uniform buffer with sprite parameters. */
	FUniformBufferRHIRef		SpriteUniformBuffer;
	uint32						CutoutParameters = 0;
	FShaderResourceViewRHIRef	CutoutGeometrySRV;
	uint32						AlignmentMode = 0;
	uint32						FacingMode = 0;

	FShaderResourceViewRHIRef	SortedIndicesSRV;
	uint32						SortedIndicesOffset = 0;
};
