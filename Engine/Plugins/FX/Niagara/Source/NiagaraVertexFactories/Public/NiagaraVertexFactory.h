// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.h: Particle vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "SceneView.h"

class FMaterial;


class FNiagaraNullSortedIndicesVertexBuffer : public FVertexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("FNiagaraNullSortedIndicesVertexBuffer"), 1)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionZeroData();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		VertexBufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}

	virtual void ReleaseRHI() override
	{
		VertexBufferSRV.SafeRelease();
		FVertexBuffer::ReleaseRHI();
	}

	FShaderResourceViewRHIRef VertexBufferSRV;
};
extern NIAGARAVERTEXFACTORIES_API TGlobalResource<FNiagaraNullSortedIndicesVertexBuffer> GFNiagaraNullSortedIndicesVertexBuffer;

/**
* Enum identifying the type of a particle vertex factory.
*/
enum ENiagaraVertexFactoryType
{
	NVFT_Sprite,
	NVFT_Ribbon,
	NVFT_Mesh,
	NVFT_MAX
};

/**
* Base class for particle vertex factories.
*/
class FNiagaraVertexFactoryBase : public FVertexFactory
{
public:

	/** Default constructor. */
	explicit FNiagaraVertexFactoryBase(ENiagaraVertexFactoryType Type, ERHIFeatureLevel::Type InFeatureLevel)
		: FVertexFactory(InFeatureLevel)
	{
		bNeedsDeclaration = false;
	}
	
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("NIAGARA_PARTICLE_FACTORY"), TEXT("1"));
	}

	ERHIFeatureLevel::Type GetFeatureLevel() const { check(HasValidFeatureLevel());  return FRenderResource::GetFeatureLevel(); }
};

/**
* Base class for Niagara vertex factory shader parameters.
*/
class FNiagaraVertexFactoryShaderParametersBase : public FVertexFactoryShaderParameters
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FNiagaraVertexFactoryShaderParametersBase, NIAGARAVERTEXFACTORIES_API, NonVirtual);

public:
	NIAGARAVERTEXFACTORIES_API void Bind(const FShaderParameterMap& ParameterMap);
	NIAGARAVERTEXFACTORIES_API void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType VertexStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const;
};
